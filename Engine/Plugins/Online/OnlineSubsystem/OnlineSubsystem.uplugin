{
	"FileVersion": 3,

	"FriendlyName": "Online Subsystem",
	"Version": 1,
	"VersionName": "1.0",
	"Description": "Shared code for interacting online subsystem implementations.",
	"Category": "Online Platform",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": true,

	"Modules": [
		{
			"Name": "OnlineSubsystem",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		}
	],
	"LocalizationTargets": [
		{
			"Name": "OnlineSubsystem",
			"LoadingPolicy": "Always"
		}
	],
	"Plugins": [
		{
			"Name": "OnlineBase",
			"Enabled": true
		}
	]
}