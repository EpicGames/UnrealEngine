// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class OnlineSubsystem : ModuleRules
{
	public OnlineSubsystem(ReadOnlyTargetRules Target) : base(Target)
    {
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		// Required for Test cpp files in OnlineSubsystemMcp
		PublicIncludePaths.Add(Path.Join(ModuleDirectory, "Test"));
		
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Json",
				"CoreOnline",
				"OnlineBase",
				"SignalProcessing"
			}
		);

		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"CoreOnline",
			}
		);

		PublicDefinitions.Add("ONLINESUBSYSTEM_PACKAGE=1");
		PublicDefinitions.Add("DEBUG_LAN_BEACON=0");

		// OnlineSubsystem cannot depend on Engine!
		PrivateDependencyModuleNames.AddRange(
			new string[] { 
				"Core",
				"CoreUObject",
				"JsonUtilities",
			}
		);
	}
}
