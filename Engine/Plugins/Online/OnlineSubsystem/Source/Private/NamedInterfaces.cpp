// Copyright Epic Games, Inc. All Rights Reserved.

#include "NamedInterfaces.h"
#include "UObject/Package.h"
#include "OnlineSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NamedInterfaces)

UNamedInterfaces::UNamedInterfaces(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer)
{
}

void UNamedInterfaces::BeginDestroy()
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		OnCleanup().Broadcast();
	}
	Super::BeginDestroy();
}

void UNamedInterfaces::Initialize()
{
	// Iterate through each configured named interface load it and create an instance
	for (int32 InterfaceIndex = 0; InterfaceIndex < NamedInterfaceDefs.Num(); InterfaceIndex++)
	{
		const FNamedInterfaceDef& Def = NamedInterfaceDefs[InterfaceIndex];
		// Load the specified interface class name
		UClass* Class = LoadClass<UObject>(NULL, *Def.InterfaceClassName, NULL, LOAD_None, NULL);
		if (Class)
		{
			int32 AddIndex = NamedInterfaces.AddZeroed();
			FNamedInterface& Interface = NamedInterfaces[AddIndex];
			// Set the object and interface names
			Interface.InterfaceName = Def.InterfaceName;
			Interface.InterfaceObject = NewObject<UObject>(GetTransientPackage(), Class);
			UE_LOG_ONLINE(Display,
				TEXT("Created named interface (%s) of type (%s)"),
				*Def.InterfaceName.ToString(),
				*Def.InterfaceClassName);
		}
		else
		{
			UE_LOG_ONLINE(Warning,
				TEXT("Failed to load class (%s) for named interface (%s)"),
				*Def.InterfaceClassName,
				*Def.InterfaceName.ToString());
		}
	}
}

UObject* UNamedInterfaces::GetNamedInterface(FName InterfaceName) const
{
	for (const FNamedInterface& Interface : NamedInterfaces)
	{
		if (Interface.InterfaceName == InterfaceName)
		{
			return Interface.InterfaceObject;
		}
	}

	return NULL;
}

void UNamedInterfaces::SetNamedInterface(FName InterfaceName, UObject* NewInterface)
{
	int32 InterfaceIdx = 0;
	for (; InterfaceIdx < NamedInterfaces.Num(); InterfaceIdx++)
	{
		const FNamedInterface& Interface = NamedInterfaces[InterfaceIdx];
		if (Interface.InterfaceName == InterfaceName)
		{
			break;
		}
	}

	if (InterfaceIdx >= NamedInterfaces.Num())
	{
		int32 AddIndex = NamedInterfaces.AddZeroed();
		FNamedInterface& Interface = NamedInterfaces[AddIndex];
		// Set the object and interface names
		Interface.InterfaceName = InterfaceName;
		Interface.InterfaceObject = NewInterface;
	}
	else
	{
		NamedInterfaces[InterfaceIdx].InterfaceObject = NewInterface;
	}
}

