// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineAsyncTaskManager.h"
#include "HAL/Event.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ScopeLock.h"
#include "HAL/IConsoleManager.h"
#include "OnlineSubsystem.h"
#include "Stats/StatsTrace.h"

int32 FOnlineAsyncTaskManager::InvocationCount = 0;

#if !UE_BUILD_SHIPPING
namespace OSSConsoleVariables
{
	/** Time to delay finalization of a task in the out queue */
	TAutoConsoleVariable<float> CVarDelayAsyncTaskOutQueue(
		TEXT("OSS.DelayAsyncTaskOutQueue"),
		0.0f,
		TEXT("Min total async task time\n")
		TEXT("Time in secs"),
		ECVF_Default);
}
#endif

/** The default value for the polling interval when not set by config */
#define POLLING_INTERVAL_MS 20

FOnlineAsyncTaskManager::FOnlineAsyncTaskManager() :
	ActiveSerialTask(nullptr),
	MaxParallelTasks(8),
	bReloadMaxParallelTasksConfig(false),
	WorkEvent(FPlatformProcess::GetSynchEventFromPool()),
	PollingInterval(POLLING_INTERVAL_MS),
	bRequestingExit(false),
	OnlineThreadId(0)
{
	int32 PollingConfig = POLLING_INTERVAL_MS;
	if (GConfig->GetInt(TEXT("OnlineSubsystem"), TEXT("PollingIntervalInMs"), PollingConfig, GEngineIni))
	{
		PollingInterval = (uint32)PollingConfig;
	}

	GConfig->GetInt(TEXT("OnlineSubsystem"), TEXT("MaxParallelTasks"), MaxParallelTasks, GEngineIni);
	GConfig->GetBool(TEXT("OnlineSubsystem"), TEXT("bEnableReportBreach"), bEnableReportBreach, GEngineIni);
	GConfig->GetFloat(TEXT("OnlineSubsystem"), TEXT("BreachTimeSeconds"), ConfigBreachTimeSeconds, GEngineIni);
}

FOnlineAsyncTaskManager::~FOnlineAsyncTaskManager()
{
	// Clean up the active serial task
	delete ActiveSerialTask;
	ActiveSerialTask = nullptr;

	// Clean up queued serial tasks
	for (FOnlineAsyncTask* Task : QueuedSerialTasks)
	{
		delete Task;
	}
	QueuedSerialTasks.Reset();

	// Clean up queued parallel tasks (TQueue)
	{
		FOnlineAsyncTask* Task = nullptr;
		while (QueuedParallelTasks.Dequeue(Task))
		{
			delete Task;
		}
	}

	// Clean up in-progress parallel tasks
	for (FOnlineAsyncTask* Task : ParallelTasks)
	{
		delete Task;
	}
	ParallelTasks.Reset();

	// Clean up completed tasks in the out queue
	for (FOnlineAsyncItem* Item : OutQueue)
	{
		delete Item;
	}
	OutQueue.Reset();

	FPlatformProcess::ReturnSynchEventToPool(WorkEvent);
}

bool FOnlineAsyncTaskManager::Init(void)
{
	return WorkEvent != nullptr;
}

uint32 FOnlineAsyncTaskManager::Run(void)
{
	LLM_SCOPE_BYTAG(OnlineSubsystem);

	InvocationCount++;
	// This should not be set yet
	check(OnlineThreadId == 0);
	FPlatformAtomics::InterlockedExchange((volatile int32*)&OnlineThreadId, FPlatformTLS::GetCurrentThreadId());

	do 
	{
		// Wait for a trigger event to start work
		WorkEvent->Wait(PollingInterval);
		if (!bRequestingExit)
		{
			Tick();
		}
	} 
	while (!bRequestingExit);

	return 0;
}

void FOnlineAsyncTaskManager::Stop(void)
{
	int32 NumInTasks = 0;
	{
		UE::TScopeLock Lock(QueuedSerialTasksLock);
		NumInTasks = QueuedSerialTasks.Num();
	}

	int32 NumOutTasks = 0;
	{
		UE::TScopeLock Lock(OutQueueLock);
		NumOutTasks = OutQueue.Num();
	}

	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskManager::Stop() ActiveSerialTask:%p Tasks[%d/%d]"), ActiveSerialTask, NumInTasks, NumOutTasks);

	// Set the variable to requesting exit before we trigger the event
	bRequestingExit = true;
	WorkEvent->Trigger();
}

void FOnlineAsyncTaskManager::Exit(void)
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskManager::Exit() started"));

	OnlineThreadId = 0;
	InvocationCount--;

	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskManager::Exit() finished"));
}

void FOnlineAsyncTaskManager::AddToInQueue(FOnlineAsyncTask* NewTask)
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskManager::AddToInQueue [%s]"), *NewTask->ToString());

	UE::TScopeLock Lock(QueuedSerialTasksLock);
	QueuedSerialTasks.Add(NewTask);
}

void FOnlineAsyncTaskManager::AddToOutQueue(FOnlineAsyncItem* CompletedItem)
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskManager::AddToOutQueue [%s]"), *CompletedItem->ToString());

	UE::TScopeLock Lock(OutQueueLock);
	OutQueue.Add(CompletedItem);
}

void FOnlineAsyncTaskManager::AddToParallelTasks(FOnlineAsyncTask* NewTask)
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskManager::AddToParallelTasks [%s]"), *NewTask->ToString());

	bReloadMaxParallelTasksConfig = true;

	QueuedParallelTasks.Enqueue(NewTask);
}

void FOnlineAsyncTaskManager::RemoveFromParallelTasks(FOnlineAsyncTask* OldTask)
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskManager::RemoveFromParallelTasks [%s]"), *OldTask->ToString());

	FScopeLock LockParallelTasks(&ParallelTasksLock);

	ParallelTasks.Remove( OldTask );
}

void FOnlineAsyncTaskManager::GameTick()
{
	// assert if not game thread
	check(IsInGameThread());

	if (bReloadMaxParallelTasksConfig)
	{
		bReloadMaxParallelTasksConfig = false;
		GConfig->GetInt(TEXT("OnlineSubsystem"), TEXT("MaxParallelTasks"), MaxParallelTasks, GEngineIni);
	}

	int32 NumParallelTasksToStart = 0;
	{
		FScopeLock LockParallelTasks(&ParallelTasksLock);
		NumParallelTasksToStart = MaxParallelTasks - ParallelTasks.Num();
	}

	FOnlineAsyncTask* ParallelTask = nullptr;
	while (NumParallelTasksToStart-- > 0 && QueuedParallelTasks.Dequeue(ParallelTask))
	{
		UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskManager::GameTick Starting parallel task [%s]"), *ParallelTask->ToString());

		ParallelTask->Initialize();

		FScopeLock LockParallelTasks(&ParallelTasksLock);
		ParallelTasks.Add(ParallelTask);
	}

	FOnlineAsyncItem* Item = nullptr;
	int32 CurrentQueueSize = 0;

#if !UE_BUILD_SHIPPING
	const float TimeToWait = OSSConsoleVariables::CVarDelayAsyncTaskOutQueue.GetValueOnGameThread();
#endif

	do 
	{
		Item = nullptr;
		// Grab a completed task from the queue
		{
			UE::TScopeLock LockOutQueue(OutQueueLock);
			CurrentQueueSize = OutQueue.Num();
			if (CurrentQueueSize > 0)
			{
				Item = OutQueue[0];

#if !UE_BUILD_SHIPPING
				if (Item && Item->GetElapsedTime() >= TimeToWait)
				{
					OutQueue.RemoveAt(0);
				}
				else
				{
					Item = nullptr;
					break;
				}
#else
				OutQueue.RemoveAt(0);	
#endif
			}
		}

		if (Item)
		{
#if !UE_BUILD_SHIPPING
			if (TimeToWait > 0.0f)
			{
				UE_LOG_ONLINE(Verbose, TEXT("Async task '%s' finalizing after %f seconds"),
					*Item->ToString(),
					Item->GetElapsedTime());
			}
#endif

			// Finish work and trigger delegates
			Item->Finalize();
			Item->TriggerDelegates();
			delete Item;
			Item = nullptr;
		}
	}
	while (CurrentQueueSize > 1);

	const double TimeNowSeconds = FPlatformTime::Seconds();

	// Detect hung online thread tick
	if(bEnableReportBreach)
	{
		FOnlineAsyncTask* HungTask = nullptr;
		bool bReportHungTick = false;
		{
			FScopeLock Lock(&OnlineThreadTickInfoLock);
			const double BreachTimeSeconds = OnlineThreadTickInfo.TickStartTime + ConfigBreachTimeSeconds;
			bReportHungTick = OnlineThreadTickInfo.bIsTicking && !OnlineThreadTickInfo.bBreachReported && TimeNowSeconds > BreachTimeSeconds;
			if (bReportHungTick)
			{
				OnlineThreadTickInfo.bBreachReported = true;
				HungTask = OnlineThreadTickInfo.CurrentTask;
			}
		}
		if (bReportHungTick)
		{
			UE_LOG_ONLINE(Warning, TEXT("OnlineAsyncTaskManager::GameTick online thread tick breached, Task=[%s]"), HungTask ? *HungTask->ToString() : TEXT("nullptr"));
		}
	}

	int32 QueueSize = 0;
	{
		UE::TScopeLock LockInQueue(QueuedSerialTasksLock);
		QueueSize = QueuedSerialTasks.Num();
	}
	FOnlineAsyncTask* ActiveTask = nullptr;
	{
		FScopeLock Lock(&ActiveSerialTaskLock);
		ActiveTask = ActiveSerialTask;
	}
	if (ActiveTask)
	{
		++QueueSize;
		if (bEnableReportBreach)
		{
			const double BreachTimeSeconds = ActiveSerialTaskStartTime + ConfigBreachTimeSeconds;
			const bool bReportBreach = !bActiveSerialTaskBreachReported && TimeNowSeconds > BreachTimeSeconds;
			if (bReportBreach)
			{
				bActiveSerialTaskBreachReported = true;

				UE_LOG_ONLINE(Warning, TEXT("OnlineAsyncTaskManager::GameTick serial task breached, Task=[%s]"), *ActiveTask->ToString());
				TArray<FOnlineAsyncTask*> SerialTaskQueue;
				{
					UE::TScopeLock LockInQueue(QueuedSerialTasksLock);
					SerialTaskQueue = QueuedSerialTasks;
				}
				for (const FOnlineAsyncTask* Task : SerialTaskQueue)
				{
					UE_LOG_ONLINE(Warning, TEXT("	blocked task [%s]"), *Task->ToString());
				}
			}
		}
	}
	else if (QueueSize > 0)
	{
		// Grab the current task from the queue
		FOnlineAsyncTask* Task = nullptr;
		{
			UE::TScopeLock LockInQueue(QueuedSerialTasksLock);
			Task = QueuedSerialTasks[0];
			QueuedSerialTasks.RemoveAt(0);
		}

		UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskManager::GameTick Starting serial task [%s]"), *Task->ToString());

		// Initialize the task before giving it away to the online thread
		Task->Initialize();
		{
			FScopeLock LockActiveTask(&ActiveSerialTaskLock);
			ActiveSerialTask = Task;
		}
		ActiveSerialTaskStartTime = TimeNowSeconds;
		bActiveSerialTaskBreachReported = false;

		// Wake up the online thread
		WorkEvent->Trigger();
	}

	SET_DWORD_STAT(STAT_Online_AsyncTasks, QueueSize);
}

void FOnlineAsyncTaskManager::Tick()
{
	SCOPE_CYCLE_COUNTER(STAT_Online_Async);
	if (bEnableReportBreach)
	{
		FScopeLock Lock(&OnlineThreadTickInfoLock);
		OnlineThreadTickInfo.bIsTicking = true;
		OnlineThreadTickInfo.TickStartTime = FPlatformTime::Seconds();
		OnlineThreadTickInfo.CurrentTask = nullptr;
		OnlineThreadTickInfo.bBreachReported = false;
	}

	// Tick Online services (possibly callbacks). 
	OnlineTick();

	{
		// Tick all the parallel tasks - Tick unrelated tasks together. 
		TArray<FOnlineAsyncTask*> CopyParallelTasks;

		// Grab a copy of the parallel list
		{
			FScopeLock LockParallelTasks(&ParallelTasksLock);
			CopyParallelTasks = ParallelTasks;
		}

		FOnlineAsyncTask* Task = nullptr;
		for (auto It = CopyParallelTasks.CreateIterator(); It; ++It)
		{
			Task = *It;

			if (bEnableReportBreach)
			{
				FScopeLock Lock(&OnlineThreadTickInfoLock);
				OnlineThreadTickInfo.CurrentTask = Task;
			}

			Task->Tick();
			Task->CancelWhenTimeout();

			if (Task->IsDone())
			{
				if (Task->WasSuccessful())
				{
					UE_LOG_ONLINE(Verbose, TEXT("Async task '%s' succeeded in %f seconds (Parallel)"),
						*Task->ToString(),
						Task->GetElapsedTime());
				}
				else
				{
					UE_LOG_ONLINE(Warning, TEXT("Async task '%s' failed in %f seconds (Parallel)"),
						*Task->ToString(),
						Task->GetElapsedTime());
				}

				// Task is done, remove from the incoming queue and add to the outgoing queue, fixing up the original parallel task queue. 
				RemoveFromParallelTasks(Task);
				AddToOutQueue(Task);
			}
		}
		if (bEnableReportBreach)
		{
			FScopeLock Lock(&OnlineThreadTickInfoLock);
			OnlineThreadTickInfo.CurrentTask = nullptr;
		}
	}

	{
		// Now process the serial "In" queue
		FOnlineAsyncTask* Task = nullptr;
		{
			FScopeLock Lock(&ActiveSerialTaskLock);
			Task = ActiveSerialTask;
		}

		if (Task)
		{
			if (bEnableReportBreach)
			{
				FScopeLock Lock(&OnlineThreadTickInfoLock);
				OnlineThreadTickInfo.CurrentTask = Task;
			}
			Task->Tick();
			Task->CancelWhenTimeout();

			if (Task->IsDone())
			{
				if (Task->WasSuccessful())
				{
					UE_LOG_ONLINE(Verbose, TEXT("Async task '%s' succeeded in %f seconds"),
						*Task->ToString(),
						Task->GetElapsedTime());
				}
				else
				{
					UE_LOG_ONLINE(Warning, TEXT("Async task '%s' failed in %f seconds"),
						*Task->ToString(),
						Task->GetElapsedTime());
				}

				// Task is done, add to the outgoing queue
				AddToOutQueue(Task);

				if (bActiveSerialTaskBreachReported)
				{
					UE_LOG_ONLINE(Warning, TEXT("The breached serial task eventually completed, Task=[%s]"), *Task->ToString());
				}

				{
					FScopeLock Lock(&ActiveSerialTaskLock);
					ActiveSerialTask = nullptr;
				}
			}
		}
	}
	if (bEnableReportBreach)
	{
		FScopeLock Lock(&OnlineThreadTickInfoLock);
		OnlineThreadTickInfo.bIsTicking = false;
	}
}
