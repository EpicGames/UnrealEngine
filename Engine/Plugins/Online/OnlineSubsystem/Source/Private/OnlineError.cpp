// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineError.h"

#define LOCTEXT_NAMESPACE "OnlineError"
#define ONLINE_ERROR_NAMESPACE "errors.com.epicgames.online.generic"

const FOnlineError& FOnlineError::Success() { static FOnlineError Error(EOnlineErrorResult::Success); return Error; }

#if ONLINE_ERROR_LEGACY
FOnlineError::FOnlineError(const TCHAR* const ErrorCodeIn)
	: FOnlineError(FString(ErrorCodeIn))
{
}

FOnlineError::FOnlineError(const int32 ErrorCodeIn)
	: bSucceeded(false)
	, Result(EOnlineErrorResult::FailExtended)
{
	SetFromErrorCode(ErrorCodeIn);
}

void FOnlineError::SetFromErrorCode(const int32 ErrorCodeIn)
{
	ErrorCode = FString::Printf(TEXT("0x%0.8X"), ErrorCodeIn);
	ErrorRaw = ErrorCode;
}

void FOnlineError::SetFromErrorMessage(const FText& InErrorMessage, const int32 InErrorCode)
{
	ErrorMessage = InErrorMessage;
	SetFromErrorCode(InErrorCode);
}
#endif

FOnlineError::FOnlineError(EOnlineErrorResult InResult)
	: Result(InResult)
{
	// Error code and loc text come from default messaging
	SetFromErrorCode(Result, DefaultErrorCode(Result), DefaultErrorMsg(Result));
}

FOnlineError::FOnlineError(EOnlineErrorResult InResult, const FString& InErrorCode, const FText& InErrorMessage)
	: Result(InResult)
{
	SetFromErrorCode(Result, InErrorCode, InErrorMessage);
}

void FOnlineError::SetFromErrorCode(EOnlineErrorResult InResult)
{
	SetFromErrorCode(Result, DefaultErrorCode(Result), DefaultErrorMsg(Result));
}

void FOnlineError::SetFromErrorCode(EOnlineErrorResult InResult, const FString& InErrorCode)
{
	SetFromErrorCode(Result, InErrorCode, FText::GetEmpty());
}

void FOnlineError::SetFromErrorCode(EOnlineErrorResult InResult, const FString& InErrorCode, const FText& InErrorText)
{
	// Legacy
	bSucceeded = (InResult == EOnlineErrorResult::Success);

	if (InResult == EOnlineErrorResult::Unknown || InResult == EOnlineErrorResult::Success)
	{
		Result = InResult;
		ErrorCode = ErrorRaw = FString();
		ErrorMessage = FText::GetEmpty();
	}
	else
	{
		Result = InResult;
		ErrorRaw = InErrorCode;

		if (InErrorCode.IsEmpty())
		{
			ErrorCode = DefaultErrorCode(Result);
		}
		else
		{
			ErrorCode = InErrorCode;
		}

		if (InErrorText.IsEmpty())
		{
			ErrorMessage = DefaultErrorMsg(Result);
		}
		else
		{
			ErrorMessage = InErrorText;
		}
	}
}

FOnlineError::FOnlineError(bool bSucceededIn)
	: bSucceeded(bSucceededIn)
	, Result(EOnlineErrorResult::Unknown)
{
}

FOnlineError::FOnlineError(const FString& ErrorCodeIn)
	: bSucceeded(false)
	, Result(EOnlineErrorResult::FailExtended)
{
	SetFromErrorCode(ErrorCodeIn);
}

FOnlineError::FOnlineError(FString&& ErrorCodeIn)
	: bSucceeded(false)
	, Result(EOnlineErrorResult::FailExtended)
{
	SetFromErrorCode(MoveTemp(ErrorCodeIn));
}

FOnlineError::FOnlineError(const FText& ErrorMessageIn)
	: bSucceeded(false)
	, Result(EOnlineErrorResult::FailExtended)
{
	SetFromErrorMessage(ErrorMessageIn);
}

void FOnlineError::SetFromErrorCode(const FString& ErrorCodeIn)
{
	ErrorCode = ErrorCodeIn;
	ErrorRaw = ErrorCodeIn;
	Result = EOnlineErrorResult::FailExtended;
}

void FOnlineError::SetFromErrorCode(FString&& ErrorCodeIn)
{
	ErrorCode = MoveTemp(ErrorCodeIn);
	ErrorRaw = ErrorCode;
	Result = EOnlineErrorResult::FailExtended;
}

void FOnlineError::SetFromErrorMessage(const FText& ErrorMessageIn)
{
	ErrorMessage = ErrorMessageIn;
	ErrorCode = FTextInspector::GetKey(ErrorMessageIn).Get(GetGenericErrorCode());
	ErrorRaw = ErrorMessageIn.ToString();
	Result = EOnlineErrorResult::FailExtended;
}

const FString FOnlineError::GetGenericErrorCode()
{
	return TEXT("GenericError");
}

FString FOnlineError::ToLogString() const
{
	if (bSucceeded)
	{
		return TEXT("Succeeded");
	}
	else
	{
		return FString::Printf(TEXT("Failure ErrorCode=%s, Message=%s, Raw=%s"),
			*ErrorCode, *ErrorMessage.ToString(), *ErrorRaw);
	}
}

FString FOnlineError::DefaultErrorCode(EOnlineErrorResult Result)
{
	switch (Result)
	{
		case EOnlineErrorResult::Success:
			return FString();
		case EOnlineErrorResult::NoConnection:
			return FString(TEXT("no_connection"));
		case EOnlineErrorResult::RequestFailure:
			return FString(TEXT("request_failure"));
		case EOnlineErrorResult::InvalidCreds:
			return FString(TEXT("invalid_creds"));
		case EOnlineErrorResult::InvalidUser:
			return FString(TEXT("invalid_user"));
		case EOnlineErrorResult::InvalidAuth:
			return FString(TEXT("invalid_auth"));
		case EOnlineErrorResult::AccessDenied:
			return FString(TEXT("access_denied"));
		case EOnlineErrorResult::TooManyRequests:
			return FString(TEXT("too_many_requests"));
		case EOnlineErrorResult::AlreadyPending:
			return FString(TEXT("already_pending"));
		case EOnlineErrorResult::InvalidParams:
			return FString(TEXT("invalid_params"));
		case EOnlineErrorResult::CantParse:
			return FString(TEXT("cant_parse"));
		case EOnlineErrorResult::InvalidResults:
			return FString(TEXT("invalid_results"));
		case EOnlineErrorResult::IncompatibleVersion:
			return FString(TEXT("incompatible_version"));
		case EOnlineErrorResult::NotConfigured:
			return FString(TEXT("not_configured"));
		case EOnlineErrorResult::NotImplemented:
			return FString(TEXT("not_implemented"));
		case EOnlineErrorResult::MissingInterface:
			return FString(TEXT("missing_interface"));
		case EOnlineErrorResult::Canceled:
			return FString(TEXT("canceled"));
		case EOnlineErrorResult::FailExtended:
			return FString(TEXT("fail_extended"));
		case EOnlineErrorResult::MismatchedUser:
			return FString(TEXT("mismatched_user"));
	};
	return FString(TEXT("unknown_error_result"));
}

FText FOnlineError::DefaultErrorMsg(EOnlineErrorResult Result)
{
	switch (Result)
	{
		case EOnlineErrorResult::Success:
			return FText::GetEmpty();
		case EOnlineErrorResult::NoConnection:
			return LOCTEXT("NotConnected", "No valid connection");
		case EOnlineErrorResult::RequestFailure:
			return LOCTEXT("RequestFailure", "Failed to send request");
		case EOnlineErrorResult::InvalidCreds:
			return LOCTEXT("InvalidCreds", "Invalid credentials");
		case EOnlineErrorResult::InvalidUser:
			return LOCTEXT("InvalidUser", "No valid user");
		case EOnlineErrorResult::InvalidAuth:
			return LOCTEXT("InvalidAuth", "No valid auth");
		case EOnlineErrorResult::AccessDenied:
			return LOCTEXT("AccessDenied", "Access denied");
		case EOnlineErrorResult::TooManyRequests:
			return LOCTEXT("TooManyRequests", "Too many requests");
		case EOnlineErrorResult::AlreadyPending:
			return LOCTEXT("AlreadyPending", "Request already pending");
		case EOnlineErrorResult::InvalidParams:
			return LOCTEXT("InvalidParams", "Invalid params specified");
		case EOnlineErrorResult::CantParse:
			return LOCTEXT("CantParse", "Cannot parse results");
		case EOnlineErrorResult::InvalidResults:
			return LOCTEXT("InvalidResults", "Results were invalid");
		case EOnlineErrorResult::IncompatibleVersion:
			return LOCTEXT("IncompatibleVersion", "Incompatible client version");
		case EOnlineErrorResult::NotConfigured:
			return LOCTEXT("NotConfigured", "No valid configuration");
		case EOnlineErrorResult::NotImplemented:
			return LOCTEXT("NotImplemented", "Not implemented");
		case EOnlineErrorResult::MissingInterface:
			return LOCTEXT("MissingInterface", "Interface not found");
		case EOnlineErrorResult::Canceled:
			return LOCTEXT("Canceled", "Operation was canceled");
		case EOnlineErrorResult::FailExtended:
			return LOCTEXT("FailExtended", "Extended error");
		case EOnlineErrorResult::MismatchedUser:
			return LOCTEXT("MismatchedUser", "Title/Store ID Mismatch");
	};
	return LOCTEXT("Unknown", "Unknown error");
}

FOnlineError FOnlineError::CreateError(const FString& ErrorNamespace, EOnlineErrorResult Result)
{
	return CreateError(ErrorNamespace, Result, DefaultErrorCode(Result), DefaultErrorMsg(Result));
}

FOnlineError FOnlineError::CreateError(const FString& ErrorNamespace, EOnlineErrorResult Result, const FString& ErrorCode, const FText& ErrorMessage)
{
	FOnlineError Error(Result, ErrorCode, ErrorMessage);
	if (!Error.GetErrorCode().IsEmpty() && !Error.GetErrorCode().Contains(TEXT("com.")))
	{
		FString Namespace(ErrorNamespace.IsEmpty() ? GetDefaultErrorNamespace() : ErrorNamespace);
		Namespace.Append(TEXT("."));

		// namespace the error code if not using a backend error
		Error.SetErrorCode(Namespace + Error.GetErrorCode());
	}

	return Error;
}

const FString& FOnlineError::GetDefaultErrorNamespace()
{
	static FString DefaultNamespace(TEXT(ONLINE_ERROR_NAMESPACE));
	return DefaultNamespace;
}

FString FOnlineError::GetErrorLegacy()
{
	return FString::Printf(TEXT("errorpath=%s errormessage=%s"), *GetErrorCode(), *GetErrorMessage().ToString());
}

#undef LOCTEXT_NAMESPACE
#undef ONLINE_ERROR_NAMESPACE
