// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/OnlineIdentityInterface.h"

FString ToDebugString(IOnlineIdentity::EPrivilegeResults PrivilegeResults)
{
	if (PrivilegeResults == IOnlineIdentity::EPrivilegeResults::NoFailures)
	{
		return TEXT("NoFailures");
	}
	else
	{
		TArray<FString> ResultNames;
		if ((uint32)PrivilegeResults & (uint32)IOnlineIdentity::EPrivilegeResults::RequiredPatchAvailable) ResultNames.Emplace(TEXT("RequiredPatchAvailable"));
		if ((uint32)PrivilegeResults & (uint32)IOnlineIdentity::EPrivilegeResults::RequiredSystemUpdate) ResultNames.Emplace(TEXT("RequiredSystemUpdate"));
		if ((uint32)PrivilegeResults & (uint32)IOnlineIdentity::EPrivilegeResults::AgeRestrictionFailure) ResultNames.Emplace(TEXT("AgeRestrictionFailure"));
		if ((uint32)PrivilegeResults & (uint32)IOnlineIdentity::EPrivilegeResults::AccountTypeFailure) ResultNames.Emplace(TEXT("AccountTypeFailure"));
		if ((uint32)PrivilegeResults & (uint32)IOnlineIdentity::EPrivilegeResults::UserNotFound) ResultNames.Emplace(TEXT("UserNotFound"));
		if ((uint32)PrivilegeResults & (uint32)IOnlineIdentity::EPrivilegeResults::UserNotLoggedIn) ResultNames.Emplace(TEXT("UserNotLoggedIn"));
		if ((uint32)PrivilegeResults & (uint32)IOnlineIdentity::EPrivilegeResults::ChatRestriction) ResultNames.Emplace(TEXT("ChatRestriction"));
		if ((uint32)PrivilegeResults & (uint32)IOnlineIdentity::EPrivilegeResults::UGCRestriction) ResultNames.Emplace(TEXT("UGCRestriction"));
		if ((uint32)PrivilegeResults & (uint32)IOnlineIdentity::EPrivilegeResults::GenericFailure) ResultNames.Emplace(TEXT("GenericFailure"));
		if ((uint32)PrivilegeResults & (uint32)IOnlineIdentity::EPrivilegeResults::OnlinePlayRestricted) ResultNames.Emplace(TEXT("OnlinePlayRestricted"));
		if ((uint32)PrivilegeResults & (uint32)IOnlineIdentity::EPrivilegeResults::NetworkConnectionUnavailable) ResultNames.Emplace(TEXT("NetworkConnectionUnavailable"));
		check((uint32)PrivilegeResults < ((uint32)IOnlineIdentity::EPrivilegeResults::NetworkConnectionUnavailable << 1));

		return FString::Join(ResultNames, TEXT(" | "));
	}
}

FString ToDebugString(EUserPrivileges::Type UserPrivilege)
{
	switch (UserPrivilege)
	{
	case EUserPrivileges::CanPlay: return TEXT("CanPlay");
	case EUserPrivileges::CanPlayOnline: return TEXT("CanPlayOnline");
	case EUserPrivileges::CanCommunicateOnline: return TEXT("CanCommunicateOnline");
	case EUserPrivileges::CanUseUserGeneratedContent: return TEXT("CanUseUserGeneratedContent");
	case EUserPrivileges::CanUserCrossPlay: return TEXT("CanUserCrossPlay");
	default: return TEXT("Unknown");
	}
}

FString ToDebugString(const FControllerPairingChangedUserInfo& ControllerPairingChangedUserInfo)
{
	return FString::Printf(TEXT("[%s, %d devices]"), *ControllerPairingChangedUserInfo.User.ToDebugString(), ControllerPairingChangedUserInfo.ControllersRemaining);
}

FPlatformUserId IOnlineIdentity::GetPlatformUserIdFromLocalUserNum(int32 LocalUserNum) const
{
	if (LocalUserNum >= 0 && LocalUserNum < MAX_LOCAL_PLAYERS)
	{
		return FPlatformMisc::GetPlatformUserForUserIndex(LocalUserNum);
	}

	return PLATFORMUSERID_NONE;
}

int32 IOnlineIdentity::GetLocalUserNumFromPlatformUserId(FPlatformUserId PlatformUserId) const
{
	int32 Index = FPlatformMisc::GetUserIndexForPlatformUser(PlatformUserId);

	if (Index >= 0 && Index < MAX_LOCAL_PLAYERS)
	{
		return Index;
	}

	return INDEX_NONE;
}
