// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineKeyValuePair.h"
#include "JsonObjectWrapper.h"
#include "UObject/EnumProperty.h"
#include "Misc/DateTime.h"
#include "UObject/TextProperty.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "OnlineSubsystem.h"
#include "JsonObjectConverter.h"

/**
 * Copy constructor. Copies the other into this object
 *
 * @param Other the other structure to copy
 */
FVariantData::FVariantData(const FVariantData& Other)
	: Type(EOnlineKeyValuePairDataType::Empty)
{
	// Use common methods for doing deep copy or just do a simple shallow copy
	if (Other.Type == EOnlineKeyValuePairDataType::String)
	{
		SetValue(Other.Value.AsTCHAR);
	}
	else if (Other.Type == EOnlineKeyValuePairDataType::Json)
	{
		SetJsonValueFromString(Other.Value.AsTCHAR);
	}
	else if (Other.Type == EOnlineKeyValuePairDataType::Blob)
	{
		SetValue(Other.Value.AsBlob.BlobSize, Other.Value.AsBlob.BlobData);
	}
	else
	{
		// Shallow copy is safe
		FMemory::Memcpy(this, &Other, sizeof(FVariantData));
	}
}

FVariantData::FVariantData(FVariantData&& Other)
	: Type(EOnlineKeyValuePairDataType::Empty)
{
	// Copy values/pointers
	FMemory::Memcpy(this, &Other, sizeof(FVariantData));

	// Reset other to empty now so it doesn't try to delete pointers
	Other.Type = EOnlineKeyValuePairDataType::Empty;
	// Clear saved values
	Other.Empty();
}

/**
 * Assignment operator. Copies the other into this object
 *
 * @param Other the other structure to copy
 */
FVariantData& FVariantData::operator=(const FVariantData& Other)
{
	if (this != &Other)
	{
		// Use common methods for doing deep copy or just do a simple shallow copy
		if (Other.Type == EOnlineKeyValuePairDataType::String)
		{
			SetValue(Other.Value.AsTCHAR);
		}
		else if (Other.Type == EOnlineKeyValuePairDataType::Json)
		{
			SetJsonValueFromString(Other.Value.AsTCHAR);
		}
		else if (Other.Type == EOnlineKeyValuePairDataType::Blob)
		{
			SetValue(Other.Value.AsBlob.BlobSize, Other.Value.AsBlob.BlobData);
		}
		else
		{
			Empty();
			// Shallow copy is safe
			FMemory::Memcpy(this, &Other, sizeof(FVariantData));
		}
	}
	return *this;
}

FVariantData& FVariantData::operator=(FVariantData&& Other)
{
	if (this != &Other)
	{
		// Copy values/pointers
		FMemory::Memcpy(this, &Other, sizeof(FVariantData));

		// Reset other to empty now so it doesn't try to delete pointers
		Other.Type = EOnlineKeyValuePairDataType::Empty;
		// Clear saved values
		Other.Empty();
	}

	return *this;
}

/**
 * Copies the data and sets the type
 *
 * @param InData the new data to assign
 */
void FVariantData::SetValue(const TCHAR* InData)
{
	Empty();
	Type = EOnlineKeyValuePairDataType::String;
	if (InData != nullptr)
	{	
		int32 StrLen = FCString::Strlen(InData);
		// Allocate a buffer for the string plus terminator
		Value.AsTCHAR = new TCHAR[StrLen + 1];
		if (StrLen > 0)
		{
			// Copy the data
			FCString::Strncpy(Value.AsTCHAR, InData, StrLen + 1);
		}
		else
		{
			Value.AsTCHAR[0] = TEXT('\0');
		}
	}
}

/**
 * Copies the data and sets the type
 *
 * @param InData the new data to assign
 */
void FVariantData::SetValue(const FString& InData)
{
	SetValue(*InData);
}

/**
 * Copies the data and sets the type
 *
 * @param InData the new data to assign
 */
void FVariantData::SetValue(int32 InData)
{
	Empty();
	Type = EOnlineKeyValuePairDataType::Int32;
	Value.AsInt = InData;
}

/**
 * Copies the data and sets the type
 *
 * @param InData the new data to assign
 */
void FVariantData::SetValue(uint32 InData)
{
	Empty();
	Type = EOnlineKeyValuePairDataType::UInt32;
	Value.AsUInt = InData;
}

/**
 * Copies the data and sets the type
 *
 * @param InData the new data to assign
 */
void FVariantData::SetValue(bool InData)
{
	Empty();
	Type = EOnlineKeyValuePairDataType::Bool;
	Value.AsBool = InData;
}

/**
 * Copies the data and sets the type
 *
 * @param InData the new data to assign
 */
void FVariantData::SetValue(double InData)
{
	Empty();
	Type = EOnlineKeyValuePairDataType::Double;
	Value.AsDouble = InData;
}

/**
 * Copies the data and sets the type
 *
 * @param InData the new data to assign
 */
void FVariantData::SetValue(float InData)
{
	Empty();
	Type = EOnlineKeyValuePairDataType::Float;
	Value.AsFloat = InData;
}

/**
 * Copies the data and sets the type
 *
 * @param InData the new data to assign
 */
void FVariantData::SetValue(const TArray<uint8>& InData)
{
	SetValue((uint32)InData.Num(), (const uint8*)InData.GetData());
}

/**
 * Copies the data and sets the type
 *
 * @param Size the length of the buffer to copy
 * @param InData the new data to assign
 */
void FVariantData::SetValue(uint32 Size, const uint8* InData)
{
	Empty();
	Type = EOnlineKeyValuePairDataType::Blob;
	if (Size > 0)
	{
		// Deep copy the binary data
		Value.AsBlob.BlobSize = Size;
		Value.AsBlob.BlobData = new uint8[Size];
		FMemory::Memcpy(Value.AsBlob.BlobData, InData, Size);
	}
}

/**
 * Copies the data and sets the type
 *
 * @param InData the new data to assign
 */
void FVariantData::SetValue(int64 InData)
{
	Empty();
	Type = EOnlineKeyValuePairDataType::Int64;
	Value.AsInt64 = InData;
}

/**
 * Copies the data and sets the type
 *
 * @param InData the new data to assign
 */
void FVariantData::SetValue(uint64 InData)
{
	Empty();
	Type = EOnlineKeyValuePairDataType::UInt64;
	Value.AsUInt64 = InData;
}

/**
* Copies the data and sets the type
*
* @param InData the new data to assign
*/
void FVariantData::SetValue(const TSharedRef<FJsonObject>& InData)
{
	FString SerializedData;
	auto Writer = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&SerializedData);
	FJsonSerializer::Serialize(InData, Writer);
	SetJsonValueFromString(SerializedData);
}

/**
* Copies the data and sets the type
*
* @param InData the new data to assign
*/
void FVariantData::SetJsonValueFromString(const FString& InData)
{
	SetValue(InData);
	Type = EOnlineKeyValuePairDataType::Json;
}

/**
 * Copies the data after verifying the type
 *
 * @param OutData out value that receives the copied data
 */
void FVariantData::GetValue(FString& OutData) const
{
	if ((Type == EOnlineKeyValuePairDataType::String || Type == EOnlineKeyValuePairDataType::Json) && Value.AsTCHAR != nullptr)
	{
		OutData = Value.AsTCHAR;
	}
	else
	{
		OutData.Reset();
	}
}

/**
 * Copies the data after verifying the type
 *
 * @param OutData out value that receives the copied data
 */
void FVariantData::GetValue(int32& OutData) const
{
	if (Type == EOnlineKeyValuePairDataType::Int32)
	{
		OutData = Value.AsInt;
	}
	else
	{
		OutData = 0;
	}
}

/**
 * Copies the data after verifying the type
 *
 * @param OutData out value that receives the copied data
 */
void FVariantData::GetValue(uint32& OutData) const
{
	if (Type == EOnlineKeyValuePairDataType::UInt32)
	{
		OutData = Value.AsUInt;
	}
	else
	{
		OutData = 0;
	}
}

/**
 * Copies the data after verifying the type
 *
 * @param OutData out value that receives the copied data
 */
void FVariantData::GetValue(bool& OutData) const
{
	if (Type == EOnlineKeyValuePairDataType::Bool)
	{
		OutData = Value.AsBool;
	}
	else
	{
		OutData = false;
	}
}

/**
 * Copies the data after verifying the type
 *
 * @param OutData out value that receives the copied data
 */
void FVariantData::GetValue(int64& OutData) const
{
	if (Type == EOnlineKeyValuePairDataType::Int64)
	{
		OutData = Value.AsInt64;
	}
	else
	{
		OutData = 0;
	}
}

/**
 * Copies the data after verifying the type
 *
 * @param OutData out value that receives the copied data
 */
void FVariantData::GetValue(uint64& OutData) const
{
	if (Type == EOnlineKeyValuePairDataType::UInt64)
	{
		OutData = Value.AsUInt64;
	}
	else
	{
		OutData = 0;
	}
}

/**
 * Copies the data after verifying the type
 *
 * @param OutData out value that receives the copied data
 */
void FVariantData::GetValue(float& OutData) const
{
	if (Type == EOnlineKeyValuePairDataType::Float)
	{
		OutData = Value.AsFloat;
	}
	else
	{
		OutData = 0.f;
	}
}

/**
 * Copies the data after verifying the type
 * NOTE: Performs a deep copy so you are responsible for freeing the data
 *
 * @param OutSize out value that receives the size of the copied data
 * @param OutData out value that receives the copied data
 */
void FVariantData::GetValue(uint32& OutSize, uint8** OutData) const
{
	if (Type == EOnlineKeyValuePairDataType::Blob)
	{
		OutSize = Value.AsBlob.BlobSize;
		// Need to perform a deep copy
		*OutData = new uint8[OutSize];
		FMemory::Memcpy(*OutData, Value.AsBlob.BlobData, OutSize);
	}
	else
	{
		OutSize = 0;
		*OutData = nullptr;
	}
}

/**
 * Copies the data after verifying the type
 *
 * @param OutData out value that receives the copied data
 */
void FVariantData::GetValue(TArray<uint8>& OutData) const
{
	if (Type == EOnlineKeyValuePairDataType::Blob)
	{
		// Presize the array so it only allocates what's needed
		OutData.Empty(Value.AsBlob.BlobSize);
		OutData.AddUninitialized(Value.AsBlob.BlobSize);
		// Copy into the array
		FMemory::Memcpy(OutData.GetData(), Value.AsBlob.BlobData, Value.AsBlob.BlobSize);
	}
	else
	{
		OutData.Empty();
	}
}

/**
 * Copies the data after verifying the type
 *
 * @param OutData out value that receives the copied data
 */
void FVariantData::GetValue(double& OutData) const
{
	if (Type == EOnlineKeyValuePairDataType::Double)
	{
		OutData = Value.AsDouble;
	}
	else
	{
		OutData = 0.0;
	}
}

/**
* Copies the data after verifying the type
*
* @param OutData out value that receives the copied data
*/
void FVariantData::GetValue(TSharedPtr<FJsonObject>& OutData) const
{
	if (Type == EOnlineKeyValuePairDataType::Json)
	{
		FString SerializedData;
		auto Reader = TJsonReaderFactory<TCHAR>::Create(Value.AsTCHAR);
		if (!FJsonSerializer::Deserialize(Reader, OutData))
		{
			OutData = MakeShared<FJsonObject>();
			UE_LOG_ONLINE(Warning, TEXT("Invalid serialized json in FVariantData: %s"), Value.AsTCHAR);
		}
	}
	else
	{
		OutData = MakeShared<FJsonObject>();
	}
}

/**
* Copies the data after verifying the type
*
* @param OutData out value that receives the copied data
*/
void FVariantData::GetValue(TArray<TSharedPtr<FJsonValue>>& OutData) const
{
	if (Type == EOnlineKeyValuePairDataType::Json)
	{
		FString SerializedData;
		auto Reader = TJsonReaderFactory<TCHAR>::Create(Value.AsTCHAR);
		if (!FJsonSerializer::Deserialize(Reader, OutData))
		{
			UE_LOG_ONLINE(Warning, TEXT("Invalid serialized json in FVariantData: %s, Error: %s"),
				Value.AsTCHAR, *Reader->GetErrorMessage());
		}
	}
}


int GetStringDigitCount(int64 Value)
{
	if (Value < 0)
	{
		const int SignLength = 1;
		return int(FMath::CeilToInt64(FMath::LogX(10.f, -Value)) + SignLength);
	}
	else
	{
		return int(FMath::CeilToInt64(FMath::LogX(10.f, Value)));
	}
}

int GetStringDigitCount(uint64 Value)
{
	return int(FMath::CeilToInt64(FMath::LogX(10.f, Value)));
}

int GetStringDigitCount(int32 Value)
{
	return GetStringDigitCount(static_cast<int64>(Value));
}

int GetStringDigitCount(uint32 Value)
{
	return GetStringDigitCount(static_cast<uint64>(Value));
}

/** Returns the maximum size of the value when encoded as an escaped string. */
int FVariantData::GetEstimatedMaxEscapedStringSize() const
{
	if (CachedEstimatedMaxEscapedStringSize != 0)
	{
		return CachedEstimatedMaxEscapedStringSize;
	}

	switch (Type)
	{
		case EOnlineKeyValuePairDataType::Bool:
		{
			// Size of the strings "true" or "false"
			CachedEstimatedMaxEscapedStringSize = 5;
			break;
		}

		case EOnlineKeyValuePairDataType::Float:
		{
			// Maximum string size of a float value.
			const int MaxDigits = 3 + FLT_MANT_DIG - FLT_MIN_EXP;
			CachedEstimatedMaxEscapedStringSize = MaxDigits;
			break;
		}

		case EOnlineKeyValuePairDataType::Double:
		{
			// Maximum string size of a double value.
			const int MaxDigits = 3 + DBL_MANT_DIG - DBL_MIN_EXP;
			CachedEstimatedMaxEscapedStringSize = MaxDigits;
			break;
		}

		case EOnlineKeyValuePairDataType::Int32:
		{
			CachedEstimatedMaxEscapedStringSize = GetStringDigitCount(Value.AsInt);
			break;
		}

		case EOnlineKeyValuePairDataType::UInt32:
		{
			CachedEstimatedMaxEscapedStringSize = GetStringDigitCount(Value.AsUInt);
			break;
		}

		case EOnlineKeyValuePairDataType::Int64:
		{
			CachedEstimatedMaxEscapedStringSize = GetStringDigitCount(Value.AsInt64);
			break;
		}

		case EOnlineKeyValuePairDataType::UInt64:
		{
			CachedEstimatedMaxEscapedStringSize = GetStringDigitCount(Value.AsUInt64);
			break;
		}

		case EOnlineKeyValuePairDataType::Json:
		case EOnlineKeyValuePairDataType::String:
		{
			check(Value.AsTCHAR);

			// Get count of characters including extra padding for escaping sequences.
			for (TCHAR* CharIt = Value.AsTCHAR; *CharIt; ++CharIt)
			{
				const int NumQuotes = *CharIt == TEXT('"');
				const int NumBackSlash = (*CharIt == TEXT('\\')) * 2;
				CachedEstimatedMaxEscapedStringSize += 1 + NumQuotes + NumBackSlash;
			}
			break;
		}

		case EOnlineKeyValuePairDataType::Empty:
		case EOnlineKeyValuePairDataType::Blob:
		default:
		{
			// Values are not exported to json.
			CachedEstimatedMaxEscapedStringSize = 0;
			break;
		}
	}

	return CachedEstimatedMaxEscapedStringSize;
}

/**
 * Increments the numeric value by 1
 */
void FVariantData::Increment()
{
	checkSlow(IsNumeric());
	switch (Type)
	{
	case EOnlineKeyValuePairDataType::Int32:
		Increment<int32, EOnlineKeyValuePairDataType::Int32>(1);
		break;
	case EOnlineKeyValuePairDataType::UInt32:
		Increment<uint32, EOnlineKeyValuePairDataType::UInt32>(1);
		break;
	case EOnlineKeyValuePairDataType::Int64:
		Increment<int64, EOnlineKeyValuePairDataType::Int64>(1);
		break;
	case EOnlineKeyValuePairDataType::UInt64:
		Increment<uint64, EOnlineKeyValuePairDataType::UInt64>(1);
		break;
	case EOnlineKeyValuePairDataType::Double:
		Increment<double, EOnlineKeyValuePairDataType::Double>(1.0);
		break;
	case EOnlineKeyValuePairDataType::Float:
		Increment<float, EOnlineKeyValuePairDataType::Float>(1.0);
		break;
	}
}

/**
 * Decrements the numeric value by 1
 */
void FVariantData::Decrement()
{
	checkSlow(IsNumeric());
	switch (Type)
	{
	case EOnlineKeyValuePairDataType::Int32:
		Decrement<int32, EOnlineKeyValuePairDataType::Int32>(1);
		break;
	case EOnlineKeyValuePairDataType::UInt32:
		Decrement<uint32, EOnlineKeyValuePairDataType::UInt32>(1);
		break;
	case EOnlineKeyValuePairDataType::Int64:
		Decrement<int64, EOnlineKeyValuePairDataType::Int64>(1);
		break;
	case EOnlineKeyValuePairDataType::UInt64:
		Decrement<uint64, EOnlineKeyValuePairDataType::UInt64>(1);
		break;
	case EOnlineKeyValuePairDataType::Double:
		Decrement<double, EOnlineKeyValuePairDataType::Double>(1.0);
		break;
	case EOnlineKeyValuePairDataType::Float:
		Decrement<float, EOnlineKeyValuePairDataType::Float>(1.0);
		break;
	}
}

/**
 * Cleans up the existing data and sets the type to EOnlineKeyValuePairDataType::Empty
 */
void FVariantData::Empty()
{
	// Be sure to delete deep allocations
	switch (Type)
	{
	case EOnlineKeyValuePairDataType::Json:
	case EOnlineKeyValuePairDataType::String:
		delete [] Value.AsTCHAR;
		break;
	case EOnlineKeyValuePairDataType::Blob:
		delete [] Value.AsBlob.BlobData;
		break;
	}

	Type = EOnlineKeyValuePairDataType::Empty;
	FMemory::Memset(&Value, 0, sizeof(ValueUnion));
	CachedEstimatedMaxEscapedStringSize = 0;
}

/**
 * Converts the data into a string representation
 */
FString FVariantData::ToString() const
{
	switch (Type)
	{
		case EOnlineKeyValuePairDataType::Bool:
		{
			bool BoolVal;
			GetValue(BoolVal);
			return FString::Printf(TEXT("%s"), BoolVal ? TEXT("true") : TEXT("false"));
		}
		case EOnlineKeyValuePairDataType::Float:
		{
			// Convert the float to a string
			float FloatVal;
			GetValue(FloatVal);
			return FString::Printf(TEXT("%f"), (double)FloatVal);
		}
		case EOnlineKeyValuePairDataType::Int32:
		{
			// Convert the int to a string
			int32 Val;
			GetValue(Val);
			return FString::Printf(TEXT("%d"), Val);
		}
		case EOnlineKeyValuePairDataType::UInt32:
		{
			// Convert the int to a string
			uint32 Val;
			GetValue(Val);
			return FString::Printf(TEXT("%d"), Val);
		}
		case EOnlineKeyValuePairDataType::Int64:
		{
			// Convert the int to a string
			int64 Val;
			GetValue(Val);
			return FString::Printf(TEXT("%lld"),Val);
		}
		case EOnlineKeyValuePairDataType::UInt64:
		{
			// Convert the int to a string
			uint64 Val;
			GetValue(Val);
			return FString::Printf(TEXT("%lld"), Val);
		}
		case EOnlineKeyValuePairDataType::Double:
		{
			// Convert the double to a string
			double Val;
			GetValue(Val);
			return FString::Printf(TEXT("%f"),Val);
		}
		case EOnlineKeyValuePairDataType::Json:
		case EOnlineKeyValuePairDataType::String:
		{
			// Copy the string out
			FString StringVal;
			GetValue(StringVal);
			return StringVal;
		}
		case EOnlineKeyValuePairDataType::Blob:
		{
			return FString::Printf(TEXT("%d byte blob"), Value.AsBlob.BlobSize);
		}
	}
	return FString();
}

/**
 * Converts the string to the specified type of data for this setting
 *
 * @param NewValue the string value to convert
 */
bool FVariantData::FromString(const FString& NewValue)
{
	switch (Type)
	{
		case EOnlineKeyValuePairDataType::Float:
		{
			// Convert the string to a float
			float FloatVal = FCString::Atof(*NewValue);
			SetValue(FloatVal);
			return true;
		}
		case EOnlineKeyValuePairDataType::Int32:
		{
			// Convert the string to a int
			int32 IntVal = FCString::Atoi(*NewValue);
			SetValue(IntVal);
			return true;
		}
		case EOnlineKeyValuePairDataType::UInt32:
		{
			// Convert the string to a int
			uint64 IntVal = FCString::Strtoui64(*NewValue, nullptr, 10);
			SetValue(static_cast<uint32>(IntVal));
			return true;
		}
		case EOnlineKeyValuePairDataType::Double:
		{
			// Convert the string to a double
			double Val = FCString::Atod(*NewValue);
			SetValue(Val);
			return true;
		}
		case EOnlineKeyValuePairDataType::Int64:
		{
			int64 Val = FCString::Atoi64(*NewValue);
			SetValue(Val);
			return true;
		}
		case EOnlineKeyValuePairDataType::UInt64:
		{
			uint64 Val = FCString::Strtoui64(*NewValue, nullptr, 10);
			SetValue(Val);
			return true;
		}
		case EOnlineKeyValuePairDataType::String:
		{
			// Copy the string
			SetValue(NewValue);
			return true;
		}
		case EOnlineKeyValuePairDataType::Bool:
		{
			bool Val = NewValue.Equals(TEXT("true"), ESearchCase::IgnoreCase) ? true : false;
			SetValue(Val);
			return true;
		}
		case EOnlineKeyValuePairDataType::Blob:
		case EOnlineKeyValuePairDataType::Empty:
			break;
	}
	return false;
}

TSharedRef<class FJsonObject> FVariantData::ToJson() const
{
	TSharedRef<FJsonObject> JsonObject(new FJsonObject());

	const TCHAR* TypeStr = TEXT("Type");
	const TCHAR* ValueStr = TEXT("Value");

	JsonObject->SetStringField(TypeStr, EOnlineKeyValuePairDataType::ToString(Type));
	
	switch (Type)
	{
		case EOnlineKeyValuePairDataType::Int32:
		{
			int32 FieldValue;
			GetValue(FieldValue);
			JsonObject->SetNumberField(ValueStr, (double)FieldValue);
			break;
		}
		case EOnlineKeyValuePairDataType::UInt32:
		{
			uint32 FieldValue;
			GetValue(FieldValue);
			JsonObject->SetNumberField(ValueStr, (double)FieldValue);
			break;
		}
		case EOnlineKeyValuePairDataType::Float:
		{
			float FieldValue;
			GetValue(FieldValue);
			JsonObject->SetNumberField(ValueStr, (double)FieldValue);
			break;
		}
		case EOnlineKeyValuePairDataType::String:
		{
			FString FieldValue;
			GetValue(FieldValue);
			JsonObject->SetStringField(ValueStr, FieldValue);
			break;
		}
		case EOnlineKeyValuePairDataType::Bool:
		{
			bool FieldValue;
			GetValue(FieldValue);
			JsonObject->SetBoolField(ValueStr, FieldValue);
			break;
		}
		case EOnlineKeyValuePairDataType::Int64:
		{
			JsonObject->SetStringField(ValueStr, ToString());
			break;
		}
		case EOnlineKeyValuePairDataType::UInt64:
		{
			JsonObject->SetStringField(ValueStr, ToString());
			break;
		}
		case EOnlineKeyValuePairDataType::Double:
		{
			double FieldValue;
			GetValue(FieldValue);
			JsonObject->SetNumberField(ValueStr, (double)FieldValue);
			break;
		}
		case EOnlineKeyValuePairDataType::Empty:
		case EOnlineKeyValuePairDataType::Blob:
		default:
		{
			JsonObject->SetStringField(ValueStr, FString());
			break;
		}	
	};

	return JsonObject;
}

bool FVariantData::FromJson(const TSharedRef<FJsonObject>& JsonObject)
{
	bool bResult = false;

	const TCHAR* TypeStr = TEXT("Type");
	const TCHAR* ValueStr = TEXT("Value");

	FString VariantTypeStr;
	if (JsonObject->TryGetStringField(TypeStr, VariantTypeStr) &&
		!VariantTypeStr.IsEmpty())
	{
		if (VariantTypeStr.Equals(EOnlineKeyValuePairDataType::ToString(EOnlineKeyValuePairDataType::Int32)))
		{
			int32 FieldValue;
			if (JsonObject->TryGetNumberField(ValueStr, FieldValue))
			{
				SetValue(FieldValue);
				bResult = true;
			}
		}
		else if (VariantTypeStr.Equals(EOnlineKeyValuePairDataType::ToString(EOnlineKeyValuePairDataType::UInt32)))
		{
			uint32 FieldValue;
			if (JsonObject->TryGetNumberField(ValueStr, FieldValue))
			{
				SetValue(FieldValue);
				bResult = true;
			}
		}
		else if (VariantTypeStr.Equals(EOnlineKeyValuePairDataType::ToString(EOnlineKeyValuePairDataType::Float)))
		{
			double FieldValue;
			if (JsonObject->TryGetNumberField(ValueStr, FieldValue))
			{
				SetValue((float)FieldValue);
				bResult = true;
			}
		}
		else if (VariantTypeStr.Equals(EOnlineKeyValuePairDataType::ToString(EOnlineKeyValuePairDataType::String)))
		{
			FString FieldValue;
			if (JsonObject->TryGetStringField(ValueStr, FieldValue))
			{
				SetValue(FieldValue);
				bResult = true;
			}
		}
		else if (VariantTypeStr.Equals(EOnlineKeyValuePairDataType::ToString(EOnlineKeyValuePairDataType::Bool)))
		{
			bool FieldValue;
			if (JsonObject->TryGetBoolField(ValueStr, FieldValue))
			{
				SetValue(FieldValue);
				bResult = true;
			}
		}
		else if (VariantTypeStr.Equals(EOnlineKeyValuePairDataType::ToString(EOnlineKeyValuePairDataType::Int64)))
		{
			FString FieldValue;
			if (JsonObject->TryGetStringField(ValueStr, FieldValue))
			{
				Type = EOnlineKeyValuePairDataType::Int64;
				bResult = FromString(FieldValue);
			}
		}
		else if (VariantTypeStr.Equals(EOnlineKeyValuePairDataType::ToString(EOnlineKeyValuePairDataType::UInt64)))
		{
			FString FieldValue;
			if (JsonObject->TryGetStringField(ValueStr, FieldValue))
			{
				Type = EOnlineKeyValuePairDataType::UInt64;
				bResult = FromString(FieldValue);
			}
		}
		else if (VariantTypeStr.Equals(EOnlineKeyValuePairDataType::ToString(EOnlineKeyValuePairDataType::Double)))
		{
			double FieldValue;
			if (JsonObject->TryGetNumberField(ValueStr, FieldValue))
			{
				SetValue(FieldValue);
				bResult = true;
			}
		}
	}

	return bResult;
}

FString FVariantData::GetTypeSuffix() const
{
	switch (Type)
	{
	case EOnlineKeyValuePairDataType::Int32:
	{
		return TEXT("_i");
	}
	case EOnlineKeyValuePairDataType::UInt32:
	{
		return TEXT("_u");
	}
	case EOnlineKeyValuePairDataType::Float:
	{
		return TEXT("_f");
	}
	case EOnlineKeyValuePairDataType::String:
	{
		return TEXT("_s");
	}
	case EOnlineKeyValuePairDataType::Bool:
	{
		return TEXT("_b");
	}
	case EOnlineKeyValuePairDataType::Int64:
	{
		return TEXT("_I");
	}
	case EOnlineKeyValuePairDataType::UInt64:
	{
		return TEXT("_U");
	}
	case EOnlineKeyValuePairDataType::Double:
	{
		return TEXT("_d");
	}
	case EOnlineKeyValuePairDataType::Json:
	{
		return TEXT("_j");
	}
	case EOnlineKeyValuePairDataType::Empty:
	case EOnlineKeyValuePairDataType::Blob:
	default:
		break;
	}
	// Default to no suffix
	return TEXT("");
}

void FVariantData::AddToJsonObject(const TSharedRef<FJsonObject>& JsonObject, const FString& Name, const bool bWithTypeSuffix /* = true*/) const
{
	switch (Type)
	{
	case EOnlineKeyValuePairDataType::Int32:
	{
		int32 FieldValue;
		GetValue(FieldValue);
		JsonObject->SetNumberField(bWithTypeSuffix ? Name + TEXT("_i") : Name, (double)FieldValue);
		break;
	}
	case EOnlineKeyValuePairDataType::UInt32:
	{
		uint32 FieldValue;
		GetValue(FieldValue);
		JsonObject->SetNumberField(bWithTypeSuffix ? Name + TEXT("_u") : Name, (double)FieldValue);
		break;
	}
	case EOnlineKeyValuePairDataType::Float:
	{
		float FieldValue;
		GetValue(FieldValue);
		JsonObject->SetNumberField(bWithTypeSuffix ? Name + TEXT("_f") : Name, (double)FieldValue);
		break;
	}
	case EOnlineKeyValuePairDataType::String:
	{
		FString FieldValue;
		GetValue(FieldValue);
		JsonObject->SetStringField(bWithTypeSuffix ? Name + TEXT("_s") : Name, FieldValue);
		break;
	}
	case EOnlineKeyValuePairDataType::Bool:
	{
		bool FieldValue;
		GetValue(FieldValue);
		JsonObject->SetBoolField(bWithTypeSuffix ? Name + TEXT("_b") : Name, FieldValue);
		break;
	}
	case EOnlineKeyValuePairDataType::Int64:
	{
		JsonObject->SetStringField(bWithTypeSuffix ? Name + TEXT("_I") : Name, ToString());
		break;
	}
	case EOnlineKeyValuePairDataType::UInt64:
	{
		JsonObject->SetStringField(bWithTypeSuffix ? Name + TEXT("_U") : Name, ToString());
		break;
	}
	case EOnlineKeyValuePairDataType::Double:
	{
		double FieldValue;
		GetValue(FieldValue);
		JsonObject->SetNumberField(bWithTypeSuffix ? Name + TEXT("_d") : Name, (double)FieldValue);
		break;
	}
	case EOnlineKeyValuePairDataType::Json:
	{
		TSharedPtr<FJsonObject> FieldValue;
		GetValue(FieldValue);
		JsonObject->SetObjectField(bWithTypeSuffix ? Name + TEXT("_j") : Name, FieldValue);
	}
	case EOnlineKeyValuePairDataType::Empty:
	case EOnlineKeyValuePairDataType::Blob:
	default:
		break;
	}
}

bool FVariantData::FromJsonValue(FStringView JsonPropertyName, const TSharedRef<class FJsonValue>& JsonValue, FString& OutName)
{
	bool bResult = false;
	int32 Index;
	if (JsonPropertyName.FindLastChar(TEXT('_'), Index) && Index + 1 < JsonPropertyName.Len())
	{
		const TCHAR DataType = JsonPropertyName[Index + 1];
		OutName = JsonPropertyName.Left(Index);
		switch (DataType)
		{
		case TEXT('i'):
		{
			int32 FieldValue;
			if (JsonValue->TryGetNumber(FieldValue))
			{
				SetValue(FieldValue);
				bResult = true;
			}
			break;
		}
		case TEXT('u'):
		{
			uint32 FieldValue;
			if (JsonValue->TryGetNumber(FieldValue))
			{
				SetValue(FieldValue);
				bResult = true;
			}
			break;
		}
		case TEXT('f'):
		{
			double FieldValue;
			if (JsonValue->TryGetNumber(FieldValue))
			{
				SetValue(static_cast<float>(FieldValue));
				bResult = true;
			}
			break;
		}
		case TEXT('s'):
		{
			FString FieldValue;
			if (JsonValue->TryGetString(FieldValue))
			{
				SetValue(FieldValue);
				bResult = true;
			}
			break;
		}
		case TEXT('b'):
		{
			bool FieldValue;
			if (JsonValue->TryGetBool(FieldValue))
			{
				SetValue(FieldValue);
				bResult = true;
			}
			break;
		}
		case TEXT('I'):
		{
			FString FieldValue;
			if (JsonValue->TryGetString(FieldValue))
			{
				Type = EOnlineKeyValuePairDataType::Int64;
				bResult = FromString(FieldValue);
			}
			break;
		}
		case TEXT('U'):
		{
			FString FieldValue;
			if (JsonValue->TryGetString(FieldValue))
			{
				Type = EOnlineKeyValuePairDataType::UInt64;
				bResult = FromString(FieldValue);
			}
			break;
		}
		case TEXT('d'):
		{
			double FieldValue;
			if (JsonValue->TryGetNumber(FieldValue))
			{
				SetValue(FieldValue);
				bResult = true;
			}
			break;
		}
		case TEXT('j'):
		{
			const TSharedPtr<FJsonObject>* FieldValueObject;
			FString FieldValueString;
			if (JsonValue->TryGetObject(FieldValueObject)) // Try to read as a JSON object
			{
				SetValue((*FieldValueObject).ToSharedRef());
				bResult = true;
			}
			else if (JsonValue->TryGetString(FieldValueString)) // Try to read as a JSON string
			{
				SetJsonValueFromString(FieldValueString);
#if !UE_BUILD_SHIPPING
				// See if this was a valid JSON string
				TSharedPtr<FJsonObject> ValidJsonObject;
				GetValue(ValidJsonObject);
				bResult = ValidJsonObject.IsValid();
#else
				bResult = true;
#endif
			}
		}
		default:
			break;
		}
	}

	return bResult;
}

/**
 * Comparison of two settings data classes
 *
 * @param Other the other settings data to compare against
 *
 * @return true if they are equal, false otherwise
 */
bool FVariantData::operator==(const FVariantData& Other) const
{
	if (Type == Other.Type)
	{
		switch (Type)
		{
		case EOnlineKeyValuePairDataType::Float:
			{
				return Value.AsFloat == Other.Value.AsFloat;
			}
		case EOnlineKeyValuePairDataType::Int32:
			{
				return Value.AsInt == Other.Value.AsInt;
			}
		case EOnlineKeyValuePairDataType::UInt32:
			{
				return Value.AsUInt == Other.Value.AsUInt;
			}
		case EOnlineKeyValuePairDataType::Int64:
			{
				return Value.AsInt64 == Other.Value.AsInt64;
			}
		case EOnlineKeyValuePairDataType::UInt64:
			{
				return Value.AsInt64 == Other.Value.AsUInt64;
			}
		case EOnlineKeyValuePairDataType::Double:
			{
				return Value.AsDouble == Other.Value.AsDouble;
			}
		case EOnlineKeyValuePairDataType::String:
		case EOnlineKeyValuePairDataType::Json:
			{
				return FCString::Strcmp(Value.AsTCHAR, Other.Value.AsTCHAR) == 0;
			}
		case EOnlineKeyValuePairDataType::Blob:
			{
				return (Value.AsBlob.BlobSize == Other.Value.AsBlob.BlobSize) && 
						 (FMemory::Memcmp(Value.AsBlob.BlobData, Other.Value.AsBlob.BlobData, Value.AsBlob.BlobSize) == 0);
			}
		case EOnlineKeyValuePairDataType::Bool:
			{
				return Value.AsBool == Other.Value.AsBool;
			}
		}
	}
	return false;
}
bool FVariantData::operator!=(const FVariantData& Other) const
{
	return !(operator==(Other));
}

bool FVariantDataConverter::VariantMapToUStruct(const FOnlineKeyValuePairs<FString, FVariantData>& VariantMap, const UStruct* StructDefinition, void* OutStruct, int64 CheckFlags, int64 SkipFlags)
{
	return ApplyVariantMapToUnchangedProperties(VariantMap, StructDefinition, OutStruct, nullptr, CheckFlags, SkipFlags);
}

bool FVariantDataConverter::ApplyVariantMapToUnchangedProperties(const FOnlineKeyValuePairs<FString, FVariantData>& VariantMap, const UStruct* StructDefinition, void* OutStruct, const void* CompareStruct, int64 CheckFlags, int64 SkipFlags)
{
	for (TFieldIterator<FProperty> PropIt(StructDefinition); PropIt; ++PropIt)
	{
		FProperty* Property = *PropIt;
		FString PropertyName = Property->GetName();

		// Check to see if we should ignore this property
		if (CheckFlags != 0 && !Property->HasAnyPropertyFlags(CheckFlags))
		{
			continue;
		}
		if (Property->HasAnyPropertyFlags(SkipFlags))
		{
			continue;
		}

		// Possible case sensitive issues?
		const FVariantData* VariantData = VariantMap.Find(PropertyName);
		if (!VariantData)
		{
			// we allow values to not be found since this mirrors the typical UObject mantra that all the fields are optional when deserializing
			continue;
		}

		void* Value = Property->ContainerPtrToValuePtr<uint8>(OutStruct);

		if (CompareStruct)
		{
			const void* CompareValue = Property->ContainerPtrToValuePtr<uint8>(CompareStruct);
			if (!Property->Identical(Value, CompareValue))
			{
				// Skip applying if the properties don't match.
				continue;
			}
		}

		if (!VariantDataToFProperty(VariantData, Property, Value, CheckFlags, SkipFlags))
		{
			UE_LOG_ONLINE(Error, TEXT("VariantMapToUStruct - Unable to parse %s.%s from Variant"), *StructDefinition->GetName(), *PropertyName);
			return false;
		}
	}

	return true;
}

bool FVariantDataConverter::VariantDataToFProperty(const FVariantData* Variant, FProperty* Property, void* OutValue, int64 CheckFlags, int64 SkipFlags)
{
	if (!Variant)
	{
		UE_LOG_ONLINE(Error, TEXT("VariantDataToFProperty - Invalid value"));
		return false;
	}

	if (Property->ArrayDim != 1)
	{
		UE_LOG_ONLINE(Warning, TEXT("Ignoring excess properties when deserializing %s"), *Property->GetName());
	}

	return ConvertScalarVariantToFProperty(Variant, Property, OutValue, CheckFlags, SkipFlags);
}

bool FVariantDataConverter::ConvertScalarVariantToFProperty(const FVariantData* Variant, FProperty* Property, void* OutValue, int64 CheckFlags, int64 SkipFlags)
{
	if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
	{
		const UEnum* Enum = EnumProperty->GetEnum();

		if (Variant->GetType() == EOnlineKeyValuePairDataType::String)
		{
			FString StrValue;
			Variant->GetValue(StrValue);

			int64 IntValue = Enum->GetValueByName(FName(*StrValue));
			if (IntValue != INDEX_NONE)
			{
				EnumProperty->GetUnderlyingProperty()->SetIntPropertyValue(OutValue, IntValue);
				return true;
			}
		}
		else
		{
			int64 Value = 0;
			if (Variant->GetType() == EOnlineKeyValuePairDataType::Double)
			{
				double DoubleValue = 0.0;
				Variant->GetValue(DoubleValue);
				Value = (int64)DoubleValue;
			}
			else if (Variant->GetType() == EOnlineKeyValuePairDataType::Float)
			{
				float FloatValue = 0.0f;
				Variant->GetValue(FloatValue);
				Value = (int64)FloatValue;
			}
			else if (Variant->GetType() == EOnlineKeyValuePairDataType::Int32)
			{
				int32 IntValue = 0;
				Variant->GetValue(IntValue);
				Value = (int64)IntValue;
			}
			else if (Variant->GetType() == EOnlineKeyValuePairDataType::UInt32)
			{
				uint32 IntValue = 0;
				Variant->GetValue(IntValue);
				Value = (int64)IntValue;
			}
			else if (Variant->GetType() == EOnlineKeyValuePairDataType::Int64)
			{
				int64 Int64Value;
				Variant->GetValue(Int64Value);
				Value = (int64)Int64Value;
			}
			else if (Variant->GetType() == EOnlineKeyValuePairDataType::UInt64)
			{
				uint64 UInt64Value;
				Variant->GetValue(UInt64Value);
				Value = (int64)UInt64Value;
			}

			// AsNumber will log an error for completely inappropriate types (then give us a default)
			EnumProperty->GetUnderlyingProperty()->SetIntPropertyValue(OutValue, Value);
		}

		UE_LOG_ONLINE(Error, TEXT("ConvertScalarVariantToFProperty - Unable to import enum %s from %s value for property %s"), *Enum->CppType, Variant->GetTypeString(), *Property->GetNameCPP());
		return false;
	}
	else if (FNumericProperty* NumericProperty = CastField<FNumericProperty>(Property))
	{
		if (NumericProperty->IsEnum() && Variant->GetType() == EOnlineKeyValuePairDataType::String)
		{
			// see if we were passed a string for the enum
			const UEnum* Enum = NumericProperty->GetIntPropertyEnum();
			check(Enum); // should be assured by IsEnum()
			FString StrValue;
			Variant->GetValue(StrValue);

			int32 IntValue = int32(Enum->GetValueByName(FName(*StrValue)));
			if (IntValue == INDEX_NONE)
			{
				UE_LOG_ONLINE(Error, TEXT("ConvertScalarVariantToFProperty - Unable import enum %s from string value %s for property %s"), *Enum->CppType, *StrValue, *Property->GetNameCPP());
				return false;
			}
			NumericProperty->SetIntPropertyValue(OutValue, (int64)IntValue);
		}
		else if (NumericProperty->IsFloatingPoint())
		{
			if (Variant->GetType() == EOnlineKeyValuePairDataType::Double)
			{
				double DoubleValue;
				Variant->GetValue(DoubleValue);
				NumericProperty->SetFloatingPointPropertyValue(OutValue, DoubleValue);
			}
			else if (Variant->GetType() == EOnlineKeyValuePairDataType::Float)
			{
				float FloatValue;
				Variant->GetValue(FloatValue);
				NumericProperty->SetFloatingPointPropertyValue(OutValue, FloatValue);
			}
		}
		else if (NumericProperty->IsInteger())
		{
			if (Variant->GetType() == EOnlineKeyValuePairDataType::String)
			{
				FString StrValue;
				Variant->GetValue(StrValue);

				// parse string -> int64 ourselves so we don't lose any precision going through AsNumber (aka double)
				NumericProperty->SetIntPropertyValue(OutValue, FCString::Atoi64(*StrValue));
			}
			else
			{
				int64 Value = 0;
				if (Variant->GetType() == EOnlineKeyValuePairDataType::Double)
				{
					double DoubleValue = 0.0;
					Variant->GetValue(DoubleValue);
					Value = (int64)DoubleValue;
				}
				else if (Variant->GetType() == EOnlineKeyValuePairDataType::Float)
				{
					float FloatValue = 0.0f;
					Variant->GetValue(FloatValue);
					Value = (int64)FloatValue;
				}
				else if (Variant->GetType() == EOnlineKeyValuePairDataType::Int32)
				{
					int32 IntValue = 0;
					Variant->GetValue(IntValue);
					Value = (int64)IntValue;
				}
				else if (Variant->GetType() == EOnlineKeyValuePairDataType::UInt32)
				{
					uint32 IntValue = 0;
					Variant->GetValue(IntValue);
					Value = (int64)IntValue;
				}
				else if (Variant->GetType() == EOnlineKeyValuePairDataType::Int64)
				{
					int64 Int64Value;
					Variant->GetValue(Int64Value);
					Value = (int64)Int64Value;
				}
				else if (Variant->GetType() == EOnlineKeyValuePairDataType::UInt64)
				{
					uint64 UInt64Value;
					Variant->GetValue(UInt64Value);
					Value = (int64)UInt64Value;
				}

				// AsNumber will log an error for completely inappropriate types (then give us a default)
				NumericProperty->SetIntPropertyValue(OutValue, Value);
			}
		}
		else
		{
			UE_LOG_ONLINE(Error, TEXT("ConvertScalarVariantToFProperty - Unable to set numeric property type %s for property %s"), *Property->GetClass()->GetName(), *Property->GetNameCPP());
			return false;
		}
	}
	else if (FBoolProperty *BoolProperty = CastField<FBoolProperty>(Property))
	{
		bool BoolValue;
		Variant->GetValue(BoolValue);
		BoolProperty->SetPropertyValue(OutValue, BoolValue);
	}
	else if (FStrProperty *StringProperty = CastField<FStrProperty>(Property))
	{
		FString StrValue;
		Variant->GetValue(StrValue);
		StringProperty->SetPropertyValue(OutValue, StrValue);
	}
	else if (FArrayProperty *ArrayProperty = CastField<FArrayProperty>(Property))
	{
		FString StrValue;
		Variant->GetValue(StrValue);

		TSharedPtr<FJsonObject> JsonObject;
		TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(StrValue);
		if (!FJsonSerializer::Deserialize(JsonReader, JsonObject) || !JsonObject.IsValid())
		{
			UE_LOG_ONLINE(Warning, TEXT("ConvertScalarVariantToFProperty - Unable to parse json=[%s]"), *StrValue);
			return false;
		}
		
		if (!FJsonObjectConverter::JsonValueToUProperty(JsonObject->GetField<EJson::Array>(Property->GetNameCPP()), Property, OutValue, 0, 0))
		{
			UE_LOG_ONLINE(Warning, TEXT("ConvertScalarVariantToFProperty - Unable to parse %s from JSON"), *Property->GetNameCPP());
			return false;
		}
	}
	else if (FTextProperty *TextProperty = CastField<FTextProperty>(Property))
	{
		if (Variant->GetType() == EOnlineKeyValuePairDataType::String)
		{
			FString StrValue;
			Variant->GetValue(StrValue);
			// assume this string is already localized, so import as invariant
			TextProperty->SetPropertyValue(OutValue, FText::FromString(StrValue));
		}
		else
		{
			UE_LOG_ONLINE(Error, TEXT("ConvertScalarVariantToFProperty - Attempted to import FText from variant that was not a string for property %s"), *Property->GetNameCPP());
			return false;
		}
	}
	else if (FStructProperty *StructProperty = CastField<FStructProperty>(Property))
	{
		static const FName NAME_DateTime(TEXT("DateTime"));
		if (Variant->GetType() == EOnlineKeyValuePairDataType::String && StructProperty->Struct->GetFName() == NAME_DateTime)
		{
			FString DateString;
			Variant->GetValue(DateString);

			FDateTime& DateTimeOut = *(FDateTime*)OutValue;
			if (DateString == TEXT("min"))
			{
				// min representable value for our date struct. Actual date may vary by platform (this is used for sorting)
				DateTimeOut = FDateTime::MinValue();
			}
			else if (DateString == TEXT("max"))
			{
				// max representable value for our date struct. Actual date may vary by platform (this is used for sorting)
				DateTimeOut = FDateTime::MaxValue();
			}
			else if (DateString == TEXT("now"))
			{
				// this value's not really meaningful from serialization (since we don't know timezone) but handle it anyway since we're handling the other keywords
				DateTimeOut = FDateTime::UtcNow();
			}
			else if (!FDateTime::ParseIso8601(*DateString, DateTimeOut))
			{
				UE_LOG_ONLINE(Error, TEXT("ConvertScalarVariantToFProperty - Unable to import FDateTime from Iso8601 String for property %s"), *Property->GetNameCPP());
				return false;
			}
		}
		else
		{
			FString StrValue;
			Variant->GetValue(StrValue);

			UScriptStruct::ICppStructOps* TheCppStructOps = StructProperty->Struct->GetCppStructOps();
			if (StructProperty->Struct != FJsonObjectWrapper::StaticStruct() && TheCppStructOps && TheCppStructOps->HasImportTextItem())
			{
				// Has its own way of converting a string back to the right data type (FUniqueNetIdRepl for instance)
				const TCHAR* ImportTextPtr = *StrValue;
				if (!TheCppStructOps->ImportTextItem(ImportTextPtr, OutValue, PPF_None, nullptr, (FOutputDevice*)GWarn))
				{
					// Fall back to trying the tagged property approach if custom ImportTextItem couldn't get it done
					Property->ImportText_Direct(ImportTextPtr, OutValue, nullptr, PPF_None);
				}
			}
			else
			{
				// More complicated UStructs convert to/from Json Objects
				TSharedPtr<FJsonObject> JsonObject;
				TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(StrValue);
				if (!FJsonSerializer::Deserialize(JsonReader, JsonObject) || !JsonObject.IsValid())
				{
					UE_LOG_ONLINE(Warning, TEXT("ConvertScalarVariantToFProperty - Unable to parse json=[%s]"), *StrValue);
					return false;
				}

				if (!FJsonObjectConverter::JsonValueToUProperty(JsonObject->GetField<EJson::Object>(Property->GetNameCPP()), Property, OutValue, 0, 0))
				{
					UE_LOG_ONLINE(Warning, TEXT("ConvertScalarVariantToFProperty - Unable to parse %s from JSON"), *Property->GetNameCPP());
					return false;
				}
			}
		}
	}
	else
	{
		// Default to expect a string for everything else
		if (Variant->GetType() == EOnlineKeyValuePairDataType::String)
		{
			FString StrValue;
			Variant->GetValue(StrValue);
			if (Property->ImportText_Direct(*StrValue, OutValue, nullptr, 0) == nullptr)
			{
				UE_LOG_ONLINE(Error, TEXT("ConvertScalarVariantToFProperty - Unable import property type %s from string value for property %s"), *Property->GetClass()->GetName(), *Property->GetNameCPP());
				return false;
			}
		}
		else
		{
			UE_LOG_ONLINE(Error, TEXT("ConvertScalarVariantToFProperty - Unable import property type %s from string value for property %s"), *Property->GetClass()->GetName(), *Property->GetNameCPP());
		}
	}

	return true;
}

bool FVariantDataConverter::UStructToVariantMap(const UStruct* StructDefinition, const void* Struct, FOnlineKeyValuePairs<FString, FVariantData>& OutVariantMap, int64 CheckFlags, int64 SkipFlags)
{
	for (TFieldIterator<FProperty> It(StructDefinition); It; ++It)
	{
		FProperty* Property = *It;

		// Check to see if we should ignore this property
		if (CheckFlags != 0 && !Property->HasAnyPropertyFlags(CheckFlags))
		{
			continue;
		}
		if (Property->HasAnyPropertyFlags(SkipFlags))
		{
			continue;
		}

		FString VariableName = Property->GetName();

		const void* Value = Property->ContainerPtrToValuePtr<uint8>(Struct);

		// set the value on the output object
		FVariantData& VariantData = OutVariantMap.Add(VariableName);

		// convert the property to an FVariantData
		if (!FPropertyToVariantData(Property, Value, CheckFlags, SkipFlags, VariantData))
		{
			VariantData.Empty();
			FFieldClass* PropClass = Property->GetClass();
			UE_LOG_ONLINE(Error, TEXT("UStructToVariantMap - Unhandled property type '%s': %s"), *PropClass->GetName(), *Property->GetPathName());
			return false;
		}
	}

	return true;
}

bool FVariantDataConverter::FPropertyToVariantData(FProperty* Property, const void* Value, int64 CheckFlags, int64 SkipFlags, FVariantData& OutVariantData)
{
	if (Property->ArrayDim == 1)
	{
		return ConvertScalarFPropertyToVariant(Property, Value, OutVariantData, CheckFlags, SkipFlags);
	}	
	else 
	{ 
		FFieldClass* PropClass = Property->GetClass();
		UE_LOG_ONLINE(Error, TEXT("FPropertyToVariantData - ArrayDim > 1 for '%s': %s"), *PropClass->GetName(), *Property->GetPathName());
	}

	return false;
}

void ConvertScalarFPropertyToJsonObject_Helper(FProperty* Property, const void* Value, FVariantData &OutVariantData)
{
	TSharedPtr<FJsonValue> Json = FJsonObjectConverter::UPropertyToJsonValue(Property, Value, 0, 0);

	TSharedRef<FJsonObject> JsonObject = MakeShareable(new FJsonObject());
	JsonObject->SetField(Property->GetNameCPP(), Json);
	OutVariantData.SetValue(JsonObject);
}

bool FVariantDataConverter::ConvertScalarFPropertyToVariant(FProperty* Property, const void* Value, FVariantData& OutVariantData, int64 CheckFlags, int64 SkipFlags)
{
	OutVariantData.Empty();

	if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
	{
		// export enums as strings
		UEnum* EnumDef = EnumProperty->GetEnum();
		FString StringValue = EnumDef->GetNameStringByValue(EnumProperty->GetUnderlyingProperty()->GetSignedIntPropertyValue(Value));
		OutVariantData.SetValue(StringValue);
	}
	else if (FNumericProperty *NumericProperty = CastField<FNumericProperty>(Property))
	{
		// see if it's an enum
		UEnum* EnumDef = NumericProperty->GetIntPropertyEnum();
		if (EnumDef != nullptr)
		{
			// export enums as strings
			FString StringValue = EnumDef->GetNameStringByValue(NumericProperty->GetSignedIntPropertyValue(Value));
			OutVariantData.SetValue(StringValue);
		}
		// We want to export numbers as numbers
		else if (NumericProperty->IsFloatingPoint())
		{
			double DoubleValue = NumericProperty->GetFloatingPointPropertyValue(Value);
			OutVariantData.SetValue(DoubleValue);
		}
		else if (NumericProperty->IsInteger())
		{
			int64 Int64Value = NumericProperty->GetSignedIntPropertyValue(Value);
			OutVariantData.SetValue((uint64)Int64Value);
		}

		// fall through to default
	}
	else if (FBoolProperty *BoolProperty = CastField<FBoolProperty>(Property))
	{
		// Export bools as bools
		bool bBoolValue = BoolProperty->GetPropertyValue(Value);
		OutVariantData.SetValue(bBoolValue);
	}
	else if (FStrProperty *StringProperty = CastField<FStrProperty>(Property))
	{
		FString StringValue = StringProperty->GetPropertyValue(Value);
		OutVariantData.SetValue(StringValue);
	}
	else if (FTextProperty *TextProperty = CastField<FTextProperty>(Property))
	{
		FText TextValue = TextProperty->GetPropertyValue(Value);
		OutVariantData.SetValue(TextValue.ToString());
	}
	else if (FArrayProperty *ArrayProperty = CastField<FArrayProperty>(Property))
	{
		ConvertScalarFPropertyToJsonObject_Helper(Property, Value, OutVariantData);
	}
	else if (FStructProperty *StructProperty = CastField<FStructProperty>(Property))
	{
		static const FName NAME_DateTime(TEXT("DateTime"));
		if (StructProperty->Struct->GetFName() == NAME_DateTime)
		{
			FString Iso8601String;
			if (Value)
			{
				const FDateTime* Data = reinterpret_cast<const FDateTime*>(Value);
				Iso8601String = Data->ToIso8601();
			}
			OutVariantData.SetValue(Iso8601String);
		}
		else
		{
			UScriptStruct::ICppStructOps* TheCppStructOps = StructProperty->Struct->GetCppStructOps();
			if (StructProperty->Struct != FJsonObjectWrapper::StaticStruct() && TheCppStructOps && TheCppStructOps->HasExportTextItem())
			{
				// If the struct has its own way of making a string, store it as such (FUniqueNetIdRepl for instance)
				FString OutValueStr;
				TheCppStructOps->ExportTextItem(OutValueStr, Value, nullptr, nullptr, PPF_None, nullptr);
				OutVariantData.SetValue(OutValueStr);
			}
			else
			{
				// More complicated UStructs convert to/from Json Objects
				ConvertScalarFPropertyToJsonObject_Helper(Property, Value, OutVariantData);
			}
		}
	}
	
	if (OutVariantData.GetType() == EOnlineKeyValuePairDataType::Empty)
	{
		// Default to export as string for everything else
		FString StringValue;
		Property->ExportTextItem_Direct(StringValue, Value, nullptr, nullptr, PPF_None);
		OutVariantData.SetValue(StringValue);
	}

	return OutVariantData.GetType() != EOnlineKeyValuePairDataType::Empty;
}
