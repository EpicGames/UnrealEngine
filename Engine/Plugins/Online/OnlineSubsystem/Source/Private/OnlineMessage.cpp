// Copyright Epic Games, Inc. All Rights Reserved.

#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonSerializer.h"
#include "Interfaces/OnlineMessageInterface.h"
#include "NboSerializerOSS.h"

void FOnlineMessagePayload::ToBytes(TArray<uint8>& OutBytes) const
{
	FNboSerializeToBufferOSS Ar(MaxPayloadSize);
	Ar << KeyValData;
	Ar.TrimBuffer();
	OutBytes = Ar.GetBuffer();
}

void FOnlineMessagePayload::FromBytes(const TArray<uint8>& InBytes)
{
	FNboSerializeFromBufferOSS Ar(InBytes.GetData(), InBytes.Num());
	Ar >> KeyValData;
}

void FOnlineMessagePayload::ToJson(FJsonObject& OutJsonObject) const
{
	TSharedRef<FJsonObject> JsonProperties = MakeShared<FJsonObject>();
	for (const auto& It : KeyValData)
	{
		const FString& PropertyName = It.Key;
		const FVariantData& PropertyValue = It.Value;

		PropertyValue.AddToJsonObject(JsonProperties, PropertyName);
	}
	OutJsonObject.SetObjectField(TEXT("Properties"), JsonProperties);
}

FString FOnlineMessagePayload::ToJsonStr() const
{
	FString PayloadJsonStr;
	TSharedRef<FJsonObject> JsonObject(new FJsonObject());
	ToJson(*JsonObject);
	auto JsonWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR> >::Create(&PayloadJsonStr);
	FJsonSerializer::Serialize(JsonObject, JsonWriter);
	JsonWriter->Close();
	return PayloadJsonStr;
}

void FOnlineMessagePayload::FromJson(const FJsonObject& JsonObject)
{
	if (JsonObject.HasTypedField<EJson::Object>(TEXT("Properties")))
	{
		KeyValData.Empty();
		const TSharedPtr<FJsonObject>& JsonProperties = JsonObject.GetObjectField(TEXT("Properties"));
		for (auto& JsonProperty : JsonProperties->Values)
		{
			FString PropertyName;
			FVariantData PropertyData;
			if (PropertyData.FromJsonValue(JsonProperty.Key, JsonProperty.Value.ToSharedRef(), PropertyName))
			{
				KeyValData.Add(PropertyName, PropertyData);
			}
		}
	}
}

void FOnlineMessagePayload::FromJsonStr(const FString& JsonStr)
{
	TSharedPtr<FJsonObject> JsonObject;
	TSharedRef<TJsonReader<> > JsonReader = TJsonReaderFactory<>::Create(JsonStr);
	if (FJsonSerializer::Deserialize(JsonReader, JsonObject) &&
		JsonObject.IsValid())
	{
		FromJson(*JsonObject);
	}
}

bool FOnlineMessagePayload::GetAttribute(const FString& AttrName, FVariantData& OutAttrValue) const
{
	const FVariantData* Value = KeyValData.Find(AttrName);
	if (Value != NULL)
	{
		OutAttrValue = *Value;
		return true;
	}
	return false;
}

void FOnlineMessagePayload::SetAttribute(const FString& AttrName, const FVariantData& AttrValue)
{
	KeyValData.Add(AttrName, AttrValue);
}
