// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineNotification.h"
#include "JsonObjectConverter.h"
#include "OnlineSubsystem.h"

FOnlineNotification::FOnlineNotification(
	const FString& InTypeStr,
	const TSharedPtr<FJsonValue>& InPayload,
	FUniqueNetIdPtr InToUserId,
	FUniqueNetIdPtr InFromUserId,
	const FString& InClientRequestIdStr
)
: TypeStr(InTypeStr)
, Payload(InPayload.IsValid() ? InPayload->AsObject() : nullptr)
, ToUserId(InToUserId)
, FromUserId(InFromUserId)
, ClientRequestIdStr(InClientRequestIdStr)
{
}

bool FOnlineNotification::ParsePayload(UStruct* StructType, void* StructPtr) const
{
	check(StructType && StructPtr);
	return Payload.IsValid() && FJsonObjectConverter::JsonObjectToUStruct(Payload.ToSharedRef(), StructType, StructPtr, 0, 0);
}

void FOnlineNotification::SetTypeFromPayload()
{
	// Lazy init of type, if supplied in payload
	if (Payload.IsValid() && TypeStr.IsEmpty())
	{
		if (!Payload->TryGetStringField(TEXT("Type"), TypeStr))
		{
			UE_LOG_ONLINE(Error, TEXT("No type in notification JSON object"));
			TypeStr = TEXT("<no type>");
		}
	}
}

void FOnlineNotification::SetClientRequestIdFromPayload()
{
	if (Payload.IsValid() && ClientRequestIdStr.IsEmpty())
	{
		if (!Payload->TryGetStringField(TEXT("client_request_id"), ClientRequestIdStr))
		{
			UE_LOG_ONLINE(Error, TEXT("No client_request_id in notification JSON object"));
			ClientRequestIdStr = TEXT("");
		}
	}
}
