// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineNotificationHandler.h"
#include "OnlineSubsystem.h"
#include "OnlineNotification.h"

// SYSTEM NOTIFICATION HANDLERS

FDelegateHandle FOnlineNotificationHandler::AddSystemNotificationBinding_Handle(FString NotificationType, const FOnlineNotificationBinding& NewBinding)
{
	if (!NewBinding.NotificationDelegate.IsBound())
	{
		UE_LOG_ONLINE(Error, TEXT("Adding empty notification binding for type %s"), *NotificationType);
		return FDelegateHandle();
	}

	TArray<FOnlineNotificationBinding>& FoundBindings = SystemBindingMap.FindOrAdd(NotificationType);
	FoundBindings.Add(NewBinding);
	return FoundBindings.Last().NotificationDelegate.GetHandle();
}

void FOnlineNotificationHandler::RemoveSystemNotificationBinding(FString NotificationType, FDelegateHandle RemoveHandle)
{
	TArray<FOnlineNotificationBinding>* FoundBindings = SystemBindingMap.Find(NotificationType);

	int32 BindingsRemoved = 0;
	if (FoundBindings)
	{
		BindingsRemoved = FoundBindings->RemoveAll([=](const FOnlineNotificationBinding& Binding) { return Binding.NotificationDelegate.GetHandle() == RemoveHandle; });
	}

	if (BindingsRemoved == 0)
	{
		UE_LOG_ONLINE(Error, TEXT("Attempted to remove binding that could not be found for type %s"), *NotificationType);
	}
}


void FOnlineNotificationHandler::ResetSystemNotificationBindings()
{
	SystemBindingMap.Empty();
}

// PLAYER NOTIFICATION HANDLERS

/** Add a notification binding for a type */
FDelegateHandle FOnlineNotificationHandler::AddPlayerNotificationBinding_Handle(const FUniqueNetId& PlayerId, FString NotificationType, const FOnlineNotificationBinding& NewBinding)
{
	if (!NewBinding.NotificationDelegate.IsBound())
	{
		UE_LOG_ONLINE(Error, TEXT("Adding empty notification binding for type %s"), *NotificationType);
		return FDelegateHandle();
	}

	NotificationTypeBindingsMap& FoundPlayerBindings = PlayerBindingMap.FindOrAdd(PlayerId.ToString());
	TArray<FOnlineNotificationBinding>& FoundPlayerTypeBindings = FoundPlayerBindings.FindOrAdd(NotificationType);
	FoundPlayerTypeBindings.Add(NewBinding);
	return FoundPlayerTypeBindings.Last().NotificationDelegate.GetHandle();
}

/** Remove the player notification handler for a type */
void FOnlineNotificationHandler::RemovePlayerNotificationBinding(const FUniqueNetId& PlayerId, FString NotificationType, FDelegateHandle RemoveHandle)
{
	int32 BindingsRemoved = 0;

	NotificationTypeBindingsMap* FoundPlayerBindings = PlayerBindingMap.Find(PlayerId.ToString());

	if (FoundPlayerBindings)
	{
		TArray<FOnlineNotificationBinding>* FoundPlayerTypeBindings = FoundPlayerBindings->Find(NotificationType);

		if (FoundPlayerTypeBindings)
		{
			BindingsRemoved = FoundPlayerTypeBindings->RemoveAll([=](const FOnlineNotificationBinding& Binding) { return Binding.NotificationDelegate.GetHandle() == RemoveHandle; });
		}
	}

	if (BindingsRemoved == 0)
	{
		UE_LOG_ONLINE(Error, TEXT("Attempted to remove binding that could not be found for player %s type %s"), *PlayerId.ToDebugString(), *NotificationType);
	}
}

/** Resets a player's notification handlers */
void FOnlineNotificationHandler::ResetPlayerNotificationBindings(const FUniqueNetId& PlayerId)
{
	NotificationTypeBindingsMap* FoundPlayerBindings = PlayerBindingMap.Find(PlayerId.ToString());
	if (FoundPlayerBindings)
	{
		FoundPlayerBindings->Reset();
	}
}

/** Resets all player notification handlers */
void FOnlineNotificationHandler::ResetAllPlayerNotificationBindings()
{
	PlayerBindingMap.Reset();
}

// RECEIVING NOTIFICATIONS

void FOnlineNotificationHandler::DeliverNotification(const FOnlineNotification& Notification)
{
	EOnlineNotificationResult CurrentResult = EOnlineNotificationResult::None;

	// Delivery system bindings, scoped out vars/results
	{
		TArray<FOnlineNotificationBinding>* FoundSystemBindings = SystemBindingMap.Find(Notification.TypeStr);

		if (FoundSystemBindings)
		{
			for (int32 i = 0; i < FoundSystemBindings->Num(); i++)
			{
				const FOnlineNotificationBinding& Binding = (*FoundSystemBindings)[i];

				if (Binding.NotificationDelegate.IsBound())
				{
					CurrentResult = Binding.NotificationDelegate.Execute(Notification);
				}
			}
		}
	}

	// Delivery player bindings
	if (Notification.ToUserId.IsValid())
	{
		NotificationTypeBindingsMap* FoundPlayerBinding = PlayerBindingMap.Find(Notification.ToUserId->ToString());

		if (FoundPlayerBinding)
		{
			TArray<FOnlineNotificationBinding>* FoundPlayerTypeBindings = FoundPlayerBinding->Find(Notification.TypeStr);
			if (FoundPlayerTypeBindings)
			{
				for (int32 i = 0; i < FoundPlayerTypeBindings->Num(); i++)
				{
					const FOnlineNotificationBinding& Binding = (*FoundPlayerTypeBindings)[i];

					if (Binding.NotificationDelegate.IsBound())
					{
						CurrentResult = Binding.NotificationDelegate.Execute(Notification);
					}
				}
			}
		}
	}

	if (CurrentResult == EOnlineNotificationResult::None)
	{
		// can be safely removed once this use case exists, just here to catch errors in initial implementation
		UE_LOG_ONLINE(Error, TEXT("Received an onlinenotification that was not handled. Type %s for %s"), *Notification.TypeStr, Notification.ToUserId.IsValid() ? *Notification.ToUserId->ToString() : TEXT("<system notification>"));
	}
}
