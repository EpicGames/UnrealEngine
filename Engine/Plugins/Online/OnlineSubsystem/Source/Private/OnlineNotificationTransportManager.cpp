// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineNotificationTransportManager.h"

FOnlineNotificationTransportManager::~FOnlineNotificationTransportManager()
{
}

bool FOnlineNotificationTransportManager::SendNotification(FNotificationTransportId TransportType, const FOnlineNotification& Notification)
{
	IOnlineNotificationTransportPtr* TransportPtrPtr = TransportMap.Find(TransportType);
	if (TransportPtrPtr != NULL)
	{
		bool result = (*TransportPtrPtr)->SendNotification(Notification);
		return result;
	}
	else
	{
		return false;
	}
}

bool FOnlineNotificationTransportManager::ReceiveTransportMessage(FNotificationTransportId TransportType, const IOnlineNotificationTransportMessage& TransportMessage)
{
	IOnlineNotificationTransportPtr* TransportPtrPtr = TransportMap.Find(TransportType);
	if (TransportPtrPtr != NULL)
	{
		bool result = (*TransportPtrPtr)->ReceiveNotification(TransportMessage);
		return result;
	}
	else
	{
		return false;
	}
}

IOnlineNotificationTransportPtr FOnlineNotificationTransportManager::GetNotificationTransport(FNotificationTransportId TransportType)
{
	IOnlineNotificationTransportPtr* TransportPtrPtr = TransportMap.Find(TransportType);

	if (TransportPtrPtr != NULL)
	{
		return *TransportPtrPtr;
	}
	else
	{
		return NULL;
	}
}

void FOnlineNotificationTransportManager::AddNotificationTransport(IOnlineNotificationTransportPtr Transport)
{
	TransportMap.Add(Transport->GetNotificationTransportId(), Transport);
}

void FOnlineNotificationTransportManager::RemoveNotificationTransport(FNotificationTransportId TransportType)
{
	TransportMap.Remove(TransportType);
}

void FOnlineNotificationTransportManager::ResetNotificationTransports()
{
	TransportMap.Empty();
}
