// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/OnlinePartyInterface.h"
#include "OnlineSubsystem.h" // IWYU pragma: keep
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonSerializer.h"

DEFINE_LOG_CATEGORY(LogOnlineParty);

FName DefaultPartyDataNamespace = NAME_Default;

bool FOnlinePartyData::operator==(const FOnlinePartyData& Other) const
{
	// Only compare KeyValAttrs, other fields are optimization details
	return KeyValAttrs.OrderIndependentCompareEqual(Other.KeyValAttrs);
}

bool FOnlinePartyData::operator!=(const FOnlinePartyData& Other) const
{
	return !operator==(Other);
}

void FOnlinePartyData::GetDirtyKeyValAttrs(FOnlineKeyValuePairs<FString, FVariantData>& OutDirtyAttrs, TArray<FString>& OutRemovedAttrs) const
{
	for (const FString& PropertyName : DirtyKeys)
	{
		const FVariantData* PropertyValue = KeyValAttrs.Find(PropertyName);
		if (PropertyValue)
		{
			OutDirtyAttrs.Emplace(PropertyName, *PropertyValue);
		}
		else
		{
			OutRemovedAttrs.Emplace(PropertyName);
		}
	}
}

void FOnlinePartyData::ToJsonFull(FString& JsonString) const
{
	JsonString.Empty();

	// iterate over key/val attrs and convert each entry to a json string
	TSharedRef<FJsonObject> JsonObject(new FJsonObject());
	TSharedRef<FJsonObject> JsonProperties = GetAllAttributesAsJsonObject();
	JsonObject->SetNumberField(TEXT("Rev"), RevisionCount);
	JsonObject->SetObjectField(TEXT("Attrs"), JsonProperties);

	auto JsonWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR> >::Create(&JsonString);
	FJsonSerializer::Serialize(JsonObject, JsonWriter);
	JsonWriter->Close();
}

void FOnlinePartyData::ToJsonDirty(FString& JsonString) const
{
	JsonString.Empty();

	// iterate over key/val attrs and convert each entry to a json string
	TSharedRef<FJsonObject> JsonObject(new FJsonObject());
	TSharedRef<FJsonObject> JsonProperties = MakeShared<FJsonObject>();
	for (const FString& PropertyName : DirtyKeys)
	{
		const FVariantData* PropertyValue = KeyValAttrs.Find(PropertyName);
		check(PropertyValue);

		PropertyValue->AddToJsonObject(JsonProperties, PropertyName);
	}
	JsonObject->SetNumberField(TEXT("Rev"), RevisionCount);
	JsonObject->SetObjectField(TEXT("Attrs"), JsonProperties);

	auto JsonWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR> >::Create(&JsonString);
	FJsonSerializer::Serialize(JsonObject, JsonWriter);
	JsonWriter->Close();
}

TSharedRef<FJsonObject> FOnlinePartyData::GetAllAttributesAsJsonObject() const
{
	TSharedRef<FJsonObject> JsonProperties = MakeShared<FJsonObject>();
	for (const TPair<FString, FVariantData>& Iterator : KeyValAttrs)
	{
		const FString& PropertyName = Iterator.Key;
		const FVariantData& PropertyValue = Iterator.Value;

		PropertyValue.AddToJsonObject(JsonProperties, PropertyName);
	}
	return JsonProperties;
}

FString FOnlinePartyData::GetAllAttributesAsJsonObjectString() const
{
	TSharedRef<FJsonObject> JsonProperties = GetAllAttributesAsJsonObject();

	FString JsonString;
	auto JsonWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR> >::Create(&JsonString);
	FJsonSerializer::Serialize(JsonProperties, JsonWriter);
	JsonWriter->Close();
	return JsonString;
}

void FOnlinePartyData::FromJson(const FString& JsonString)
{
	// json string to key/val attrs
	TSharedPtr<FJsonObject> JsonObject;
	TSharedRef<TJsonReader<> > JsonReader = TJsonReaderFactory<>::Create(JsonString);
	if (FJsonSerializer::Deserialize(JsonReader, JsonObject) &&
		JsonObject.IsValid())
	{
		if (JsonObject->HasTypedField<EJson::Object>(TEXT("Attrs")))
		{
			const TSharedPtr<FJsonObject>& JsonProperties = JsonObject->GetObjectField(TEXT("Attrs"));
			for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& JsonProperty : JsonProperties->Values)
			{
				FString PropertyName;
				FVariantData PropertyData;
				if (PropertyData.FromJsonValue(FString(JsonProperty.Key), JsonProperty.Value.ToSharedRef(), PropertyName))
				{
					KeyValAttrs.Add(PropertyName, PropertyData);
				}
			}
		}

		if (JsonObject->HasTypedField<EJson::Number>(TEXT("Rev")))
		{
			int32 NewRevisionCount = JsonObject->GetIntegerField(TEXT("Rev"));
			if ((RevisionCount != 0) && (NewRevisionCount != RevisionCount) && (NewRevisionCount != (RevisionCount + 1)))
			{
				UE_LOG_ONLINE_PARTY(Warning, TEXT("Unexpected revision received.  Current %d, new %d"), RevisionCount, NewRevisionCount);
			}
			RevisionCount = NewRevisionCount;
		}
	}
}

FString FPartyInvitationRecipient::ToDebugString() const
{
	return FString::Printf(TEXT("Id=[%s], PlatformData=[%s]"), *Id->ToDebugString(), *PlatformData);
}

bool FPartyConfiguration::operator==(const FPartyConfiguration& Other) const
{
	return JoinRequestAction == Other.JoinRequestAction &&
		PresencePermissions == Other.PresencePermissions &&
		InvitePermissions == Other.InvitePermissions &&
		bChatEnabled == Other.bChatEnabled &&
		bIsAcceptingMembers == Other.bIsAcceptingMembers &&
		NotAcceptingMembersReason == Other.NotAcceptingMembersReason &&
		MaxMembers == Other.MaxMembers &&
		Nickname == Other.Nickname &&
		Description == Other.Description &&
		Password == Other.Password;
}

bool FPartyConfiguration::operator!=(const FPartyConfiguration& Other) const
{
	return !operator==(Other);
}

const TCHAR* ToString(const EPartyState Value)
{
	switch (Value)
	{
	case EPartyState::None:
	{
		return TEXT("None");
	}
	case EPartyState::CreatePending:
	{
		return TEXT("CreatePending");
	}
	case EPartyState::JoinPending:
	{
		return TEXT("JoinPending");
	}
	case EPartyState::LeavePending:
	{
		return TEXT("LeavePending");
	}
	case EPartyState::Active:
	{
		return TEXT("Active");
	}
	case EPartyState::Disconnected:
	{
		return TEXT("Disconnected");
	}
	case EPartyState::CleanUp:
	{
		return TEXT("CleanUp");
	}
	}
	return TEXT("Unknown");
}

EPartyState EPartyStateFromString(const TCHAR* Value)
{
	if (FCString::Stricmp(Value, TEXT("CreatePending")) == 0)
	{
		return EPartyState::CreatePending;
	}
	else if (FCString::Stricmp(Value, TEXT("JoinPending")) == 0)
	{
		return EPartyState::JoinPending;
	}
	else if (FCString::Stricmp(Value, TEXT("LeavePending")) == 0)
	{
		return EPartyState::LeavePending;
	}
	else if (FCString::Stricmp(Value, TEXT("Active")) == 0)
	{
		return EPartyState::Active;
	}
	else if (FCString::Stricmp(Value, TEXT("Disconnected")) == 0)
	{
		return EPartyState::Disconnected;
	}
	else if (FCString::Stricmp(Value, TEXT("CleanUp")) == 0)
	{
		return EPartyState::CleanUp;
	}
	return EPartyState::None;
}

const TCHAR* ToString(const EMemberConnectionStatus Value)
{
	switch (Value)
	{
	case EMemberConnectionStatus::Uninitialized:
		return TEXT("Uninitialized");
	case EMemberConnectionStatus::Disconnected:
		return TEXT("Disconnected");
	case EMemberConnectionStatus::Initializing:
		return TEXT("Initializing");
	case EMemberConnectionStatus::Connected:
		return TEXT("Connected");
	}
	checkNoEntry();
	return TEXT("Uninitialized");
}

const TCHAR* ToString(const EMemberExitedReason Value)
{
	switch (Value)
	{
	case EMemberExitedReason::Unknown:
	{
		return TEXT("Unknown");
	}
	case EMemberExitedReason::Left:
	{
		return TEXT("Left");
	}
	case EMemberExitedReason::Removed:
	{
		return TEXT("Removed");
	}
	case EMemberExitedReason::Kicked:
	{
		return TEXT("Kicked");
	}
	}
	return TEXT("Unknown"); // Same as EMemberExitedReason::Unknown, which is ok because it is only used when we do not have enough information
}

const TCHAR* ToString(const EPartyInvitationRemovedReason Value)
{
	switch (Value)
	{
	case EPartyInvitationRemovedReason::Unknown:
	{
		return TEXT("Unknown");
	}
	case EPartyInvitationRemovedReason::Accepted:
	{
		return TEXT("Accepted");
	}
	case EPartyInvitationRemovedReason::Declined:
	{
		return TEXT("Declined");
	}
	case EPartyInvitationRemovedReason::Cleared:
	{
		return TEXT("Cleared");
	}
	case EPartyInvitationRemovedReason::Expired:
	{
		return TEXT("Expired");
	}
	case EPartyInvitationRemovedReason::Invalidated:
	{
		return TEXT("Invalidated");
	}
	}
	return TEXT("Unknown"); // Same as EMemberExitedReason::Unknown, which is ok because it is only used when we do not have enough information
}

const TCHAR* ToString(const EPartyRequestToJoinRemovedReason Value)
{
	switch (Value)
	{
	case EPartyRequestToJoinRemovedReason::Unknown:
	{
		return TEXT("Unknown");
	}
	case EPartyRequestToJoinRemovedReason::Cancelled:
	{
		return TEXT("Cancelled");
	}
	case EPartyRequestToJoinRemovedReason::Expired:
	{
		return TEXT("Expired");
	}
	case EPartyRequestToJoinRemovedReason::Dismissed:
	{
		return TEXT("Dismissed");
	}
	case EPartyRequestToJoinRemovedReason::Accepted:
	{
		return TEXT("Accepted");
	}
	}
	return TEXT("Unknown");
}

const TCHAR* ToString(const ECreatePartyCompletionResult Value)
{
	switch (Value)
	{
	case ECreatePartyCompletionResult::UnknownClientFailure:
	{
		return TEXT("UnknownClientFailure");
	}
	case ECreatePartyCompletionResult::AlreadyCreatingParty:
	{
		return TEXT("AlreadyCreatingParty");
	}
	case ECreatePartyCompletionResult::AlreadyInParty:
	{
		return TEXT("AlreadyInParty");
	}
	case ECreatePartyCompletionResult::FailedToCreateMucRoom:
	{
		return TEXT("FailedToCreateMucRoom");
	}
	case ECreatePartyCompletionResult::NoResponse:
	{
		return TEXT("NoResponse");
	}
	case ECreatePartyCompletionResult::LoggedOut:
	{
		return TEXT("LoggedOut");
	}
	case ECreatePartyCompletionResult::NotPrimaryUser:
	{
		return TEXT("NotPrimaryUser");
	}
	case ECreatePartyCompletionResult::UnknownInternalFailure:
	{
		return TEXT("UnknownInternalFailure");
	}
	case ECreatePartyCompletionResult::Succeeded:
	{
		return TEXT("Succeeded");
	}
	}
	return TEXT("Unknown");
}

const TCHAR* ToString(const ESendPartyInvitationCompletionResult Value)
{
	switch (Value)
	{
	case ESendPartyInvitationCompletionResult::NotLoggedIn:
	{
		return TEXT("NotLoggedIn");
	}
	case ESendPartyInvitationCompletionResult::InvitePending:
	{
		return TEXT("InvitePending");
	}
	case ESendPartyInvitationCompletionResult::AlreadyInParty:
	{
		return TEXT("AlreadyInParty");
	}
	case ESendPartyInvitationCompletionResult::PartyFull:
	{
		return TEXT("PartyFull");
	}
	case ESendPartyInvitationCompletionResult::NoPermission:
	{
		return TEXT("NoPermission");
	}
	case ESendPartyInvitationCompletionResult::RateLimited:
	{
		return TEXT("RateLimited");
	}
	case ESendPartyInvitationCompletionResult::UnknownInternalFailure:
	{
		return TEXT("UnknownInternalFailure");
	}
	case ESendPartyInvitationCompletionResult::Succeeded:
	{
		return TEXT("Succeeded");
	}
	}
	return TEXT("Unknown");
}

const TCHAR* ToString(const EJoinPartyCompletionResult Value)
{
	switch (Value)
	{
	case EJoinPartyCompletionResult::UnknownClientFailure:
	{
		return TEXT("UnknownClientFailure");
	}
	case EJoinPartyCompletionResult::BadBuild:
	{
		return TEXT("BadBuild");
	}
	case EJoinPartyCompletionResult::InvalidAccessKey:
	{
		return TEXT("InvalidAccessKey");
	}
	case EJoinPartyCompletionResult::AlreadyInLeadersJoiningList:
	{
		return TEXT("AlreadyInLeadersJoiningList");
	}
	case EJoinPartyCompletionResult::AlreadyInLeadersPartyRoster:
	{
		return TEXT("AlreadyInLeadersPartyRoster");
	}
	case EJoinPartyCompletionResult::NoSpace:
	{
		return TEXT("NoSpace");
	}
	case EJoinPartyCompletionResult::NotApproved:
	{
		return TEXT("NotApproved");
	}
	case EJoinPartyCompletionResult::RequesteeNotMember:
	{
		return TEXT("RequesteeNotMember");
	}
	case EJoinPartyCompletionResult::RequesteeNotLeader:
	{
		return TEXT("RequesteeNotLeader");
	}
	case EJoinPartyCompletionResult::NoResponse:
	{
		return TEXT("NoResponse");
	}
	case EJoinPartyCompletionResult::LoggedOut:
	{
		return TEXT("LoggedOut");
	}
	case EJoinPartyCompletionResult::IncompatiblePlatform:
	{
		return TEXT("IncompatiblePlatform");
	}
	case EJoinPartyCompletionResult::AlreadyJoiningParty:
	{
		return TEXT("AlreadyJoiningParty");
	}
	case EJoinPartyCompletionResult::AlreadyInParty:
	{
		return TEXT("AlreadyInParty");
	}
	case EJoinPartyCompletionResult::JoinInfoInvalid:
	{
		return TEXT("JoinInfoInvalid");
	}
	case EJoinPartyCompletionResult::AlreadyInPartyOfSpecifiedType:
	{
		return TEXT("AlreadyInPartyOfSpecifiedType");
	}
	case EJoinPartyCompletionResult::MessagingFailure:
	{
		return TEXT("MessagingFailure");
	}
	case EJoinPartyCompletionResult::GameSpecificReason:
	{
		return TEXT("GameSpecificReason");
	}
	case EJoinPartyCompletionResult::MismatchedApp:
	{
		return TEXT("MismatchedApp");
	}
	case EJoinPartyCompletionResult::Succeeded:
	{
		return TEXT("Succeeded");
	}
	case EJoinPartyCompletionResult::UnknownInternalFailure:
	{
		return TEXT("DeprecatedUnknownInternalFailure");
	}
	}
	return TEXT("Unknown");
}

const TCHAR* ToString(const ELeavePartyCompletionResult Value)
{
	switch (Value)
	{
	case ELeavePartyCompletionResult::UnknownClientFailure:
	{
		return TEXT("UnknownClientFailure");
	}
	case ELeavePartyCompletionResult::NoResponse:
	{
		return TEXT("NoResponse");
	}
	case ELeavePartyCompletionResult::LoggedOut:
	{
		return TEXT("LoggedOut");
	}
	case ELeavePartyCompletionResult::UnknownParty:
	{
		return TEXT("UnknownParty");
	}
	case ELeavePartyCompletionResult::LeavePending:
	{
		return TEXT("LeavePending");
	}
	case ELeavePartyCompletionResult::Succeeded:
	{
		return TEXT("Succeeded");
	}
	case ELeavePartyCompletionResult::UnknownLocalUser:
	{
		return TEXT("DeprecatedUnknownLocalUser");
	}
	case ELeavePartyCompletionResult::NotMember:
	{
		return TEXT("DeprecatedNotMember");
	}
	case ELeavePartyCompletionResult::MessagingFailure:
	{
		return TEXT("DeprecatedMessagingFailure");
	}
	case ELeavePartyCompletionResult::UnknownTransportFailure:
	{
		return TEXT("DeprecatedUnknownTransportFailure");
	}
	case ELeavePartyCompletionResult::UnknownInternalFailure:
	{
		return TEXT("DeprecatedUnknownInternalFailure");
	}
	}
	return TEXT("Unknown");
}

const TCHAR* ToString(const EUpdateConfigCompletionResult Value)
{
	switch (Value)
	{
	case EUpdateConfigCompletionResult::UnknownClientFailure:
	{
		return TEXT("UnknownClientFailure");
	}
	case EUpdateConfigCompletionResult::UnknownParty:
	{
		return TEXT("UnknownParty");
	}
	case EUpdateConfigCompletionResult::LocalMemberNotMember:
	{
		return TEXT("LocalMemberNotMember");
	}
	case EUpdateConfigCompletionResult::LocalMemberNotLeader:
	{
		return TEXT("LocalMemberNotLeader");
	}
	case EUpdateConfigCompletionResult::RemoteMemberNotMember:
	{
		return TEXT("RemoteMemberNotMember");
	}
	case EUpdateConfigCompletionResult::MessagingFailure:
	{
		return TEXT("MessagingFailure");
	}
	case EUpdateConfigCompletionResult::NoResponse:
	{
		return TEXT("NoResponse");
	}
	case EUpdateConfigCompletionResult::UnknownInternalFailure:
	{
		return TEXT("UnknownInternalFailure");
	}
	case EUpdateConfigCompletionResult::Succeeded:
	{
		return TEXT("Succeeded");
	}
	}
	return TEXT("Unknown");
}

const TCHAR* ToString(const EKickMemberCompletionResult Value)
{
	switch (Value)
	{
	case EKickMemberCompletionResult::UnknownClientFailure:
	{
		return TEXT("UnknownClientFailure");
	}
	case EKickMemberCompletionResult::UnknownParty:
	{
		return TEXT("UnknownParty");
	}
	case EKickMemberCompletionResult::LocalMemberNotMember:
	{
		return TEXT("LocalMemberNotMember");
	}
	case EKickMemberCompletionResult::LocalMemberNotLeader:
	{
		return TEXT("LocalMemberNotLeader");
	}
	case EKickMemberCompletionResult::RemoteMemberNotMember:
	{
		return TEXT("RemoteMemberNotMember");
	}
	case EKickMemberCompletionResult::MessagingFailure:
	{
		return TEXT("MessagingFailure");
	}
	case EKickMemberCompletionResult::NoResponse:
	{
		return TEXT("NoResponse");
	}
	case EKickMemberCompletionResult::LoggedOut:
	{
		return TEXT("LoggedOut");
	}
	case EKickMemberCompletionResult::UnknownInternalFailure:
	{
		return TEXT("UnknownInternalFailure");
	}
	case EKickMemberCompletionResult::Succeeded:
	{
		return TEXT("Succeeded");
	}
	}
	return TEXT("Unknown");
}

const TCHAR* ToString(const EPromoteMemberCompletionResult Value)
{
	switch (Value)
	{
	case EPromoteMemberCompletionResult::UnknownClientFailure:
	{
		return TEXT("UnknownClientFailure");
	}
	case EPromoteMemberCompletionResult::UnknownParty:
	{
		return TEXT("UnknownParty");
	}
	case EPromoteMemberCompletionResult::LocalMemberNotMember:
	{
		return TEXT("LocalMemberNotMember");
	}
	case EPromoteMemberCompletionResult::LocalMemberNotLeader:
	{
		return TEXT("LocalMemberNotLeader");
	}
	case EPromoteMemberCompletionResult::TargetIsSelf:
	{
		return TEXT("TargetIsSelf");
	}
	case EPromoteMemberCompletionResult::TargetNotMember:
	{
		return TEXT("TargetNotMember");
	}
	case EPromoteMemberCompletionResult::MessagingFailure:
	{
		return TEXT("MessagingFailure");
	}
	case EPromoteMemberCompletionResult::NoResponse:
	{
		return TEXT("NoResponse");
	}
	case EPromoteMemberCompletionResult::LoggedOut:
	{
		return TEXT("LoggedOut");
	}
	case EPromoteMemberCompletionResult::UnknownInternalFailure:
	{
		return TEXT("UnknownInternalFailure");
	}
	case EPromoteMemberCompletionResult::Succeeded:
	{
		return TEXT("Succeeded");
	}
	}
	return TEXT("Unknown");
}

const TCHAR* ToString(const EInvitationResponse Value)
{
	switch (Value)
	{
	case EInvitationResponse::UnknownFailure:
	{
		return TEXT("UnknownFailure");
	}
	case EInvitationResponse::BadBuild:
	{
		return TEXT("BadBuild");
	}
	case EInvitationResponse::Rejected:
	{
		return TEXT("Rejected");
	}
	case EInvitationResponse::Accepted:
	{
		return TEXT("Accepted");
	}
	}
	return TEXT("Unknown");
}

const TCHAR* ToString(const PartySystemPermissions::EPermissionType Value)
{
	switch (Value)
	{
	case PartySystemPermissions::EPermissionType::Noone:
	{
		return TEXT("Noone");
	}
	case PartySystemPermissions::EPermissionType::Leader:
	{
		return TEXT("Leader");
	}
	case PartySystemPermissions::EPermissionType::Friends:
	{
		return TEXT("Friends");
	}
	case PartySystemPermissions::EPermissionType::Anyone:
	{
		return TEXT("Anyone");
	}
	}
	return TEXT("Unknown");
}

PartySystemPermissions::EPermissionType PartySystemPermissionTypeFromString(const TCHAR* Value)
{
	if (FCString::Stricmp(Value, TEXT("Leader")) == 0)
	{
		return PartySystemPermissions::EPermissionType::Leader;
	}
	else if (FCString::Stricmp(Value, TEXT("Friends")) == 0)
	{
		return PartySystemPermissions::EPermissionType::Friends;
	}
	else if (FCString::Stricmp(Value, TEXT("Anyone")) == 0)
	{
		return PartySystemPermissions::EPermissionType::Anyone;
	}
	return PartySystemPermissions::EPermissionType::Noone;
}

const TCHAR* ToString(const EJoinRequestAction Value)
{
	switch (Value)
	{
	case EJoinRequestAction::Manual:
	{
		return TEXT("Manual");
	}
	case EJoinRequestAction::AutoApprove:
	{
		return TEXT("AutoApprove");
	}
	case EJoinRequestAction::AutoReject:
	{
		return TEXT("AutoReject");
	}
	}
	return TEXT("Unknown");
}

const TCHAR* ToString(const ERequestToJoinPartyCompletionResult Value)
{
	switch (Value)
	{
	case ERequestToJoinPartyCompletionResult::ValidationFailure:
	{
		return TEXT("ValidationFailure");
	}
	case ERequestToJoinPartyCompletionResult::NotAuthorized:
	{
		return TEXT("NotAuthorized");
	}
	case ERequestToJoinPartyCompletionResult::Forbidden:
	{
		return TEXT("Forbidden");
	}
	case ERequestToJoinPartyCompletionResult::UserNotFound:
	{
		return TEXT("UserNotFound");
	}
	case ERequestToJoinPartyCompletionResult::AlreadyExists:
	{
		return TEXT("AlreadyExists");
	}
	case ERequestToJoinPartyCompletionResult::RateLimited:
	{
		return TEXT("RateLimited");
	}
	case ERequestToJoinPartyCompletionResult::UnknownInternalFailure:
	{
		return TEXT("UnknownInternalFailure");
	}
	case ERequestToJoinPartyCompletionResult::Succeeded:
	{
		return TEXT("Succeeded");
	}
	}
	return TEXT("Unknown");
}

EJoinRequestAction JoinRequestActionFromString(const TCHAR* Value)
{
	if (FCString::Stricmp(Value, TEXT("AutoApprove")) == 0)
	{
		return EJoinRequestAction::AutoApprove;
	}
	else if (FCString::Stricmp(Value, TEXT("AutoReject")) == 0)
	{
		return EJoinRequestAction::AutoReject;
	}
	return EJoinRequestAction::Manual;
}

FString ToDebugString(const FPartyConfiguration& PartyConfiguration)
{
	return FString::Printf(TEXT("JoinRequestAction(%s) RemoveOnDisconnect(%d) Publish(%s) Chat(%d) Invite(%s) Accepting(%d) Not Accepting Reason(%d) MaxMembers: %d Nickname: %s Description: %s Password: %s"),
			ToString(PartyConfiguration.JoinRequestAction),
			PartyConfiguration.bShouldRemoveOnDisconnection,
			ToString(PartyConfiguration.PresencePermissions),
			PartyConfiguration.bChatEnabled,
			ToString(PartyConfiguration.InvitePermissions),
			PartyConfiguration.bIsAcceptingMembers,
			PartyConfiguration.NotAcceptingMembersReason,
			PartyConfiguration.MaxMembers,
			*PartyConfiguration.Nickname,
			*PartyConfiguration.Description,
			PartyConfiguration.Password.IsEmpty() ? TEXT("not set") : *PartyConfiguration.Password
		);
}

FString ToDebugString(const IOnlinePartyJoinInfo& JoinInfo)
{
	return JoinInfo.ToDebugString();
}

/**
 * Dump key/value pairs for debugging
 */
FString ToDebugString(const FOnlineKeyValuePairs<FString, FVariantData>& KeyValAttrs)
{
	FString Result;
	bool bPrintedAny = false;
	for (const TPair<FString, FVariantData>& Iterator : KeyValAttrs)
	{
		if (bPrintedAny)
		{
			Result += TEXT(",");
		}
		Result += FString::Printf(TEXT("[%s=%s]"), *Iterator.Key, *Iterator.Value.ToString());
		bPrintedAny = true;
	}
	return Result;
}

/**
* Dump state about the party data for debugging
*/
FString ToDebugString(const FOnlinePartyData& PartyData)
{
	FString Result;
	
	int32 TotalBytesPerSec = PartyData.TotalPackets ? (PartyData.TotalBytes / PartyData.TotalPackets) : 0;
	int32 TotalEffectiveBytesPerSec = PartyData.TotalPackets ? (PartyData.TotalEffectiveBytes / PartyData.TotalPackets) : 0;

	Result += FString::Printf(TEXT("%dB [%d B/pkt], %dB [%d B/pkt], Rev: %d,"), 
		PartyData.TotalBytes, TotalBytesPerSec,
		PartyData.TotalEffectiveBytes, TotalEffectiveBytesPerSec, 
		PartyData.RevisionCount);

	Result += ToDebugString(PartyData.GetKeyValAttrs());
	return Result;
}

FString IOnlinePartyJoinInfo::ToDebugString() const 
{
	return FString::Printf(TEXT("SourceUserId(%s) SourceDisplayName(%s) PartyId(%s) HasKey(%d) HasPassword(%d) IsAcceptingMembers(%d) NotAcceptingReason(%d)"),
		*(GetSourceUserId()->ToDebugString()),
		*(GetSourceDisplayName()),
		*(GetPartyId()->ToDebugString()),
		HasKey() ? 1 : 0, HasPassword() ? 1 : 0,
		IsAcceptingMembers() ? 1 : 0,
		GetNotAcceptingReason());
};
