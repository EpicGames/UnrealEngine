// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/OnlinePresenceInterface.h"

namespace EOnlinePresenceState
{
	const TCHAR* ToString(EOnlinePresenceState::Type EnumVal)
	{
		switch (EnumVal)
		{
		case Online:
			return TEXT("Online");
		case Offline:
			return TEXT("Offline");
		case Away:
			return TEXT("Away");
		case ExtendedAway:
			return TEXT("ExtendedAway");
		case DoNotDisturb:
			return TEXT("DoNotDisturb");
		case Chat:
			return TEXT("Chat");
		}
		return TEXT("");
	}

	EOnlinePresenceState::Type FromString(const TCHAR* StringVal)
	{
		if (FCString::Stricmp(StringVal, TEXT("Online")) == 0)
		{
			return EOnlinePresenceState::Online;
		}
		else if (FCString::Stricmp(StringVal, TEXT("Offline")) == 0)
		{
			return EOnlinePresenceState::Offline;
		}
		else if (FCString::Stricmp(StringVal, TEXT("Away")) == 0)
		{
			return EOnlinePresenceState::Away;
		}
		else if (FCString::Stricmp(StringVal, TEXT("ExtendedAway")) == 0)
		{
			return EOnlinePresenceState::ExtendedAway;
		}
		else if (FCString::Stricmp(StringVal, TEXT("DoNotDisturb")) == 0)
		{
			return EOnlinePresenceState::DoNotDisturb;
		}
		else if (FCString::Stricmp(StringVal, TEXT("Chat")) == 0)
		{
			return EOnlinePresenceState::Chat;
		}
		// Default to Offline / generally unavailable
		return EOnlinePresenceState::Offline;
	}

	// Move these into statics in ToLocText once deprecation period expires
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const FText OnlineText =  NSLOCTEXT("OnlinePresence", "Online", "Online");
	const FText OfflineText =  NSLOCTEXT("OnlinePresence", "Offline", "Offline");
	const FText AwayText =  NSLOCTEXT("OnlinePresence", "Away", "Away");
	const FText DoNotDisturbText =  NSLOCTEXT("OnlinePresence", "DoNotDisturb", "Do Not Disturb");
	const FText ChatText =  NSLOCTEXT("OnlinePresence", "Chat", "Chat");
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** 
	 * @return the loc text version of the enum passed in 
	 */
	const FText ToLocText(EOnlinePresenceState::Type EnumVal)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		switch (EnumVal)
		{
		case Online:
			return OnlineText;
		case Offline:
			return OfflineText;
		case ExtendedAway:
			// falls through to return away text
		case Away:
			return AwayText;
		case DoNotDisturb:
			return DoNotDisturbText;
		case Chat:
			return ChatText;
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		return FText::GetEmpty();
	}
}

// Implementation for OnlineSubsystems that have not implemented this version yet
void IOnlinePresence::SetPresence(const FUniqueNetId& User, FOnlinePresenceSetPresenceParameters&& Parameters, const FOnPresenceTaskCompleteDelegate& Delegate)
{
	TSharedPtr<FOnlineUserPresence> ExistingPresence;
	const bool bNeedsExistingPresence = !Parameters.StatusStr.IsSet() || !Parameters.State.IsSet() || !Parameters.Properties.IsSet();
	if (bNeedsExistingPresence)
	{
		GetCachedPresence(User, ExistingPresence);
	}

	FOnlineUserPresenceStatus LegacyParameters;

	if (Parameters.StatusStr.IsSet())
	{
		LegacyParameters.StatusStr = MoveTemp(Parameters.StatusStr.GetValue());
	}
	else if (ExistingPresence)
	{
		LegacyParameters.StatusStr = ExistingPresence->Status.StatusStr;
	}

	if (Parameters.State.IsSet())
	{
		LegacyParameters.State = Parameters.State.GetValue();
	}
	else if (ExistingPresence)
	{
		LegacyParameters.State = ExistingPresence->Status.State;
	}

	if (Parameters.Properties.IsSet())
	{
		LegacyParameters.Properties = MoveTemp(Parameters.Properties.GetValue());
	}
	else if (ExistingPresence)
	{
		LegacyParameters.Properties = ExistingPresence->Status.Properties;
	}

	SetPresence(User, LegacyParameters, Delegate);
}
