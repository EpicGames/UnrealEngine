// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/OnlineSessionInterface.h"
#include "Interfaces/OnlineSessionDelegates.h"
#include "OnlineSubsystem.h" // IWYU pragma: keep
#include "OnlineError.h"

const TCHAR* ToLogString(EOnSessionParticipantLeftReason LeaveReason)
{
	switch (LeaveReason)
	{
	case EOnSessionParticipantLeftReason::Left:			return TEXT("Left");
	case EOnSessionParticipantLeftReason::Disconnected:	return TEXT("Disconnected");
	case EOnSessionParticipantLeftReason::Kicked:		return TEXT("Kicked");
	case EOnSessionParticipantLeftReason::Closed:		return TEXT("Closed");
	}

	checkNoEntry();
	return TEXT("Invalid");
}

// Workaround for not being able to clear a delegate handle for a lambda while the lambda is being executed
void OnOnlineSessionStartMatchmakingBroadcast(FName DelegateSessionName, bool bWasSuccessful, IOnlineSession* OnlineSession, FName RequestedSessionName, FDelegateHandle* DelegateHandle, const FOnStartMatchmakingComplete CompletionDelegate)
{
	if (DelegateSessionName == RequestedSessionName)
	{
		OnlineSession->ClearOnMatchmakingCompleteDelegate_Handle(*DelegateHandle);
		delete DelegateHandle;

		FOnlineError OnlineError(bWasSuccessful);
		CompletionDelegate.ExecuteIfBound(RequestedSessionName, OnlineError, FSessionMatchmakingResults());
	}
}

bool IOnlineSession::StartMatchmaking(const TArray<FSessionMatchmakingUser>& LocalPlayers, FName SessionName, const FOnlineSessionSettings& NewSessionSettings, TSharedRef<FOnlineSessionSearch>& SearchSettings, const FOnStartMatchmakingComplete& CompletionDelegate)
{
	// TODO: Deprecate other StartMatchmaking function in favor of this when this is implemented on all platforms
	TArray<FUniqueNetIdRef> LocalPlayerIds;
	LocalPlayerIds.Reserve(LocalPlayers.Num());
	for (const FSessionMatchmakingUser& LocalPlayer : LocalPlayers)
	{
		LocalPlayerIds.Emplace(LocalPlayer.UserId);
	}
	bool bStartMatchmakingSuccess = StartMatchmaking(LocalPlayerIds, SessionName, NewSessionSettings, SearchSettings);
	if (bStartMatchmakingSuccess)
	{
		FDelegateHandle* DelegateHandle = new FDelegateHandle;
		*DelegateHandle = AddOnMatchmakingCompleteDelegate_Handle(FOnMatchmakingCompleteDelegate::CreateStatic(OnOnlineSessionStartMatchmakingBroadcast, this, SessionName, DelegateHandle, CompletionDelegate));
	}
	return bStartMatchmakingSuccess;
}

FString IOnlineSession::GetVoiceChatRoomName(int32 LocalUserNum, const FName& SessionName)
{
	UE_LOG_ONLINE_SESSION(Warning, TEXT("[IOnlineSession::GetVoiceChatRoomName] This functionality is not implemented on this OSS."));

	return FString();
}

void IOnlineSession::RemovePlayerFromSession(int32 LocalUserNum, FName SessionName, const FUniqueNetId& TargetPlayerId)
{
	UE_LOG_ONLINE_SESSION(Warning, TEXT("[IOnlineSession::RemovePlayerFromSession] This functionality is not implemented by default."));
}
