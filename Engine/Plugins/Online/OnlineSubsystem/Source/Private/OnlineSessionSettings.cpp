// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSessionSettings.h"
#include "OnlineSubsystem.h" // IWYU pragma: keep
#include "Interfaces/OnlineSessionInterface.h"
#include "Logging/LogScopedVerbosityOverride.h"

void DumpNamedSession(const FNamedOnlineSession* NamedSession)
{
	if (NamedSession != NULL)
	{
		LOG_SCOPE_VERBOSITY_OVERRIDE(LogOnlineSession, ELogVerbosity::VeryVerbose);
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("dumping NamedSession: "));
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("	SessionName: %s"), *NamedSession->SessionName.ToString());	
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("	HostingPlayerNum: %d"), NamedSession->HostingPlayerNum);
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("	SessionState: %s"), EOnlineSessionState::ToString(NamedSession->SessionState));
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("	RegisteredPlayers: "));
		if (NamedSession->RegisteredPlayers.Num())
		{
			for (int32 UserIdx=0; UserIdx < NamedSession->RegisteredPlayers.Num(); UserIdx++)
			{
				UE_LOG_ONLINE_SESSION(Verbose, TEXT("	    %d: %s"), UserIdx, *NamedSession->RegisteredPlayers[UserIdx]->ToDebugString());
			}
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Verbose, TEXT("	    0 registered players"));
		}

		DumpSession(NamedSession);
	}
}

void DumpSession(const FOnlineSession* Session)
{
	if (Session != NULL)
	{
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("dumping Session: "));
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("	OwningPlayerName: %s"), *Session->OwningUserName);	
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("	OwningPlayerId: %s"), Session->OwningUserId.IsValid() ? *Session->OwningUserId->ToDebugString() : TEXT("") );
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("	NumOpenPrivateConnections: %d"), Session->NumOpenPrivateConnections);	
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("	NumOpenPublicConnections: %d"), Session->NumOpenPublicConnections);	
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("	SessionInfo: %s"), Session->SessionInfo.IsValid() ? *Session->SessionInfo->ToDebugString() : TEXT("NULL"));
		DumpSessionSettings(&Session->SessionSettings);
	}
}

void DumpSessionSettings(const FOnlineSessionSettings* SessionSettings)
{
	if (SessionSettings != NULL)
	{
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("dumping SessionSettings: "));
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("\tNumPublicConnections: %d"), SessionSettings->NumPublicConnections);
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("\tNumPrivateConnections: %d"), SessionSettings->NumPrivateConnections);
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("\tbIsLanMatch: %s"), SessionSettings->bIsLANMatch ? TEXT("true") : TEXT("false"));
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("\tbIsDedicated: %s"), SessionSettings->bIsDedicated ? TEXT("true") : TEXT("false"));
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("\tbUsesStats: %s"), SessionSettings->bUsesStats ? TEXT("true") : TEXT("false"));
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("\tbShouldAdvertise: %s"), SessionSettings->bShouldAdvertise ? TEXT("true") : TEXT("false"));
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("\tbAllowJoinInProgress: %s"), SessionSettings->bAllowJoinInProgress ? TEXT("true") : TEXT("false"));
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("\tbAllowInvites: %s"), SessionSettings->bAllowInvites ? TEXT("true") : TEXT("false"));
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("\tbUsesPresence: %s"), SessionSettings->bUsesPresence ? TEXT("true") : TEXT("false"));
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("\tbUseLobbiesIfAvailable: %s"), SessionSettings->bUseLobbiesIfAvailable ? TEXT("true") : TEXT("false"));
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("\tbAllowJoinViaPresence: %s"), SessionSettings->bAllowJoinViaPresence ? TEXT("true") : TEXT("false"));
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("\tbAllowJoinViaPresenceFriendsOnly: %s"), SessionSettings->bAllowJoinViaPresenceFriendsOnly ? TEXT("true") : TEXT("false"));
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("\tBuildUniqueId: 0x%08x"), SessionSettings->BuildUniqueId);
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("\tSettings:"));
		for (FSessionSettings::TConstIterator It(SessionSettings->Settings); It; ++It)
		{
			FName Key = It.Key();
			const FOnlineSessionSetting& Setting = It.Value();
			UE_LOG_ONLINE_SESSION(Verbose, TEXT("\t\t%s=%s"), *Key.ToString(), *Setting.ToString());
		}
	}
}

template<typename ValueType>
void FOnlineSessionSettings::Set(FName Key, const ValueType& Value, EOnlineDataAdvertisementType::Type InType, int32 InID)
{
	FOnlineSessionSetting* Setting = Settings.Find(Key);
	if (Setting)
	{
		Setting->Data.SetValue(Value);
		Setting->AdvertisementType = InType;
		Setting->ID = InID;
	}
	else
	{
		Settings.Add(Key, FOnlineSessionSetting(Value, InType, InID));
	}
}

/** Explicit instantiation of supported types to Set template above */
#if !UE_BUILD_DOCS
template ONLINESUBSYSTEM_API void FOnlineSessionSettings::Set(FName Key, const int32& Value, EOnlineDataAdvertisementType::Type InType, int32 InID);
template ONLINESUBSYSTEM_API void FOnlineSessionSettings::Set(FName Key, const float& Value, EOnlineDataAdvertisementType::Type InType, int32 InID);
template ONLINESUBSYSTEM_API void FOnlineSessionSettings::Set(FName Key, const uint64& Value, EOnlineDataAdvertisementType::Type InType, int32 InID);
template ONLINESUBSYSTEM_API void FOnlineSessionSettings::Set(FName Key, const double& Value, EOnlineDataAdvertisementType::Type InType, int32 InID);
template ONLINESUBSYSTEM_API void FOnlineSessionSettings::Set(FName Key, const FString& Value, EOnlineDataAdvertisementType::Type InType, int32 InID);
template ONLINESUBSYSTEM_API void FOnlineSessionSettings::Set(FName Key, const bool& Value, EOnlineDataAdvertisementType::Type InType, int32 InID);
template ONLINESUBSYSTEM_API void FOnlineSessionSettings::Set(FName Key, const TArray<uint8>& Value, EOnlineDataAdvertisementType::Type InType, int32 InID);
#endif

template<typename ValueType> 
void FOnlineSessionSettings::Set(FName Key, const ValueType& Value, EOnlineDataAdvertisementType::Type InType)
{
	FOnlineSessionSetting* Setting = Settings.Find(Key);
	if (Setting)
	{
		Setting->Data.SetValue(Value);
		Setting->AdvertisementType = InType;
	}
	else
	{
		Settings.Add(Key, FOnlineSessionSetting(Value, InType));
	}
}

/** Explicit instantiation of supported types to Set template above */
#if !UE_BUILD_DOCS
template ONLINESUBSYSTEM_API void FOnlineSessionSettings::Set(FName Key, const int32& Value, EOnlineDataAdvertisementType::Type InType);
template ONLINESUBSYSTEM_API void FOnlineSessionSettings::Set(FName Key, const float& Value, EOnlineDataAdvertisementType::Type InType);
template ONLINESUBSYSTEM_API void FOnlineSessionSettings::Set(FName Key, const uint64& Value, EOnlineDataAdvertisementType::Type InType);
template ONLINESUBSYSTEM_API void FOnlineSessionSettings::Set(FName Key, const double& Value, EOnlineDataAdvertisementType::Type InType);
template ONLINESUBSYSTEM_API void FOnlineSessionSettings::Set(FName Key, const FString& Value, EOnlineDataAdvertisementType::Type InType);
template ONLINESUBSYSTEM_API void FOnlineSessionSettings::Set(FName Key, const bool& Value, EOnlineDataAdvertisementType::Type InType);
template ONLINESUBSYSTEM_API void FOnlineSessionSettings::Set(FName Key, const TArray<uint8>& Value, EOnlineDataAdvertisementType::Type InType);
#endif

void FOnlineSessionSettings::Set(FName Key, const FOnlineSessionSetting& SrcSetting)
{
	FOnlineSessionSetting* Setting = Settings.Find(Key);
	if (Setting)
	{
		Setting->Data = SrcSetting.Data;
		Setting->AdvertisementType = SrcSetting.AdvertisementType;
	}
	else
	{
		Settings.Add(Key, SrcSetting);
	}
}

template<typename ValueType> 
bool FOnlineSessionSettings::Get(FName Key, ValueType& Value) const
{
	const FOnlineSessionSetting* Setting = Settings.Find(Key);
	if (Setting)
	{
		Setting->Data.GetValue(Value);
		return true;
	}

	return false;
}

/** Explicit instantiation of supported types to Get template above */
template ONLINESUBSYSTEM_API bool FOnlineSessionSettings::Get(FName Key, int32& Value) const;
template ONLINESUBSYSTEM_API bool FOnlineSessionSettings::Get(FName Key, float& Value) const;
template ONLINESUBSYSTEM_API bool FOnlineSessionSettings::Get(FName Key, uint64& Value) const;
template ONLINESUBSYSTEM_API bool FOnlineSessionSettings::Get(FName Key, double& Value) const;
template ONLINESUBSYSTEM_API bool FOnlineSessionSettings::Get(FName Key, bool& Value) const;
template ONLINESUBSYSTEM_API bool FOnlineSessionSettings::Get(FName Key, FString& Value) const;
template ONLINESUBSYSTEM_API bool FOnlineSessionSettings::Get(FName Key, TArray<uint8>& Value) const;

bool FOnlineSessionSettings::Remove(FName Key)
{
	return Settings.Remove(Key) > 0;
}

EOnlineDataAdvertisementType::Type FOnlineSessionSettings::GetAdvertisementType(FName Key) const
{
	const FOnlineSessionSetting* Setting = Settings.Find(Key);
	if (Setting)
	{
		return Setting->AdvertisementType;
	}

	UE_LOG_ONLINE_SESSION(Warning, TEXT("Unable to find key for advertisement type request: %s"), *Key.ToString());
	return EOnlineDataAdvertisementType::DontAdvertise;
}

int32 FOnlineSessionSettings::GetID(FName Key) const
{
	const FOnlineSessionSetting* Setting = Settings.Find(Key);
	if (Setting)
	{
		return Setting->ID;
	}

	UE_LOG_ONLINE_SESSION(Warning, TEXT("Unable to find key for ID request: %s"), *Key.ToString());
	return INVALID_SESSION_SETTING_ID;
}

template<typename ValueType>
void FOnlineSearchSettings::Set(FName Key, const ValueType& Value, EOnlineComparisonOp::Type InType, int32 InID)
{
	FOnlineSessionSearchParam* SearchParam = SearchParams.Find(Key);
	if (SearchParam)
	{
		SearchParam->Data.SetValue(Value);
		SearchParam->ComparisonOp = InType;
		SearchParam->ID = InID;
	}
	else
	{
		SearchParams.Add(Key, FOnlineSessionSearchParam(Value, InType, InID));
	}
}

/// @cond DOXYGEN_WARNINGS

/** Explicit instantiation of supported types to Set template above */
template ONLINESUBSYSTEM_API void FOnlineSearchSettings::Set<int32>(FName Key, const int32& Value, EOnlineComparisonOp::Type InType, int32 InID);
template ONLINESUBSYSTEM_API void FOnlineSearchSettings::Set<float>(FName Key, const float& Value, EOnlineComparisonOp::Type InType, int32 InID);
template ONLINESUBSYSTEM_API void FOnlineSearchSettings::Set<uint64>(FName Key, const uint64& Value, EOnlineComparisonOp::Type InType, int32 InID);
template ONLINESUBSYSTEM_API void FOnlineSearchSettings::Set<double>(FName Key, const double& Value, EOnlineComparisonOp::Type InType, int32 InID);
template ONLINESUBSYSTEM_API void FOnlineSearchSettings::Set<FString>(FName Key, const FString& Value, EOnlineComparisonOp::Type InType, int32 InID);
template ONLINESUBSYSTEM_API void FOnlineSearchSettings::Set< TArray<uint8> >(FName Key, const TArray<uint8>& Value, EOnlineComparisonOp::Type InType, int32 InID);
template ONLINESUBSYSTEM_API void FOnlineSearchSettings::Set<bool>(FName Key, const bool& Value, EOnlineComparisonOp::Type InType, int32 InID);

/// @endcond

template<typename ValueType> 
void FOnlineSearchSettings::Set(FName Key, const ValueType& Value, EOnlineComparisonOp::Type InType)
{
	FOnlineSessionSearchParam* SearchParam = SearchParams.Find(Key);
	if (SearchParam)
	{
		SearchParam->Data.SetValue(Value);
		SearchParam->ComparisonOp = InType;
	}
	else
	{
		SearchParams.Add(Key, FOnlineSessionSearchParam(Value, InType));
	}
}

/// @cond DOXYGEN_WARNINGS

/** Explicit instantiation of supported types to Set template above */
template ONLINESUBSYSTEM_API void FOnlineSearchSettings::Set<int32>(FName Key, const int32& Value, EOnlineComparisonOp::Type InType);
template ONLINESUBSYSTEM_API void FOnlineSearchSettings::Set<float>(FName Key, const float& Value, EOnlineComparisonOp::Type InType);
template ONLINESUBSYSTEM_API void FOnlineSearchSettings::Set<uint64>(FName Key, const uint64& Value, EOnlineComparisonOp::Type InType);
template ONLINESUBSYSTEM_API void FOnlineSearchSettings::Set<double>(FName Key, const double& Value, EOnlineComparisonOp::Type InType);
template ONLINESUBSYSTEM_API void FOnlineSearchSettings::Set<FString>(FName Key, const FString& Value, EOnlineComparisonOp::Type InType);
template ONLINESUBSYSTEM_API void FOnlineSearchSettings::Set< TArray<uint8> >(FName Key, const TArray<uint8>& Value, EOnlineComparisonOp::Type InType);
template ONLINESUBSYSTEM_API void FOnlineSearchSettings::Set<bool>(FName Key, const bool& Value, EOnlineComparisonOp::Type InType);

/// @endcond

template<typename ValueType> 
bool FOnlineSearchSettings::Get(FName Key, ValueType& Value) const
{
	const FOnlineSessionSearchParam* SearchParam = SearchParams.Find(Key);
	if (SearchParam)
	{
		SearchParam->Data.GetValue(Value);
		return true;
	}

	return false;
}

/// @cond DOXYGEN_WARNINGS

/** Explicit instantiation of supported types to Get template above */
template ONLINESUBSYSTEM_API bool FOnlineSearchSettings::Get<int32>(FName Key, int32& Value) const;
template ONLINESUBSYSTEM_API bool FOnlineSearchSettings::Get<float>(FName Key, float& Value) const;
template ONLINESUBSYSTEM_API bool FOnlineSearchSettings::Get<uint64>(FName Key, uint64& Value) const;
template ONLINESUBSYSTEM_API bool FOnlineSearchSettings::Get<double>(FName Key, double& Value) const;
template ONLINESUBSYSTEM_API bool FOnlineSearchSettings::Get<FString>(FName Key, FString& Value) const;
template ONLINESUBSYSTEM_API bool FOnlineSearchSettings::Get< TArray<uint8> >(FName Key, TArray<uint8>& Value) const;
template ONLINESUBSYSTEM_API bool FOnlineSearchSettings::Get<bool>(FName Key, bool& Value) const;

/// @endcond

EOnlineComparisonOp::Type FOnlineSearchSettings::GetComparisonOp(FName Key) const
{
	const FOnlineSessionSearchParam* SearchParam = SearchParams.Find(Key);
	if (SearchParam)
	{
		return SearchParam->ComparisonOp;
	}

	UE_LOG_ONLINE_SESSION(Warning, TEXT("Unable to find key for comparison op request: %s"), *Key.ToString());
	return EOnlineComparisonOp::Equals;
}

