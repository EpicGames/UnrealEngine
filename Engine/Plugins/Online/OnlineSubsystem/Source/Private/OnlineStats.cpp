// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineStats.h"

/**
 *	Get a key value pair by key name
 * @param StatName key name to search for
 * @return KeyValuePair if found, NULL otherwise
 */
FVariantData* FOnlineStats::FindStatByName(const FString& StatName)
{
	return Properties.Find(StatName);
}

/**
 * Sets a stat of type float to the value specified. Does nothing
 * if the stat is not of the right type.
 *
 * @param StatName the stat to change the value of
 * @param Value the new value to assign to the stat
 */
void FOnlineStats::SetFloatStat(const FString& StatName, float Value)
{
	FVariantData* Stat = FindStatByName(StatName);
	if (Stat != NULL)
	{
		if (Stat->GetType() == EOnlineKeyValuePairDataType::Float)
		{
			// Set the value
			Stat->SetValue(Value);
		}
	}
	else
	{
		FVariantData NewValue(Value);
		Properties.Add(StatName, NewValue);
	}
}

/**
 * Sets a stat of type int32 to the value specified. Does nothing
 * if the stat is not of the right type.
 *
 * @param StatName the stat to change the value of
 * @param Value the new value to assign to the stat
 */
void FOnlineStats::SetIntStat(const FString& StatName, int32 Value)
{
	FVariantData* Stat = FindStatByName(StatName);
	if (Stat != NULL && Stat->GetType() == EOnlineKeyValuePairDataType::Int32)
	{
		// Set the value
		Stat->SetValue(Value);
	}
	else
	{
		FVariantData NewValue(Value);
		Properties.Add(StatName, NewValue);
	}
}

/**
 * Increments a stat of type float by the value specified. Does nothing
 * if the stat is not of the right type.
 *
 * @param StatName the stat to increment
 * @param IncBy the value to increment by
 */
void FOnlineStats::IncrementFloatStat(const FString& StatName, float IncBy)
{
	FVariantData* Stat = FindStatByName(StatName);
	if (Stat != NULL && Stat->GetType() == EOnlineKeyValuePairDataType::Float)
	{
		// Set the value
		Stat->Increment<float, EOnlineKeyValuePairDataType::Float>(IncBy);
	}
	else
	{
		FVariantData NewValue(IncBy);
		Properties.Add(StatName, NewValue);
	}
}

/**
 * Increments a stat of type int32 by the value specified. Does nothing
 * if the stat is not of the right type.
 *
 * @param StatName the stat to increment
 * @param IncBy the value to increment by
 */
void FOnlineStats::IncrementIntStat(const FString& StatName, int32 IncBy)
{
	FVariantData* Stat = FindStatByName(StatName);
	if (Stat != NULL && Stat->GetType() == EOnlineKeyValuePairDataType::Int32)
	{
		// Set the value
		Stat->Increment<int32, EOnlineKeyValuePairDataType::Int32>(IncBy);
	}
	else
	{
		FVariantData NewValue(IncBy);
		Properties.Add(StatName, NewValue);
	}
}

/**
 * Decrements a stat of type float by the value specified. Does nothing
 * if the stat is not of the right type.
 *
 * @param StatName the stat to decrement
 * @param DecBy the value to decrement by
 */
void FOnlineStats::DecrementFloatStat(const FString& StatName, float DecBy)
{
	FVariantData* Stat = FindStatByName(StatName);
	if (Stat != NULL && Stat->GetType() == EOnlineKeyValuePairDataType::Float)
	{
		// Set the value
		Stat->Decrement<float, EOnlineKeyValuePairDataType::Float>(DecBy);
	}
	else
	{
		FVariantData NewValue(-DecBy);
		Properties.Add(StatName, NewValue);
	}
}

/**
 * Decrements a stat of type int32 by the value specified. Does nothing
 * if the stat is not of the right type.
 *
 * @param StatName the stat to decrement
 * @param DecBy the value to decrement by
 */
void FOnlineStats::DecrementIntStat(const FString& StatName, int32 DecBy)
{
	FVariantData* Stat = FindStatByName(StatName);
	if (Stat != NULL && Stat->GetType() == EOnlineKeyValuePairDataType::Int32)
	{
		// Set the value
		Stat->Decrement<int32, EOnlineKeyValuePairDataType::Int32>(DecBy);
	}
	else
	{
		FVariantData NewValue(-DecBy);
		Properties.Add(StatName, NewValue);
	}
}

FString FOnlineStatsRow::ToLogString() const
{
	FString LogString = FString::Printf(TEXT("%d : %s"), Rank, *NickName);

	for (const TPair<FString, FVariantData>& Column : Columns)
	{
		LogString += FString::Printf(TEXT("\t\t%s : %s"), *Column.Key, *Column.Value.ToString());
	}

	return LogString;
}

FString FOnlineLeaderboardRead::ToLogString() const
{
	FString LogString = FString::Printf(TEXT("\nLeaderboardName: %s\nSortedColumn: %s\nRows:\n"), *LeaderboardName, *SortedColumn);

	for (const FOnlineStatsRow& Row : Rows)
	{
		LogString += FString::Printf(TEXT("\t%s\n"), *Row.ToLogString());
	}

	return LogString;
}