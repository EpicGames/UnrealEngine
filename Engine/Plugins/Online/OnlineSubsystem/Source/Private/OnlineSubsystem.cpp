// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSubsystem.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/TrackedActivity.h"
#include "HAL/IConsoleManager.h"
#include "Online/OnlineSessionNames.h"
#include "OnlineIdentityErrors.h" // IWYU pragma: keep
#include "OnlineSessionSettings.h"
#include "Online/OnlineBase.h"

#include "Interfaces/OnlineChatInterface.h"
#include "Interfaces/OnlineIdentityInterface.h"
#include "Interfaces/OnlineUserInterface.h"
#include "Interfaces/OnlineEventsInterface.h"
#include "Interfaces/OnlineSessionInterface.h"
#include "Interfaces/OnlinePurchaseInterface.h"
#include "Interfaces/OnlineSharingInterface.h"
#include "Interfaces/OnlineFriendsInterface.h"
#include "Interfaces/OnlineExternalUIInterface.h"
#include "Interfaces/OnlineAchievementsInterface.h"
#include "Interfaces/OnlineUserCloudInterface.h"
#include "Interfaces/OnlineTitleFileInterface.h"
#include "Interfaces/OnlinePresenceInterface.h"
#include "Interfaces/VoiceInterface.h"
#include "Interfaces/OnlineLeaderboardInterface.h"
#include "Interfaces/OnlineTournamentInterface.h"
#include "Interfaces/OnlineStatsInterface.h"

LLM_DEFINE_TAG(OnlineSubsystem);

DEFINE_LOG_CATEGORY(LogOnline);
DEFINE_LOG_CATEGORY(LogOnlineGame);
DEFINE_LOG_CATEGORY(LogOnlineChat);
DEFINE_LOG_CATEGORY(LogVoiceEngine);
DEFINE_LOG_CATEGORY(LogOnlineVoice);
DEFINE_LOG_CATEGORY(LogOnlineSession);
DEFINE_LOG_CATEGORY(LogOnlineUser);
DEFINE_LOG_CATEGORY(LogOnlineFriend);
DEFINE_LOG_CATEGORY(LogOnlineIdentity);
DEFINE_LOG_CATEGORY(LogOnlinePresence);
DEFINE_LOG_CATEGORY(LogOnlineExternalUI);
DEFINE_LOG_CATEGORY(LogOnlineAchievements);
DEFINE_LOG_CATEGORY(LogOnlineLeaderboard);
DEFINE_LOG_CATEGORY(LogOnlineCloud);
DEFINE_LOG_CATEGORY(LogOnlineTitleFile);
DEFINE_LOG_CATEGORY(LogOnlineEntitlement);
DEFINE_LOG_CATEGORY(LogOnlineEvents);
DEFINE_LOG_CATEGORY(LogOnlineSharing);
DEFINE_LOG_CATEGORY(LogOnlineStoreV2);
DEFINE_LOG_CATEGORY(LogOnlinePurchase);
DEFINE_LOG_CATEGORY(LogOnlineTournament);
DEFINE_LOG_CATEGORY(LogOnlineStats);

#if STATS
ONLINESUBSYSTEM_API DEFINE_STAT(STAT_Online_Async);
ONLINESUBSYSTEM_API DEFINE_STAT(STAT_Online_AsyncTasks);
ONLINESUBSYSTEM_API DEFINE_STAT(STAT_Session_Interface);
ONLINESUBSYSTEM_API DEFINE_STAT(STAT_Voice_Interface);
#endif

#ifndef NULL_SUBSYSTEM
const FName NULL_SUBSYSTEM(TEXT("NULL"));
#endif

#ifndef GOOGLEPLAY_SUBSYSTEM
const FName GOOGLEPLAY_SUBSYSTEM(TEXT("GOOGLEPLAY"));
#endif

#ifndef IOS_SUBSYSTEM
const FName IOS_SUBSYSTEM(TEXT("IOS"));
#endif

#ifndef APPLE_SUBSYSTEM
const FName APPLE_SUBSYSTEM(TEXT("APPLE"));
#endif

#ifndef AMAZON_SUBSYSTEM
const FName AMAZON_SUBSYSTEM(TEXT("AMAZON"));
#endif

#ifndef GOOGLE_SUBSYSTEM
const FName GOOGLE_SUBSYSTEM(TEXT("GOOGLE"));
#endif

#ifndef FACEBOOK_SUBSYSTEM
const FName FACEBOOK_SUBSYSTEM(TEXT("FACEBOOK"));
#endif

#ifndef GAMECIRCLE_SUBSYSTEM
const FName GAMECIRCLE_SUBSYSTEM(TEXT("GAMECIRCLE"));
#endif

#ifndef STEAM_SUBSYSTEM
const FName STEAM_SUBSYSTEM(TEXT("STEAM"));
#endif

#ifndef PS4_SUBSYSTEM
const FName PS4_SUBSYSTEM(TEXT("PS4"));
#endif

#ifndef PS4SERVER_SUBSYSTEM
const FName PS4SERVER_SUBSYSTEM(TEXT("PS4SERVER"));
#endif

#ifndef PS5_SUBSYSTEM
const FName PS5_SUBSYSTEM(TEXT("PS5"));
#endif

#ifndef GDK_SUBSYSTEM
const FName GDK_SUBSYSTEM(TEXT("GDK"));
#endif

#ifndef THUNDERHEAD_SUBSYSTEM
const FName THUNDERHEAD_SUBSYSTEM(TEXT("THUNDERHEAD"));
#endif

#ifndef MCP_SUBSYSTEM
const FName MCP_SUBSYSTEM(TEXT("MCP"));
#endif

#ifndef TENCENT_SUBSYSTEM
const FName TENCENT_SUBSYSTEM(TEXT("TENCENT"));
#endif

#ifndef NINTENDO_SUBSYSTEM
const FName NINTENDO_SUBSYSTEM(TEXT("NINTENDO"));
#endif

#ifndef SAMSUNG_SUBSYSTEM
const FName SAMSUNG_SUBSYSTEM(TEXT("SAMSUNG"));
#endif

#ifndef ONESTORE_SUBSYSTEM
const FName ONESTORE_SUBSYSTEM(TEXT("ONESTORE"));
#endif

#ifndef EOS_SUBSYSTEM
const FName EOS_SUBSYSTEM(TEXT("EOS"));
#endif

/** The default key that will update presence text in the platform's UI */
const FString DefaultPresenceKey = TEXT("RichPresence");

/** Custom presence data that is not seen by users but can be polled */
const FString CustomPresenceDataKey = TEXT("CustomData");

/** Name of the client that sent the presence update */
const FString DefaultAppIdKey = TEXT("AppId");

/** User friendly name of the client that sent the presence update */
const FString DefaultProductNameKey = TEXT("ProductName");

/** Platform of the client that sent the presence update */
const FString DefaultPlatformKey = TEXT("Platform");

/** Override Id of the client to set the presence state to */
const FString OverrideAppIdKey = TEXT("OverrideAppId");

/** Id of the session for the presence update. @todo samz - SessionId on presence data should be FUniqueNetId not uint64 */
const FString DefaultSessionIdKey = TEXT("SessionId");

/** Resource the client is logged in with */
const FString PresenceResourceKey = TEXT("ResourceKey");

PRAGMA_DISABLE_DEPRECATION_WARNINGS
namespace OnlineIdentity
{
	namespace Errors
	{
		// Params
		const FString AuthLoginParam = TEXT("auth_login");
		const FString AuthTypeParam = TEXT("auth_type");
		const FString AuthPasswordParam = TEXT("auth_password");

		// Results
		const FString NoUserId = TEXT("no_user_id");
		const FString NoAuthToken = TEXT("no_auth_token");
		const FString NoAuthType = TEXT("no_auth_type");
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

/** Workaround, please avoid using this */
FUniqueNetIdPtr GetFirstSignedInUser(IOnlineIdentityPtr IdentityInt)
{
	if (IdentityInt.IsValid())
	{
		// find an entry for a fully logged in user
		for (int32 i = 0; i < MAX_LOCAL_PLAYERS; i++)
		{
			FUniqueNetIdPtr UserId = IdentityInt->GetUniquePlayerId(i);
			if (UserId.IsValid() && UserId->IsValid() && IdentityInt->GetLoginStatus(*UserId) == ELoginStatus::LoggedIn)
			{
				return UserId;
			}
		}
		// find an entry for a locally logged in user
		for (int32 i = 0; i < MAX_LOCAL_PLAYERS; i++)
		{
			FUniqueNetIdPtr UserId = IdentityInt->GetUniquePlayerId(i);
			if (UserId.IsValid() && UserId->IsValid())
			{
				return UserId;
			}
		}
	}
	return nullptr;
}

int32 GetBuildUniqueId()
{
	static bool bStaticCheck = false;
	static int32 BuildId = 0;
	static bool bUseBuildIdOverride = false;
	static int32 BuildIdOverride = 0;
	

	if (!bStaticCheck)
	{
		bStaticCheck = true;
		FString BuildIdOverrideCommandLineString;
		if (FParse::Value(FCommandLine::Get(), TEXT("BuildIdOverride="), BuildIdOverrideCommandLineString))
		{
			BuildIdOverride = FCString::Atoi(*BuildIdOverrideCommandLineString);
		}
		if (BuildIdOverride != 0)
		{
			bUseBuildIdOverride = true;
		}
		else
		{
			if (!GConfig->GetBool(TEXT("OnlineSubsystem"), TEXT("bUseBuildIdOverride"), bUseBuildIdOverride, GEngineIni))
			{
				UE_LOG_ONLINE(Warning, TEXT("Missing bUseBuildIdOverride= in [OnlineSubsystem] of DefaultEngine.ini"));
			}

			if (!GConfig->GetInt(TEXT("OnlineSubsystem"), TEXT("BuildIdOverride"), BuildIdOverride, GEngineIni))
			{
				UE_LOG_ONLINE(Warning, TEXT("Missing BuildIdOverride= in [OnlineSubsystem] of DefaultEngine.ini"));
			}
		}

		if (bUseBuildIdOverride == false)
		{
			// Removed old hashing code to use something more predictable and easier to override for when
			// it's necessary to force compatibility with an older build
			BuildId = FNetworkVersion::GetNetworkCompatibleChangelist();
		}
		else
		{
			BuildId = BuildIdOverride;
		}

		// use a cvar so it can be modified at runtime
		TAutoConsoleVariable<int32>& CVarBuildIdOverride = GetBuildIdOverrideCVar();
		CVarBuildIdOverride->Set(BuildId);
		

		// Will add an info entry to the console
		static FTrackedActivity Ta(TEXT("BuildId"), *FString::FromInt(BuildId), FTrackedActivity::ELight::None, FTrackedActivity::EType::Info);
		CVarBuildIdOverride->SetOnChangedCallback(FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* CVar) { Ta.Update(*CVar->GetString()); }));
	}

	return GetBuildIdOverrideCVar()->GetInt();
}

TAutoConsoleVariable<FString> CVarPlatformOverride(
	TEXT("oss.PlatformOverride"),
	TEXT(""),
	TEXT("Overrides the detected platform of this client for various debugging\n")
	TEXT("Valid values WIN MAC PSN XBL IOS AND LIN SWT SWT2 OTHER"),
	ECVF_Cheat);

FString IOnlineSubsystem::GetLocalPlatformName()
{
	FString OnlinePlatform;

	// Priority: CVar -> Command line -> INI, defaults to OTHER
	OnlinePlatform = CVarPlatformOverride.GetValueOnAnyThread();
#if !UE_BUILD_SHIPPING
	if (OnlinePlatform.IsEmpty())
	{
		FParse::Value(FCommandLine::Get(), TEXT("PLATFORMTEST="), OnlinePlatform);
	}
#endif
	if (OnlinePlatform.IsEmpty())
	{
		// Only read from config once so we don't re-load ini files if they get unloaded. This value is expected to never change. Use the cvar if you need dynamic changing.
		static bool bReadFromConfig = false;
		static FString ConfigValue;
		if (!bReadFromConfig)
		{
			GConfig->GetString(TEXT("OnlineSubsystem"), TEXT("LocalPlatformName"), ConfigValue, GEngineIni);
			// Regardless of whether we found a string in config or not, do not read again
			bReadFromConfig = true;
		}
		OnlinePlatform = ConfigValue;
	}

	if (!OnlinePlatform.IsEmpty())
	{
		OnlinePlatform.ToUpperInline();
	}
	else
	{
		OnlinePlatform = OSS_PLATFORM_NAME_OTHER;
	}
	return OnlinePlatform;
}

bool IsPlayerInSessionImpl(IOnlineSession* SessionInt, FName SessionName, const FUniqueNetId& UniqueId)
{
	bool bFound = false;
	FNamedOnlineSession* Session = SessionInt->GetNamedSession(SessionName);
	if (Session != NULL)
	{
		const bool bIsSessionOwner = Session->OwningUserId.IsValid()  && *Session->OwningUserId == UniqueId;

		FUniqueNetIdMatcher PlayerMatch(UniqueId);
		if (bIsSessionOwner || 
			Session->RegisteredPlayers.IndexOfByPredicate(PlayerMatch) != INDEX_NONE)
		{
			bFound = true;
		}
	}
	return bFound;
}

bool IsUniqueIdLocal(const FUniqueNetId& UniqueId)
{
	if (IOnlineSubsystem::DoesInstanceExist(UniqueId.GetType()))
	{
		IOnlineSubsystem* OnlineSub = IOnlineSubsystem::Get(UniqueId.GetType());
		return OnlineSub ? OnlineSub->IsLocalPlayer(UniqueId) : false;
	}

	return false;
}

int32 GetBeaconPortFromSessionSettings(const FOnlineSessionSettings& SessionSettings)
{
	int32 BeaconListenPort = DEFAULT_BEACON_PORT;
	if (!SessionSettings.Get(SETTING_BEACONPORT, BeaconListenPort) || BeaconListenPort <= 0)
	{
		// Reset the default BeaconListenPort back to DEFAULT_BEACON_PORT because the SessionSettings value does not exist or was not valid
		BeaconListenPort = DEFAULT_BEACON_PORT;
	}
	return BeaconListenPort;
}

#if !UE_BUILD_SHIPPING

static void ResetAchievements()
{
	IOnlineSubsystem* OnlineSub = IOnlineSubsystem::Get();
	IOnlineIdentityPtr IdentityInterface = OnlineSub ? OnlineSub->GetIdentityInterface() : nullptr;
	if (!IdentityInterface.IsValid())
	{
		UE_LOG_ONLINE(Warning, TEXT("ResetAchievements command: couldn't get the identity interface"));
		return;
	}
	
	FUniqueNetIdPtr UserId = IdentityInterface->GetUniquePlayerId(0);
	if(!UserId.IsValid())
	{
		UE_LOG_ONLINE(Warning, TEXT("ResetAchievements command: invalid UserId"));
		return;
	}

	IOnlineAchievementsPtr AchievementsInterface = OnlineSub ? OnlineSub->GetAchievementsInterface() : nullptr;
	if (!AchievementsInterface.IsValid())
	{
		UE_LOG_ONLINE(Warning, TEXT("ResetAchievements command: couldn't get the achievements interface"));
		return;
	}

	AchievementsInterface->ResetAchievements(*UserId);
}

FAutoConsoleCommand CmdResetAchievements(
	TEXT("online.ResetAchievements"),
	TEXT("Reset achievements for the currently logged in user."),
	FConsoleCommandDelegate::CreateStatic(ResetAchievements)
	);

#endif


bool CheckSubsystemDisabledByCommandLine(const FName& SubsystemName)
{
#if (!UE_BUILD_SHIPPING && !UE_BUILD_SHIPPING_WITH_EDITOR) || ENABLE_PGO_PROFILE
	// In non-shipping builds, or PGO_PROFILE enabled Shipping builds, check for a command line override to disable the OSS
	// Cache the results in a static map to avoid string allocations after the first query
	static TMap<FName, bool> DisableMap;
	static FRWLock DisableMapLock;

	// Check the cache first
	{
		UE::TReadScopeLock ReadScope(DisableMapLock);

		bool* FoundValue = DisableMap.Find(SubsystemName);
		if (FoundValue)
		{
			return *FoundValue;
		}
	}

	// Check the actual command line, add to cache
	bool bIsDisabledByCommandLine = FParse::Param(FCommandLine::Get(), *FString::Printf(TEXT("no%s"), *SubsystemName.ToString()));

	{
		UE::TWriteScopeLock WriteScope(DisableMapLock);
		DisableMap.Add(SubsystemName, bIsDisabledByCommandLine);
	}

	return bIsDisabledByCommandLine;
#else
	return false;
#endif
}

bool CheckSubsystemEnabledByConfig(const FName& SubsystemName, const FName& InstanceName, bool& bOutEnabled)
{
	// Map of [SubsystemName, InstanceName] to bEnabled config value, unset if the value is not specified.
	// Note that the subsystem is queried on it's own so there will always be a [SubsystemName, None] entry in the map
	static TMap<TTuple<FName, FName>, TOptional<bool>> EnableMap;
	static FRWLock EnableMapLock;

	TTuple<FName, FName> MapKeyNames = MakeTuple(SubsystemName, InstanceName);

	// Check the cache first
	{
		UE::TReadScopeLock ReadScope(EnableMapLock);
		TOptional<bool>* FoundEnableValue = EnableMap.Find(MapKeyNames);
		if (FoundEnableValue)
		{
			bool bFoundValueIsSet = FoundEnableValue->IsSet();
			if (bFoundValueIsSet)
			{
				bOutEnabled = FoundEnableValue->GetValue();
			}

			return bFoundValueIsSet;
		}
	}

	// The section name is different if an InstanceName is specified
	FString ConfigSection;
	if (InstanceName == NAME_None)
	{
		ConfigSection = FString::Printf(TEXT("OnlineSubsystem%s"), *SubsystemName.ToString());
	}
	else
	{
		ConfigSection = FString::Printf(TEXT("OnlineSubsystem%s %s"), *SubsystemName.ToString(), *InstanceName.ToString());
	}

	bool bFoundConfigValue = GConfig->GetBool(*ConfigSection, TEXT("bEnabled"), bOutEnabled, GEngineIni);

	// Only set the TOptional if a config value was found. Otherwise being unset means the config value was unspecified.
	TOptional<bool> MapValue;
	if (bFoundConfigValue)
	{
		MapValue = bOutEnabled;
	}

	// Write new value to the cache
	{
		UE::TWriteScopeLock WriteScope(EnableMapLock);
		EnableMap.Add(MapKeyNames, MapValue);
	}

	return bFoundConfigValue;
}

bool IOnlineSubsystem::IsEnabled(const FName& SubsystemName, const FName& InstanceName)
{
	bool bIsDisabledByCommandLine = CheckSubsystemDisabledByCommandLine(SubsystemName);
	if (!bIsDisabledByCommandLine)
	{
		// First check the subsystem by itself
		bool bIsEnabledByConfig = false;
		bool bConfigOptionExists = CheckSubsystemEnabledByConfig(SubsystemName, NAME_None, bIsEnabledByConfig);

		// Then check with the instance name if one is specified
		if (InstanceName != NAME_None)
		{
			bConfigOptionExists |= CheckSubsystemEnabledByConfig(SubsystemName, InstanceName, bIsEnabledByConfig);
		}

		UE_CLOG_ONLINE(!bConfigOptionExists, Verbose, TEXT("[OnlineSubsystem%s].bEnabled is not set, defaulting to true"), *SubsystemName.ToString());
	
		return !bConfigOptionExists || bIsEnabledByConfig;
	}
	return false;
}
