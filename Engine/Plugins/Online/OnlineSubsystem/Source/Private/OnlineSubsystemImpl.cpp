// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSubsystemImpl.h"
#include "AutoRTFM.h"
#include "Misc/App.h"
#include "Misc/CoreDelegates.h"
#include "NamedInterfaces.h"
#include "OnlineError.h"
#include "Interfaces/OnlineIdentityInterface.h"
#include "Interfaces/OnlineSessionInterface.h"
#include "Interfaces/OnlineFriendsInterface.h"
#include "Interfaces/OnlinePresenceInterface.h"
#include "Interfaces/OnlinePurchaseInterface.h"

#if UE_BUILD_SHIPPING
#include "Dom/JsonObject.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#endif

namespace OSSConsoleVariables
{
	// CVars
	TAutoConsoleVariable<int32> CVarVoiceLoopback(
		TEXT("OSS.VoiceLoopback"),
		0,
		TEXT("Enables voice loopback\n")
		TEXT("1 Enabled. 0 Disabled."),
		ECVF_Default);
}

const FName FOnlineSubsystemImpl::DefaultInstanceName(TEXT("DefaultInstance"));

FOnlineSubsystemImpl::FOnlineSubsystemImpl(FName InSubsystemName, FName InInstanceName) :
	FOnlineSubsystemImpl(InSubsystemName, InInstanceName, FTSTicker::GetCoreTicker())
{
	// This is just here to prime BuildId variable and add as info in console
	GetBuildUniqueId();
}

FOnlineSubsystemImpl::FOnlineSubsystemImpl(FName InSubsystemName, FName InInstanceName, FTSTicker& Ticker) :
	FTSTickerObjectBase(0.0f, Ticker),
	SubsystemName(InSubsystemName),
	InstanceName(InInstanceName),
	bForceDedicated(false),
	NamedInterfaces(nullptr),
	bTickerStarted(true)
{
	FCoreDelegates::TSOnConfigSectionsChanged().AddRaw(this, &FOnlineSubsystemImpl::OnConfigSectionsChanged);
}

FOnlineSubsystemImpl::~FOnlineSubsystemImpl()
{
}

void FOnlineSubsystemImpl::PreUnload()
{
}

bool FOnlineSubsystemImpl::Shutdown()
{
	FCoreDelegates::TSOnConfigSectionsChanged().RemoveAll(this);
	OnNamedInterfaceCleanup();
	StopTicker();
	return true;
}

FString FOnlineSubsystemImpl::FilterResponseStr(const FString& ResponseStr, const TArray<FString>& RedactFields)
{
#if UE_BUILD_SHIPPING
	const FString Redacted = TEXT("[REDACTED]");
	if (RedactFields.Num() > 0)
	{
		TSharedPtr< FJsonObject > JsonObject;
		TSharedRef< TJsonReader<> > JsonReader = TJsonReaderFactory<>::Create(ResponseStr);
		if (FJsonSerializer::Deserialize(JsonReader, JsonObject) && JsonObject.IsValid())
		{
			for (const auto& RedactField : RedactFields)
			{
				// @todo support redacting other types - issues with GetHashKey using TMultiMap<EJson, FString>
				if (JsonObject->HasTypedField<EJson::String>(RedactField))
				{
					JsonObject->SetStringField(RedactField, Redacted);
				}
			}
			FString NewResponseStr;
			TSharedRef< TJsonWriter<> > Writer = TJsonWriterFactory<>::Create(&NewResponseStr);
			if (FJsonSerializer::Serialize(JsonObject.ToSharedRef(), Writer))
			{
				return NewResponseStr;
			}
		}
	}
	return Redacted;
#else
	return ResponseStr;
#endif
}

void FOnlineSubsystemImpl::ExecuteDelegateNextTick(const FNextTickDelegate& Callback)
{
	UE_AUTORTFM_ONCOMMIT(this, Callback)
	{
		NextTickQueue.Enqueue(Callback);
	};
}

void FOnlineSubsystemImpl::StartTicker()
{
	bTickerStarted = true;
}

void FOnlineSubsystemImpl::StopTicker()
{
	bTickerStarted = false;
}

bool FOnlineSubsystemImpl::Tick(float DeltaTime)
{
	if (bTickerStarted)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSubsystemImpl_Tick);
		if (!NextTickQueue.IsEmpty())
		{
			// unload the next-tick queue into our buffer. Any further executes (from within callbacks) will happen NEXT frame (as intended)
			FNextTickDelegate Temp;
			while (NextTickQueue.Dequeue(Temp))
			{
				CurrentTickBuffer.Add(Temp);
			}

			// execute any functions in the current tick array
			for (const auto& Callback : CurrentTickBuffer)
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSubsystemImpl_Tick_ExecuteCallback);
				Callback.ExecuteIfBound();
			}
			CurrentTickBuffer.SetNum(0, EAllowShrinking::No); // keep the memory around
		}
	}
	return true;
}

void FOnlineSubsystemImpl::PIEEndCleanup()
{
	// As part of the Editor teardown process, we stop any tickers
	StopTicker();
}

void FOnlineSubsystemImpl::InitNamedInterfaces()
{
	NamedInterfaces = NewObject<UNamedInterfaces>();
	if (NamedInterfaces)
	{
		NamedInterfaces->Initialize();
		NamedInterfaces->OnCleanup().AddRaw(this, &FOnlineSubsystemImpl::OnNamedInterfaceCleanup);
		NamedInterfaces->AddToRoot();
	}
}

void FOnlineSubsystemImpl::OnNamedInterfaceCleanup()
{
	if (NamedInterfaces)
	{
		UE_LOG_ONLINE(Display, TEXT("Removing %d named interfaces"), NamedInterfaces->GetNumInterfaces());
		NamedInterfaces->RemoveFromRoot();
		NamedInterfaces->OnCleanup().RemoveAll(this);
		NamedInterfaces = nullptr;
	}
}

UObject* FOnlineSubsystemImpl::GetNamedInterface(FName InterfaceName)
{
	if (!NamedInterfaces)
	{
		InitNamedInterfaces();
	}

	if (NamedInterfaces)
	{
		return NamedInterfaces->GetNamedInterface(InterfaceName);
	}

	return nullptr;
}

void FOnlineSubsystemImpl::SetNamedInterface(FName InterfaceName, UObject* NewInterface)
{
	if (!NamedInterfaces)
	{
		InitNamedInterfaces();
	}

	if (NamedInterfaces)
	{
		return NamedInterfaces->SetNamedInterface(InterfaceName, NewInterface);
	}
}

bool FOnlineSubsystemImpl::IsServer() const
{
#if WITH_EDITOR
	FName WorldContextHandle = (InstanceName != NAME_None && InstanceName != DefaultInstanceName) ? InstanceName : NAME_None;
	return IsServerForOnlineSubsystems(WorldContextHandle);
#else
	return IsServerForOnlineSubsystems(NAME_None);
#endif
}

bool FOnlineSubsystemImpl::IsLocalPlayer(const FUniqueNetId& UniqueId) const
{
	if (!IsDedicated())
	{
		IOnlineIdentityPtr IdentityInt = GetIdentityInterface();
		if (IdentityInt.IsValid())
		{
			for (int32 LocalUserNum = 0; LocalUserNum < MAX_LOCAL_PLAYERS; LocalUserNum++)
			{
				FUniqueNetIdPtr LocalUniqueId = IdentityInt->GetUniquePlayerId(LocalUserNum);
				if (LocalUniqueId.IsValid() && UniqueId == *LocalUniqueId)
				{
					return true;
				}
			}
		}
	}

	return false;
}

IMessageSanitizerPtr FOnlineSubsystemImpl::GetMessageSanitizer(int32 LocalUserNum, FString& OutAuthTypeToExclude) const
{
	IMessageSanitizerPtr MessageSanitizer;
	IOnlineSubsystem* SanitizerSubsystem = IOnlineSubsystem::GetByConfig(TEXT("SanitizerPlatformService"));
	if (!SanitizerSubsystem)
	{
		SanitizerSubsystem = IOnlineSubsystem::GetByPlatform();
	}

	if (SanitizerSubsystem && SanitizerSubsystem != static_cast<const IOnlineSubsystem*>(this))
	{
		MessageSanitizer = SanitizerSubsystem->GetMessageSanitizer(LocalUserNum, OutAuthTypeToExclude);
	}

	if (!MessageSanitizer && FParse::Param(FCommandLine::Get(), TEXT("testsanitizer")))
	{
		SanitizerSubsystem = IOnlineSubsystem::Get(NULL_SUBSYSTEM);
		if (SanitizerSubsystem)
		{
			return SanitizerSubsystem->GetMessageSanitizer(LocalUserNum, OutAuthTypeToExclude);
		}
	}

	return MessageSanitizer;
}

bool FOnlineSubsystemImpl::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	bool bWasHandled = false;

	if (FParse::Command(&Cmd, TEXT("FRIEND")))
	{
		bWasHandled = HandleFriendExecCommands(InWorld, Cmd, Ar);
	}
	else if (FParse::Command(&Cmd, TEXT("IDENTITY")))
	{
		bWasHandled = HandleIdentityExecCommands(InWorld, Cmd, Ar);
	}
	else if (FParse::Command(&Cmd, TEXT("SESSION")))
	{
		bWasHandled = HandleSessionExecCommands(InWorld, Cmd, Ar);
	}
	else if (FParse::Command(&Cmd, TEXT("PRESENCE")))
	{
		bWasHandled = HandlePresenceExecCommands(InWorld, Cmd, Ar);
	}
	else if (FParse::Command(&Cmd, TEXT("PURCHASE")))
	{
		bWasHandled = HandlePurchaseExecCommands(InWorld, Cmd, Ar);
	}
	else if (FParse::Command(&Cmd, TEXT("STORE")))
	{
		bWasHandled = HandleStoreExecCommands(InWorld, Cmd, Ar);
	}

	return bWasHandled;
}

bool FOnlineSubsystemImpl::IsEnabled() const
{
	return IOnlineSubsystem::IsEnabled(SubsystemName, InstanceName == FOnlineSubsystemImpl::DefaultInstanceName ? NAME_None : InstanceName);
}

FText FOnlineSubsystemImpl::GetSocialPlatformName() const
{
	return FText::GetEmpty();
}

void FOnlineSubsystemImpl::DumpReceipts(const FUniqueNetId& UserId)
{
	IOnlinePurchasePtr PurchaseInt = GetPurchaseInterface();
	if (PurchaseInt.IsValid())
	{
		TArray<FPurchaseReceipt> Receipts;
		PurchaseInt->GetReceipts(UserId, Receipts);
		if (Receipts.Num() > 0)
		{
			for (const FPurchaseReceipt& Receipt : Receipts)
			{
				UE_LOG_ONLINE(Display, TEXT("Receipt: %s %s"),
							  *Receipt.TransactionId,
							  LexToString(Receipt.TransactionState));

				UE_LOG_ONLINE(Display, TEXT("-Offers:"));
				for (const FPurchaseReceipt::FReceiptOfferEntry& ReceiptOffer : Receipt.ReceiptOffers)
				{
					UE_LOG_ONLINE(Display, TEXT(" -Namespace: %s Id: %s Quantity: %d"),
								  *ReceiptOffer.Namespace,
								  *ReceiptOffer.OfferId,
								  ReceiptOffer.Quantity);

					UE_LOG_ONLINE(Display, TEXT(" -LineItems:"));
					for (const FPurchaseReceipt::FLineItemInfo& LineItem : ReceiptOffer.LineItems)
					{
						UE_LOG_ONLINE(Display, TEXT("  -Name: %s Id: %s ValidationInfo: %d bytes"),
									  *LineItem.ItemName,
									  *LineItem.UniqueId,
									  LineItem.ValidationInfo.Len());
					}
				}
			}
		}
		else
		{
			UE_LOG_ONLINE(Display, TEXT("No receipts!"));
		}
	}
}

void FOnlineSubsystemImpl::FinalizeReceipts(const FUniqueNetId& UserId)
{
	IOnlinePurchasePtr PurchaseInt = GetPurchaseInterface();
	if (PurchaseInt.IsValid())
	{
		TArray<FPurchaseReceipt> Receipts;
		PurchaseInt->GetReceipts(UserId, Receipts);
		for (const FPurchaseReceipt& Receipt : Receipts)
		{
			UE_LOG_ONLINE(Display, TEXT("Receipt: %s %s"),
				*Receipt.TransactionId,
				LexToString(Receipt.TransactionState));

			TArray<TTuple<FStringView,FStringView>> AlreadyUsedFinalizeInfo;

			for (const FPurchaseReceipt::FReceiptOfferEntry& ReceiptOffer : Receipt.ReceiptOffers)
			{
				UE_LOG_ONLINE(Display, TEXT(" -Namespace: %s Id: %s Quantity: %d"),
					*ReceiptOffer.Namespace,
					*ReceiptOffer.OfferId,
					ReceiptOffer.Quantity);

				UE_LOG_ONLINE(Display, TEXT(" -LineItems:"));
				
				for (const FPurchaseReceipt::FLineItemInfo& LineItem : ReceiptOffer.LineItems)
				{
					UE_LOG_ONLINE(Display, TEXT("  -Name: %s Id: %s ValidationInfo: %d bytes"),
						*LineItem.ItemName,
						*LineItem.UniqueId,
						LineItem.ValidationInfo.Len());

					TTuple<FStringView,FStringView> CurrentFinalizeInfo(LineItem.UniqueId, LineItem.ValidationInfo);
					bool bAlreadyFinalized = AlreadyUsedFinalizeInfo.Find(CurrentFinalizeInfo) != INDEX_NONE;
					if (bAlreadyFinalized)
					{
						UE_LOG_ONLINE(Display, TEXT("Already finalized by sibling LineItem"));					
					}
					else if (!LineItem.IsRedeemable())
					{
						UE_LOG_ONLINE(Display, TEXT("Not redeemable"));
					}
					else
					{
						UE_LOG_ONLINE(Display, TEXT("Finalizing %s!"), *Receipt.TransactionId);
						PurchaseInt->FinalizePurchase(UserId, LineItem.UniqueId, LineItem.ValidationInfo);
						AlreadyUsedFinalizeInfo.AddUnique(MoveTemp(CurrentFinalizeInfo));
					}
				}
			}	
		}
	}
}

void FOnlineSubsystemImpl::OnConfigSectionsChanged(const FString& IniFilename, const TSet<FString>& SectionNames)
{
	if (IniFilename == GEngineIni)
	{
		for (const FString& SectionName : SectionNames)
		{
			if (SectionName.StartsWith(TEXT("OnlineSubsystem")))
			{
				ReloadConfigs(SectionNames);
				break;
			}
		}
	}
}

void FOnlineSubsystemImpl::OnQueryReceiptsComplete(const FOnlineError& Result, FUniqueNetIdPtr UserId)
{
	UE_LOG_ONLINE(Display, TEXT("OnQueryReceiptsComplete %s"), *Result.ToLogString());
	DumpReceipts(*UserId);
}

bool FOnlineSubsystemImpl::HandlePurchaseExecCommands(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	bool bWasHandled = false;
	
	IOnlinePurchasePtr PurchaseInt = GetPurchaseInterface();
	IOnlineIdentityPtr IdentityInt = GetIdentityInterface();
	if (PurchaseInt.IsValid() && IdentityInt.IsValid())
	{
		if (FParse::Command(&Cmd, TEXT("RECEIPTS")))
		{
			FString CommandStr = FParse::Token(Cmd, false);
			if (CommandStr.IsEmpty())
			{
				UE_LOG_ONLINE(Warning, TEXT("usage: PURCHASE RECEIPTS <command> <userid>"));
			}
			else
			{
				FString UserIdStr = FParse::Token(Cmd, false);
				if (UserIdStr.IsEmpty())
				{
					UE_LOG_ONLINE(Warning, TEXT("usage: PURCHASE RECEIPTS <command> <userid>"));
				}
				else
				{
					FUniqueNetIdPtr UserId = IdentityInt->CreateUniquePlayerId(UserIdStr);
					if (UserId.IsValid())
					{
						if (CommandStr == TEXT("RESTORE"))
						{
							FOnQueryReceiptsComplete CompletionDelegate;
							CompletionDelegate.BindRaw(this, &FOnlineSubsystemImpl::OnQueryReceiptsComplete, UserId);
							PurchaseInt->QueryReceipts(*UserId, true, CompletionDelegate);
							
							bWasHandled = true;
						}
						else if (CommandStr == TEXT("DUMP"))
						{
							DumpReceipts(*UserId);
							bWasHandled = true;
						}
						else if (CommandStr == TEXT("FINALIZE"))
						{
							FinalizeReceipts(*UserId);
							bWasHandled = true;
						}
					}
				}
			}
		}
		else if (FParse::Command(&Cmd, TEXT("BUY")))
		{
			FString CommaSeparatedProductIds = FParse::Token(Cmd, false);
			FString UserIdStr = FParse::Token(Cmd, false);
			TArray<FString> ProductIds;
			CommaSeparatedProductIds.ParseIntoArray(ProductIds, TEXT(","), true);
			if (ProductIds.IsEmpty() || UserIdStr.IsEmpty())
			{
				UE_LOG_ONLINE(Warning, TEXT("usage: PURCHASE BUY <comma_separated_productids> <userid>"));
			}
			else
			{
				bWasHandled = true;
				if (FUniqueNetIdPtr UserId = IdentityInt->CreateUniquePlayerId(UserIdStr))
				{
					FPurchaseCheckoutRequest Request;
					Request.AccountId = UserIdStr;
					for (const FString& ProductId : ProductIds)
					{
						Request.AddPurchaseOffer(FString(), ProductId, 1);
					}
					PurchaseInt->Checkout(*UserId, Request, FOnPurchaseCheckoutComplete::CreateLambda([](const FOnlineError& Result, const TSharedRef<FPurchaseReceipt>& Receipt)
						{
							UE_LOG_ONLINE(Log, TEXT("Checkout Result=%s TransactionId=%s TransactionState=%s"), *Result.ToLogString(), *Receipt->TransactionId, LexToString(Receipt->TransactionState));
							for (const FPurchaseReceipt::FReceiptOfferEntry& OfferEntry : Receipt->ReceiptOffers)
							{
								UE_LOG_ONLINE(Log, TEXT("  OfferEntry Namespace=%s OfferId=%s Quantity=%d"), *OfferEntry.Namespace, *OfferEntry.OfferId, OfferEntry.Quantity);
								for (const FPurchaseReceipt::FLineItemInfo& LineItem : OfferEntry.LineItems)
								{
									UE_LOG_ONLINE(Log, TEXT("    LineItem Name=%s UniqueId=%s ValidationInfo=%s"), *LineItem.ItemName, *LineItem.UniqueId, *LineItem.ValidationInfo);
								}
							}
						}));
				}
			}
		}
	}

	return bWasHandled;
}

bool FOnlineSubsystemImpl::HandleStoreExecCommands(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	bool bWasHandled = false;

	IOnlineStoreV2Ptr StoreInt = GetStoreV2Interface();
	IOnlineIdentityPtr IdentityInt = GetIdentityInterface();
	if (StoreInt.IsValid() && IdentityInt.IsValid())
	{
		if (FParse::Command(&Cmd, TEXT("LIST")))
		{
			FString ProductStr = FParse::Token(Cmd, false);
			FString UserIdStr = FParse::Token(Cmd, false);
			if (ProductStr.IsEmpty() || UserIdStr.IsEmpty())
			{
				UE_LOG_ONLINE(Warning, TEXT("usage: STORE LIST <product> <userid>"));
			}
			else
			{
				bWasHandled = true;
				if (FUniqueNetIdPtr UserId = IdentityInt->CreateUniquePlayerId(UserIdStr))
				{
					TArray<FString> ProductIds;
					ProductIds.Add(ProductStr);
					StoreInt->QueryOffersById(*UserId, ProductIds, FOnQueryOnlineStoreOffersComplete::CreateLambda([StoreInt](bool bWasSuccessful, const TArray<FUniqueOfferId>& /*OfferIds*/, const FString& Error) 
					{
						if(!bWasSuccessful)
						{
							UE_LOG_ONLINE(Error, TEXT("QueryOffersById failed: %s"), *Error);
						}
						TArray<FOnlineStoreOfferRef> Offers;
						StoreInt->GetOffers(Offers);
						for (const FOnlineStoreOfferRef& Product : Offers)
						{
							UE_LOG_ONLINE(Log, TEXT("Product Id=%s, Title=%s, Price=%s"),
								*Product->OfferId,
								*Product->Title.ToString(),
								*Product->PriceText.ToString());
						}
					}));
				}
			}
		}
	}
	return bWasHandled;
}

bool FOnlineSubsystemImpl::HandleFriendExecCommands(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	bool bWasHandled = false;

	if (FParse::Command(&Cmd, TEXT("BLOCK")))
	{
		FString LocalNumStr = FParse::Token(Cmd, false);
		int32 LocalNum = FCString::Atoi(*LocalNumStr);

		FString UserId = FParse::Token(Cmd, false);
		if (UserId.IsEmpty() || LocalNum < 0 || LocalNum > MAX_LOCAL_PLAYERS)
		{
			UE_LOG_ONLINE(Warning, TEXT("usage: FRIEND BLOCK <localnum> <userid>"));
		}
		else
		{
			IOnlineIdentityPtr IdentityInt = GetIdentityInterface();
			if (IdentityInt.IsValid())
			{
				FUniqueNetIdPtr BlockUserId = IdentityInt->CreateUniquePlayerId(UserId);
				IOnlineFriendsPtr FriendsInt = GetFriendsInterface();
				if (FriendsInt.IsValid())
				{
					FriendsInt->BlockPlayer(0, *BlockUserId);
				}
			}
		}
	}
	else if (FParse::Command(&Cmd, TEXT("QUERYRECENT")))
	{
		IOnlineIdentityPtr IdentityInt = GetIdentityInterface();
		if (IdentityInt.IsValid())
		{
			int32 LocalUserNum = 0;
			FString LocalUserNumStr = FParse::Token(Cmd, false);
			if (!LocalUserNumStr.IsEmpty())
			{
				LocalUserNum = FCString::Atoi(*LocalUserNumStr);
			}
			FString Namespace = FParse::Token(Cmd, false);

			FUniqueNetIdPtr UserId = IdentityInt->GetUniquePlayerId(LocalUserNum);
			IOnlineFriendsPtr FriendsInt = GetFriendsInterface();
			if (FriendsInt.IsValid())
			{
				FriendsInt->QueryRecentPlayers(*UserId, Namespace);
			}
		}

		bWasHandled = true;
	}
	else if (FParse::Command(&Cmd, TEXT("DUMPRECENT")))
	{
		IOnlineFriendsPtr FriendsInt = GetFriendsInterface();
		if (FriendsInt.IsValid())
		{
			FriendsInt->DumpRecentPlayers();
		}
		bWasHandled = true;
	}
	else if (FParse::Command(&Cmd, TEXT("DUMPBLOCKED")))
	{
		IOnlineFriendsPtr FriendsInt = GetFriendsInterface();
		if (FriendsInt.IsValid())
		{
			FriendsInt->DumpBlockedPlayers();
		}
		bWasHandled = true;
	}
	if (FParse::Command(&Cmd, TEXT("READFRIENDSLIST"))) /* ONLINE FRIEND READFRIENDSLIST LocalUserNum=0 ListName=default */
	{
		IOnlineFriendsPtr FriendsInt = GetFriendsInterface();
		if (FriendsInt.IsValid())
		{
			int LocalUserNum = 0;
			FParse::Value(Cmd, TEXT("LocalUserNum="), LocalUserNum);

			FString ListName;
			FParse::Value(Cmd, TEXT("ListName="), ListName);

			FriendsInt->ReadFriendsList(LocalUserNum, ListName, FOnReadFriendsListComplete());
		}
		bWasHandled = true;
	}
	else if (FParse::Command(&Cmd, TEXT("DUMPFRIENDSLIST"))) /* ONLINE FRIEND DUMPFRIENDSLIST LocalUserNum=0 ListName=default */
	{
		IOnlineFriendsPtr FriendsInt = GetFriendsInterface();
		if (FriendsInt.IsValid())
		{
			int LocalUserNum = 0;
			FParse::Value(Cmd, TEXT("LocalUserNum="), LocalUserNum);

			FString ListName;
			FParse::Value(Cmd, TEXT("ListName="), ListName);

			TArray< TSharedRef<FOnlineFriend> > Friends;
			if (FriendsInt->GetFriendsList(LocalUserNum, ListName, Friends))
			{
				UE_LOG_ONLINE_FRIEND(Log, TEXT("GetFriendsList returned %d friends"), Friends.Num());

				// Log each friend's data out
				for (int32 Index = 0; Index < Friends.Num(); Index++)
				{
					const FOnlineFriend& Friend = *Friends[Index];
					const FOnlineUserPresence& Presence = Friend.GetPresence();
					UE_LOG_ONLINE_FRIEND(Log, TEXT("\tIdx %d DisplayName (%s) NetId (%s)"), Index, *Friend.GetDisplayName(), *Friend.GetUserId()->ToDebugString());
					UE_LOG_ONLINE_FRIEND(Log, TEXT("\t\t Invite status (%s)"), EInviteStatus::ToString(Friend.GetInviteStatus()));
					UE_LOG_ONLINE_FRIEND(Log, TEXT("\t\t Presence: %s"), *Presence.Status.StatusStr);
					UE_LOG_ONLINE_FRIEND(Log, TEXT("\t\t State: %s"), EOnlinePresenceState::ToString(Presence.Status.State));
					UE_LOG_ONLINE_FRIEND(Log, TEXT("\t\t bIsOnline (%s)"), *LexToString(Presence.bIsOnline));
					UE_LOG_ONLINE_FRIEND(Log, TEXT("\t\t bIsPlaying (%s)"), *LexToString(Presence.bIsPlaying));
					UE_LOG_ONLINE_FRIEND(Log, TEXT("\t\t bIsPlayingThisGame (%s)"), *LexToString(Presence.bIsPlayingThisGame));
					UE_LOG_ONLINE_FRIEND(Log, TEXT("\t\t bIsJoinable (%s)"), *LexToString(Presence.bIsJoinable));
					UE_LOG_ONLINE_FRIEND(Log, TEXT("\t\t bHasVoiceSupport (%s)"), *LexToString(Presence.bHasVoiceSupport));
				}
			}
			else
			{
				UE_LOG_ONLINE(Warning, TEXT("[%hs] Friend list with name %s was not found"), __FUNCTION__, *ListName);
			}
		}
		bWasHandled = true;
	}
	else if (FParse::Command(&Cmd, TEXT("GETDISPLAYNAME"))) /* ONLINE FRIEND GETDISPLAYNAME LocalUserNum=0 ListName=default FriendIndex=0 Platform=Epic */
	{
		IOnlineFriendsPtr FriendsInt = GetFriendsInterface();
		if (FriendsInt.IsValid())
		{
			int LocalUserNum = 0;
			FParse::Value(Cmd, TEXT("LocalUserNum="), LocalUserNum);

			FString ListName;
			FParse::Value(Cmd, TEXT("ListName="), ListName);

			int FriendIndex = 0;
			FParse::Value(Cmd, TEXT("FriendIndex="), FriendIndex);

			TArray<TSharedRef<FOnlineFriend>> FriendList;
			FriendsInt->GetFriendsList(LocalUserNum, ListName, FriendList);

			if (FriendList.IsValidIndex(FriendIndex))
			{
				const TSharedRef<FOnlineFriend>& Friend = FriendList[FriendIndex];

				FString Platform;
				FParse::Value(Cmd, TEXT("Platform="), Platform);

				const FString DisplayName = Friend->GetDisplayName(Platform);

				UE_LOG_ONLINE(Log, TEXT("[%hs] GetDisplayName(%s) call for user %s returned with: %s"), __FUNCTION__, *Platform, *Friend->GetUserId()->ToDebugString(), *DisplayName);
			}
			else
			{
				UE_LOG_ONLINE(Warning, TEXT("[%hs] Index %d not valid for friend list %s"), __FUNCTION__, FriendIndex, *ListName);
			}
		}

		bWasHandled = true;
	}

	return bWasHandled;
}

bool FOnlineSubsystemImpl::HandleIdentityExecCommands(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	bool bWasHandled = false;

	if (FParse::Command(&Cmd, TEXT("TRIGGERONAUTHABOUTTOEXPIREDELEGATES"))) /* ONLINE IDENTITY TRIGGERONAUTHABOUTTOEXPIREDELEGATES LocalUserNum=0 */
	{
		IOnlineIdentityPtr IdentityInt = GetIdentityInterface();
		if (IdentityInt.IsValid())
		{
			int LocalUserNum = 0;
			FParse::Value(Cmd, TEXT("LocalUserNum="), LocalUserNum);

			IdentityInt->TriggerOnAuthAboutToExpireDelegates(LocalUserNum);
		}
		
		bWasHandled = true;
	}

	return bWasHandled;
}

bool FOnlineSubsystemImpl::HandlePresenceExecCommands(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	bool bWasHandled = false;
	if (FParse::Command(&Cmd, TEXT("DUMP")))
	{
		IOnlinePresencePtr PresenceInt = GetPresenceInterface();
		if (PresenceInt.IsValid())
		{
			IOnlinePresence::FOnPresenceTaskCompleteDelegate CompletionDelegate = IOnlinePresence::FOnPresenceTaskCompleteDelegate::CreateLambda([this](const FUniqueNetId& UserId, const bool bWasSuccessful)
			{
				UE_LOG_ONLINE(Display, TEXT("Presence [%s]"), *UserId.ToDebugString());
				if (bWasSuccessful)
				{
					IOnlinePresencePtr LambdaPresenceInt = GetPresenceInterface();
					if (LambdaPresenceInt.IsValid())
					{
						TSharedPtr<FOnlineUserPresence> UserPresence;
						if (LambdaPresenceInt->GetCachedPresence(UserId, UserPresence) == EOnlineCachedResult::Success &&
							UserPresence.IsValid())
						{
							UE_LOG_ONLINE(Display, TEXT("- %s"), *UserPresence->ToDebugString());
						}
						else
						{
							UE_LOG_ONLINE(Display, TEXT("Failed to get cached presence"));
						}
					}
				}
				else
				{
					UE_LOG_ONLINE(Display, TEXT("Failed to query presence"));
				}
			});

			TArray<TSharedRef<FOnlineFriend>> FriendsList;
			IOnlineFriendsPtr FriendsInt = GetFriendsInterface();
			if (FriendsInt.IsValid())
			{
				FriendsInt->GetFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FriendsList);
			}

			// Query and dump friends presence
			for (const TSharedRef<FOnlineFriend>& Friend : FriendsList)
			{
				PresenceInt->QueryPresence(*Friend->GetUserId(), CompletionDelegate);
			}

			// Query own presence
			FUniqueNetIdPtr UserId = GetFirstSignedInUser(GetIdentityInterface());
			if (UserId.IsValid())
			{
				PresenceInt->QueryPresence(*UserId, CompletionDelegate);
			}
		}
		bWasHandled = true;
	}

	return bWasHandled;
}

bool FOnlineSubsystemImpl::HandleSessionExecCommands(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	bool bWasHandled = false;

	if (FParse::Command(&Cmd, TEXT("DUMP")))
	{
		IOnlineSessionPtr SessionsInt = GetSessionInterface();
		if (SessionsInt.IsValid())
		{
			SessionsInt->DumpSessionState();
		}
		bWasHandled = true;
	}

	return bWasHandled;
}

IOnlineGroupsPtr FOnlineSubsystemImpl::GetGroupsInterface() const
{
	return nullptr;
}

IOnlinePartyPtr FOnlineSubsystemImpl::GetPartyInterface() const
{
	return nullptr;
}

IOnlineSharedCloudPtr FOnlineSubsystemImpl::GetSharedCloudInterface() const
{
	return nullptr;
}

IOnlineUserCloudPtr FOnlineSubsystemImpl::GetUserCloudInterface() const
{
	return nullptr;
}

IOnlineEntitlementsPtr FOnlineSubsystemImpl::GetEntitlementsInterface() const
{
	return nullptr;
}

IOnlineLeaderboardsPtr FOnlineSubsystemImpl::GetLeaderboardsInterface() const
{
	return nullptr;
}

IOnlineVoicePtr FOnlineSubsystemImpl::GetVoiceInterface() const
{
	return nullptr;
}

IOnlineExternalUIPtr FOnlineSubsystemImpl::GetExternalUIInterface() const
{
	return nullptr;
}

IOnlineTimePtr FOnlineSubsystemImpl::GetTimeInterface() const
{
	return nullptr;
}

IOnlineIdentityPtr FOnlineSubsystemImpl::GetIdentityInterface() const
{
	return nullptr;
}

IOnlineTitleFilePtr FOnlineSubsystemImpl::GetTitleFileInterface() const
{
	return nullptr;
}

IOnlineStoreV2Ptr FOnlineSubsystemImpl::GetStoreV2Interface() const
{
	return nullptr;
}

IOnlinePurchasePtr FOnlineSubsystemImpl::GetPurchaseInterface() const
{
	return nullptr;
}

IOnlineEventsPtr FOnlineSubsystemImpl::GetEventsInterface() const
{
	return nullptr;
}

IOnlineAchievementsPtr FOnlineSubsystemImpl::GetAchievementsInterface() const
{
	return nullptr;
}

IOnlineSharingPtr FOnlineSubsystemImpl::GetSharingInterface() const
{
	return nullptr;
}

IOnlineUserPtr FOnlineSubsystemImpl::GetUserInterface() const
{
	return nullptr;
}

IOnlineMessagePtr FOnlineSubsystemImpl::GetMessageInterface() const
{
	return nullptr;
}

IOnlinePresencePtr FOnlineSubsystemImpl::GetPresenceInterface() const
{
	return nullptr;
}

IOnlineChatPtr FOnlineSubsystemImpl::GetChatInterface() const
{
	return nullptr;
}

IOnlineStatsPtr FOnlineSubsystemImpl::GetStatsInterface() const
{
	return nullptr;
}

IOnlineGameActivityPtr FOnlineSubsystemImpl::GetGameActivityInterface() const
{
	return nullptr;
}

IOnlineGameItemStatsPtr FOnlineSubsystemImpl::GetGameItemStatsInterface() const
{
	return nullptr;
}

IOnlineGameMatchesPtr FOnlineSubsystemImpl::GetGameMatchesInterface() const
{
	return nullptr;
}

IOnlineTurnBasedPtr FOnlineSubsystemImpl::GetTurnBasedInterface() const
{
	return nullptr;
}

IOnlineTournamentPtr FOnlineSubsystemImpl::GetTournamentInterface() const
{
	return nullptr;
}

IOnlineContentAgeRestrictionPtr FOnlineSubsystemImpl::GetOnlineContentAgeRestrictionInterface() const
{
	return nullptr;
}
