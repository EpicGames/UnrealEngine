// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSubsystemModule.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ScopeLock.h"
#include "OnlineSubsystemImpl.h"
#include "OnlineDelegates.h"

IMPLEMENT_MODULE( FOnlineSubsystemModule, OnlineSubsystem );

namespace
{
	static bool bPostStartupModule = false;
}

/**
 * OnlineDelegates.h static declarations
 */
FOnlineSubsystemDelegates::FOnOnlineSubsystemCreated FOnlineSubsystemDelegates::OnOnlineSubsystemCreated;
FOnlineSubsystemDelegates::FOnDefaultOnlineSubsystemPreReloaded FOnlineSubsystemDelegates::OnDefaultOnlineSubsystemPreReloaded;

static inline FName GetOnlineModuleName(const FString& SubsystemName, const TMap<FString, FName>& ModuleRedirects)
{
	FString ModuleBase(TEXT("OnlineSubsystem"));

	FName ModuleName;
	if (!SubsystemName.StartsWith(ModuleBase, ESearchCase::CaseSensitive))
	{
		const FName* const ModuleOverride = ModuleRedirects.Find(SubsystemName);
		if (ModuleOverride != nullptr)
		{
			ModuleName = *ModuleOverride;
		}
		else
		{
			ModuleName = FName(*(ModuleBase + SubsystemName));
		}
	}
	else
	{
		ModuleName = FName(*SubsystemName);
	}

	return ModuleName;
}

/**
 * Helper function that loads a given platform service module if it isn't already loaded
 *
 * @param SubsystemName Name of the requested platform service to load
 * @return The module interface of the requested platform service, NULL if the service doesn't exist
 */
static IModuleInterface* LoadSubsystemModule(const FString& SubsystemName, const TMap<FString, FName>& ModuleRedirects)
{
	const bool bAttemptLoadModule = IOnlineSubsystem::IsEnabled(FName(*SubsystemName));
	if (bAttemptLoadModule)
	{
		const FName ModuleName = GetOnlineModuleName(SubsystemName, ModuleRedirects);
		FModuleManager& ModuleManager = FModuleManager::Get();

		if (!ModuleManager.IsModuleLoaded(ModuleName))
		{
			UE_CLOGF(bPostStartupModule, LogOnline, Warning, "%s attempting to load module \"%ls\" after FOnlineSubsystemModule::StartupModule. This can result in shutdown issues due to the loaded module unloading before OSS itself. Please add the module to the [OnlineSubsystem] AdditionalModulesToLoad config array to fix.", __FUNCTION__, *ModuleName.ToString());
			// Attempt to load the module
			ModuleManager.LoadModule(ModuleName);
		}

		return ModuleManager.GetModule(ModuleName);
	}

	return nullptr;
}

void FOnlineSubsystemModule::StartupModule()
{
	// These should not be LoadModuleChecked because these modules might not exist
	// For all modules loaded here, we want to ensure they will still exist during ShutdownModule.
	// We will always load these modules at the cost of extra modules loaded for the few OSS (like Null) that don't use it.
	TArray<FString> AdditionalModulesToLoad;
	GConfig->GetArray(TEXT("OnlineSubsystem"), TEXT("AdditionalModulesToLoad"), AdditionalModulesToLoad, GEngineIni);
	for (const FString& AdditionalModule : AdditionalModulesToLoad)
	{
		if (FModuleManager::Get().ModuleExists(*AdditionalModule))
		{
			FModuleManager::Get().LoadModule(*AdditionalModule);
		}
	}

	ProcessConfigDefinedModuleRedirects();

	// Also load the console/platform specific OSS which might not necessarily be the default OSS instance
	FString InterfaceString;
	GConfig->GetString(TEXT("OnlineSubsystem"), TEXT("NativePlatformService"), InterfaceString, GEngineIni);
	NativePlatformService = FName(*InterfaceString);

	// Some default OSSes may rely on the native OSS for functionality. This config is to ensure the native is loaded first in cases where this is desired

	bool bLoadNativeOSSBeforeDefault = false;
	GConfig->GetBool(TEXT("OnlineSubsystem"), TEXT("bLoadNativeOSSBeforeDefault"), bLoadNativeOSSBeforeDefault, GEngineIni);

	if(bLoadNativeOSSBeforeDefault)
	{
		IOnlineSubsystem::GetByPlatform();
		LoadDefaultSubsystem();
		ProcessConfigDefinedSubsystems();
	}
	else
	{
		LoadDefaultSubsystem();
		ProcessConfigDefinedSubsystems();
		IOnlineSubsystem::GetByPlatform();
	}

	bPostStartupModule = true;
}

void FOnlineSubsystemModule::PreUnloadCallback()
{
	PreUnloadOnlineSubsystem();
}

void FOnlineSubsystemModule::ShutdownModule()
{
	ShutdownOnlineSubsystem();
}

static inline void ReadOnlineSubsystemConfigPairs(const TCHAR* Section, const TCHAR* Key, const FString& ConfigFile, TArray<TPair<FString, FString>>& OutPairs)
{
	TArray<FString> ConfigPairs;
	GConfig->GetArray(Section, Key, ConfigPairs, ConfigFile);
	OutPairs.Reserve(ConfigPairs.Num());

	ParseOnlineSubsystemConfigPairs(ConfigPairs, OutPairs);
}

void FOnlineSubsystemModule::ProcessConfigDefinedSubsystems()
{
	// Takes on the pattern "(ServiceNameString=SubsystemName)"
	// For example "(GameFeature=NULL)" to have OnlineSubsystemNull be the provider for "GameFeature"
	TArray<TPair<FString, FString>> ServicePairs;
	ReadOnlineSubsystemConfigPairs(TEXT("OnlineSubsystem"), TEXT("ConfigDefinedPlatformServices"), GEngineIni, ServicePairs);

	for (TPair<FString, FString>& ServicePair : ServicePairs)
	{
		UE_LOGF(LogOnline, Verbose, "ConfigDefinedPlatformServices: Associating OnlineSubsystem %ls with identifier %ls", *ServicePair.Value, *ServicePair.Key);
		ConfigDefinedSubsystems.Add(MoveTemp(ServicePair.Key), FName(*ServicePair.Value));
	}
}

void FOnlineSubsystemModule::ProcessConfigDefinedModuleRedirects()
{
	// Takes on the pattern "(SubsystemName=ModuleName)"
	// For example "(Test=OnlineSubsystemNull)" to have OnlineSubsystemNull be the provider for "Test"
	TArray<TPair<FString, FString>> RedirectPairs;
	ReadOnlineSubsystemConfigPairs(TEXT("OnlineSubsystem"), TEXT("ModuleRedirects"), GEngineIni, RedirectPairs);

	for (TPair<FString, FString>& RedirectPair : RedirectPairs)
	{
		UE_LOGF(LogOnline, Verbose, "ProcessConfigDefinedModuleRedirects: Associating module %ls with OnlineSubsystem %ls", *RedirectPair.Value, *RedirectPair.Key);
		ModuleRedirects.Emplace(MoveTemp(RedirectPair.Key), FName(*RedirectPair.Value));
	}
}

bool FOnlineSubsystemModule::TryLoadSubsystemAndSetDefault(FName SubsystemName)
{
	// A module loaded with its factory method set for creation and a default instance of the online subsystem is required
	bool bLoaded = false;
	const FString SubsystemNameString = SubsystemName.ToString();
	if (IOnlineSubsystem::IsEnabled(SubsystemName))
	{
		if (LoadSubsystemModule(SubsystemNameString, ModuleRedirects))
		{
			if (OnlineFactories.Contains(SubsystemName))
			{
				IOnlineSubsystem* OnlineSubsystem = GetOnlineSubsystem(SubsystemName);
				if (OnlineSubsystem != nullptr)
				{
					UE_LOG_ONLINE(Log, TEXT("TryLoadSubsystemAndSetDefault: Loaded subsystem for type [%s]"), *SubsystemNameString);
					DefaultPlatformService = SubsystemName;
					bLoaded = true;
				}
				else
				{
					//UE_LOG_ONLINE(Warning, TEXT("TryLoadSubsystemAndSetDefault: GetOnlineSubsystem([%s]) failed"), *SubsystemNameString);
				}
			}
			else
			{
				UE_LOG_ONLINE(Warning, TEXT("TryLoadSubsystemAndSetDefault: OnlineFactories does not contain [%s]"), *SubsystemNameString);
			}
		}
		else
		{
			UE_LOG_ONLINE(Warning, TEXT("TryLoadSubsystemAndSetDefault: LoadSubsystemModule([%s]) failed"), *SubsystemNameString);
		}
	}
	else
	{
		UE_LOG_ONLINE(Log, TEXT("TryLoadSubsystemAndSetDefault: [%s] disabled"), *SubsystemNameString);
	}

	return bLoaded;
}

void FOnlineSubsystemModule::LoadDefaultSubsystem()
{
	FString InterfaceString;
	GConfig->GetString(TEXT("OnlineSubsystem"), TEXT("DefaultPlatformService"), InterfaceString, GEngineIni);

	bool bHasLoadedModule = false;
	if (InterfaceString.Len() > 0)
	{
		bHasLoadedModule = TryLoadSubsystemAndSetDefault(FName(*InterfaceString));
	}

	// if default fails, attempt to load Null
	if (!bHasLoadedModule)
	{
		//UE_LOG_ONLINE(Warning, TEXT("LoadDefaultSubsystem: Failed to load Subsystem=[%s], falling back to NULL_SUBSYSTEM"), *InterfaceString);
		bHasLoadedModule = TryLoadSubsystemAndSetDefault(NULL_SUBSYSTEM);
	}

	if (!bHasLoadedModule)
	{
		UE_LOG_ONLINE(Log, TEXT("Failed to load any Online Subsystem Modules"));
	}
}

void FOnlineSubsystemModule::ReloadDefaultSubsystem()
{
	if (IOnlineSubsystem* OSS = GetOnlineSubsystem(DefaultPlatformService))
	{
		FOnlineSubsystemDelegates::OnDefaultOnlineSubsystemPreReloaded.Broadcast(OSS);
	}
	DestroyOnlineSubsystem(DefaultPlatformService);
	// Clear our InstanceNames cache so we can re-establish it in case the DefaultPlatformService
	InstanceNames.Empty();
	LoadDefaultSubsystem();
}

void FOnlineSubsystemModule::PreUnloadOnlineSubsystem()
{
	// Shutdown all online subsystem instances
	UE::TScopeLock Lock(OnlineSubsystemsLock);
	for (TMap<FName, IOnlineSubsystemPtr>::TIterator It(OnlineSubsystems); It; ++It)
	{
		It.Value()->PreUnload();
	}
}

void FOnlineSubsystemModule::ShutdownOnlineSubsystem()
{
	FModuleManager& ModuleManager = FModuleManager::Get();

	// Shutdown all online subsystem instances
	{
		UE::TScopeLock Lock(OnlineSubsystemsLock);
		for (TMap<FName, IOnlineSubsystemPtr>::TIterator It(OnlineSubsystems); It; ++It)
		{
			It.Value()->Shutdown();
		}
		OnlineSubsystems.Empty();
	}

	// Unload all the supporting factories
	for (TMap<FName, IOnlineFactory*>::TIterator It(OnlineFactories); It; ++It)
	{
		UE_LOG_ONLINE(Verbose, TEXT("Unloading online subsystem: %s"), *It.Key().ToString());

		// Unloading the module will do proper cleanup
		FName ModuleName = GetOnlineModuleName(It.Key().ToString(), ModuleRedirects);

		It.RemoveCurrent(); // Remove safely from map otherwise UnloadModule will do it and kill our iteration

		const bool bIsShutdown = true;
		ModuleManager.UnloadModule(ModuleName, bIsShutdown);
	} 
	//ensure(OnlineFactories.Num() == 0);
}

void FOnlineSubsystemModule::RegisterPlatformService(const FName FactoryName, IOnlineFactory* Factory)
{
	if (!OnlineFactories.Contains(FactoryName))
	{
		OnlineFactories.Add(FactoryName, Factory);
	}
}

void FOnlineSubsystemModule::UnregisterPlatformService(const FName FactoryName)
{
	if (OnlineFactories.Contains(FactoryName))
	{
		OnlineFactories.Remove(FactoryName);
	}
}

void FOnlineSubsystemModule::EnumerateOnlineSubsystems(FEnumerateOnlineSubsystemCb& EnumCb)
{
	UE::TScopeLock Lock(OnlineSubsystemsLock);
	for (TPair<FName, IOnlineSubsystemPtr>& OnlineSubsystem : OnlineSubsystems)
	{
		if (OnlineSubsystem.Value.IsValid())
		{
			EnumCb(OnlineSubsystem.Value.Get());
		}
	}
}

FName FOnlineSubsystemModule::ParseOnlineSubsystemName(const FName& FullName, FName& SubsystemName, FName& InstanceName) const
{
	FInstanceNameEntry* Entry = InstanceNames.Find(FullName);
	if (Entry)
	{
		SubsystemName = Entry->SubsystemName;
		InstanceName = Entry->InstanceName;	
	}
	else
	{
		SubsystemName = DefaultPlatformService;
		InstanceName = FOnlineSubsystemImpl::DefaultInstanceName;

		if (!FullName.IsNone())
		{
			FString FullNameStr = FullName.ToString();

			int32 DelimIdx = INDEX_NONE;
			static const TCHAR InstanceDelim = ':';
			if (FullNameStr.FindChar(InstanceDelim, DelimIdx))
			{
				if (DelimIdx > 0)
				{
					SubsystemName = FName(*FullNameStr.Left(DelimIdx));
				}

				if ((DelimIdx + 1) < FullNameStr.Len())
				{
					InstanceName = FName(*FullNameStr.RightChop(DelimIdx + 1));
				}
			}
			else
			{
				SubsystemName = FName(*FullNameStr);
			}			
		}

		Entry = &InstanceNames.Add(FullName);
		Entry->SubsystemName = SubsystemName;
		Entry->InstanceName = InstanceName;
		Entry->FullPath = FName(*FString::Printf(TEXT("%s:%s"), *SubsystemName.ToString(), *InstanceName.ToString()));
	}
	return Entry->FullPath;
}

namespace
{
IOnlineSubsystemPtr FindExistingSubsystem(const TMap<FName, IOnlineSubsystemPtr>& OnlineSubsystems, const FName& KeyName)
{
	IOnlineSubsystemPtr Result;
	if (const IOnlineSubsystemPtr* ExistingOnlineSubsystem = OnlineSubsystems.Find(KeyName))
	{
		Result = *ExistingOnlineSubsystem;
	}
	return Result;
}

IOnlineFactory* FindOrLoadOnlineFactory(const TMap<FName, IOnlineFactory*>& OnlineFactories, const TMap<FString, FName>& ModuleRedirects, const FName& SubsystemName)
{
	IOnlineFactory* const * ExistingFactory = OnlineFactories.Find(SubsystemName);
	if (ExistingFactory == nullptr)
	{
		// Attempt to load the requested factory. This is expected to modify OnlineFactories.
		if (LoadSubsystemModule(SubsystemName.ToString(), ModuleRedirects))
		{
			// If the module loaded successfully this should be non-NULL
			ExistingFactory = OnlineFactories.Find(SubsystemName);
		}
	}
	return ExistingFactory ? *ExistingFactory : nullptr;
}
} // anonymous

IOnlineSubsystem* FOnlineSubsystemModule::GetOnlineSubsystem(const FName InSubsystemName)
{
	FName SubsystemName, InstanceName;
	FName KeyName = ParseOnlineSubsystemName(InSubsystemName, SubsystemName, InstanceName);

	IOnlineSubsystemPtr OnlineSubsystem;
	if (!SubsystemName.IsNone())
	{
		bool bWasNewlyCreated = false;
		{
			UE::TScopeLock Lock(OnlineSubsystemsLock);
			OnlineSubsystem = FindExistingSubsystem(OnlineSubsystems, KeyName);
			if (!OnlineSubsystem)
			{
				if (IOnlineSubsystem::IsEnabled(SubsystemName))
				{
					IOnlineFactory* OSSFactory = FindOrLoadOnlineFactory(OnlineFactories, ModuleRedirects, SubsystemName);
					if (OSSFactory != nullptr)
					{
						OnlineSubsystem = OSSFactory->CreateSubsystem(InstanceName);
						if (OnlineSubsystem.IsValid())
						{
							UE_LOG_ONLINE(Log, TEXT("Created online subsystem instance for: %s"), *InSubsystemName.ToString());

							OnlineSubsystems.Add(KeyName, OnlineSubsystem);
							bWasNewlyCreated = true;
						}
						else
						{
							const bool bNotedPreviously = OnlineSubsystemFailureNotes.Contains(KeyName);
							if (!bNotedPreviously)
							{
								UE_LOG_ONLINE(Log, TEXT("Unable to create OnlineSubsystem instance %s"), (InstanceName == FOnlineSubsystemImpl::DefaultInstanceName) ? *SubsystemName.ToString() : *KeyName.ToString());
								OnlineSubsystemFailureNotes.Add(KeyName);
							}
						}
					}
				}
			}
		}
		if (bWasNewlyCreated)
		{
			FOnlineSubsystemDelegates::OnOnlineSubsystemCreated.Broadcast(OnlineSubsystem.Get());
		}
	}

	return OnlineSubsystem.Get();
}

IOnlineSubsystem* FOnlineSubsystemModule::GetNativeSubsystem(bool bAutoLoad)
{
	if (!NativePlatformService.IsNone())
	{
		if (bAutoLoad || IOnlineSubsystem::IsLoaded(NativePlatformService))
		{
			return IOnlineSubsystem::Get(NativePlatformService);
		}
	}
	return nullptr;
}

IOnlineSubsystem* FOnlineSubsystemModule::GetSubsystemByConfig(const FString& ConfigString, bool bAutoLoad)
{
	const FName* const CachedConfig = ConfigDefinedSubsystems.Find(ConfigString);
	if (CachedConfig && !CachedConfig->IsNone())
	{
		if (bAutoLoad || IOnlineSubsystem::IsLoaded(*CachedConfig))
		{
			return IOnlineSubsystem::Get(*CachedConfig);
		}
	}

	return nullptr;
}


void FOnlineSubsystemModule::DestroyOnlineSubsystem(const FName InSubsystemName)
{
	FName SubsystemName, InstanceName;
	FName KeyName = ParseOnlineSubsystemName(InSubsystemName, SubsystemName, InstanceName);

	if (!SubsystemName.IsNone())
	{
		IOnlineSubsystemPtr OnlineSubsystem;
		{
			UE::TScopeLock Lock(OnlineSubsystemsLock);
			OnlineSubsystems.RemoveAndCopyValue(KeyName, OnlineSubsystem);
		}
		if (OnlineSubsystem.IsValid())
		{
			OnlineSubsystem->Shutdown();
			OnlineSubsystemFailureNotes.Remove(KeyName);
		}
		else
		{
			UE_LOG_ONLINE(Warning, TEXT("OnlineSubsystem instance %s not found, unable to destroy."), *KeyName.ToString());
		}
	}
}

bool FOnlineSubsystemModule::DoesInstanceExist(const FName InSubsystemName) const
{
	FName SubsystemName, InstanceName;
	const FName KeyName = ParseOnlineSubsystemName(InSubsystemName, SubsystemName, InstanceName);
	if (!SubsystemName.IsNone())
	{
		IOnlineSubsystemPtr OnlineSubsystem;
		{
			UE::TScopeLock Lock(OnlineSubsystemsLock);
			OnlineSubsystem = FindExistingSubsystem(OnlineSubsystems, KeyName);
		}
		return OnlineSubsystem.IsValid();
	}

	return false;
}

bool FOnlineSubsystemModule::IsOnlineSubsystemLoaded(const FName InSubsystemName) const
{
	bool bIsLoaded = false;

	FName SubsystemName, InstanceName;
	ParseOnlineSubsystemName(InSubsystemName, SubsystemName, InstanceName);

	if (!SubsystemName.IsNone())
	{
		if (FModuleManager::Get().IsModuleLoaded(GetOnlineModuleName(SubsystemName.ToString(), ModuleRedirects)))
		{
			bIsLoaded = true;
		}
	}
	return bIsLoaded;
}

