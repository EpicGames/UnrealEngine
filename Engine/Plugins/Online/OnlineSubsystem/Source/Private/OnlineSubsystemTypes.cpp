// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSubsystemTypes.h"

#include "Online/CoreOnline.h"

namespace EOnlineEnvironment
{
	const TCHAR* ToString(EOnlineEnvironment::Type EnvironmentType)
	{
		switch (EnvironmentType)
		{
		case Development: return TEXT("Development");
		case Certification: return TEXT("Certification");
		case Production: return TEXT("Production");
		case Unknown: default: return TEXT("Unknown");
		};
	}
} // namespace EOnlineEnvironment

namespace ELoginStatus
{
	const TCHAR* ToString(ELoginStatus::Type EnumVal)
	{
		switch (EnumVal)
		{
		case NotLoggedIn:
		{
			return TEXT("NotLoggedIn");
		}
		case UsingLocalProfile:
		{
			return TEXT("UsingLocalProfile");
		}
		case LoggedIn:
		{
			return TEXT("LoggedIn");
		}
		}
		return TEXT("");
	}
} // namespace ELoginStatus

namespace EOnlineServerConnectionStatus
{
	const TCHAR* ToString(EOnlineServerConnectionStatus::Type EnumVal)
	{
		switch (EnumVal)
		{
		case Normal:
		{
			return TEXT("Normal");
		}
		case NotConnected:
		{
			return TEXT("NotConnected");
		}
		case Connected:
		{
			return TEXT("Connected");
		}
		case ConnectionDropped:
		{
			return TEXT("ConnectionDropped");
		}
		case NoNetworkConnection:
		{
			return TEXT("NoNetworkConnection");
		}
		case ServiceUnavailable:
		{
			return TEXT("ServiceUnavailable");
		}
		case UpdateRequired:
		{
			return TEXT("UpdateRequired");
		}
		case ServersTooBusy:
		{
			return TEXT("ServersTooBusy");
		}
		case DuplicateLoginDetected:
		{
			return TEXT("DuplicateLoginDetected");
		}
		case InvalidUser:
		{
			return TEXT("InvalidUser");
		}
		case NotAuthorized:
		{
			return TEXT("NotAuthorized");
		}
		case InvalidSession:
		{
			return TEXT("InvalidSession");
		}
		default:
		{
			return TEXT("Unknown");
		}
		}
	}
} // namespace EOnlineServerConnectionStatus

namespace EFeaturePrivilegeLevel
{
	const TCHAR* ToString(EFeaturePrivilegeLevel::Type EnumVal)
	{
		switch (EnumVal)
		{
		case Undefined:
		{
			return TEXT("Undefined");
		}
		case Disabled:
		{
			return TEXT("Disabled");
		}
		case EnabledFriendsOnly:
		{
			return TEXT("EnabledFriendsOnly");
		}
		case Enabled:
		{
			return TEXT("Enabled");
		}
		}
		return TEXT("");
	}
} // namespace EFeaturePrivilegeLevel

namespace EOnlineAsyncTaskState
{
	const TCHAR* ToString(EOnlineAsyncTaskState::Type EnumVal)
	{
		switch (EnumVal)
		{
		case NotStarted:
		{
			return TEXT("NotStarted");
		}
		case InProgress:
		{
			return TEXT("InProgress");
		}
		case Done:
		{
			return TEXT("Done");
		}
		case Failed:
		{
			return TEXT("Failed");
		}
		}
		return TEXT("");
	}
} // namespace EOnlineAsyncTaskState

namespace EOnlineFriendState
{
	const TCHAR* ToString(EOnlineFriendState::Type EnumVal)
	{
		switch (EnumVal)
		{
		case Offline:
		{
			return TEXT("Offline");
		}
		case Online:
		{
			return TEXT("Online");
		}
		case Away:
		{
			return TEXT("Away");
		}
		case Busy:
		{
			return TEXT("Busy");
		}
		}
		return TEXT("");
	}
} // namespace EOnlineFriendState

namespace ELeaderboardSort
{
	const TCHAR* ToString(ELeaderboardSort::Type EnumVal)
	{
		switch (EnumVal)
		{
		case None:
		{
			return TEXT("None");
		}
		case Ascending:
		{
			return TEXT("Ascending");
		}
		case Descending:
		{
			return TEXT("Descending");
		}
		}
		return TEXT("");
	}
} // namespace ELeaderboardSort

namespace ELeaderboardFormat
{
	const TCHAR* ToString(ELeaderboardFormat::Type EnumVal)
	{
		switch (EnumVal)
		{
		case Number:
		{
			return TEXT("Number");
		}
		case Seconds:
		{
			return TEXT("Seconds");
		}
		case Milliseconds:
		{
			return TEXT("Milliseconds");
		}
		}
		return TEXT("");
	}
} // namespace ELeaderboardFormat

namespace ELeaderboardUpdateMethod
{
	const TCHAR* ToString(ELeaderboardUpdateMethod::Type EnumVal)
	{
		switch (EnumVal)
		{
		case KeepBest:
		{
			return TEXT("KeepBest");
		}
		case Force:
		{
			return TEXT("Force");
		}
		}
		return TEXT("");
	}
} // namespace ELeaderboardUpdateMethod

namespace EOnlineSessionState
{
	const TCHAR* ToString(EOnlineSessionState::Type EnumVal)
	{
		switch (EnumVal)
		{

		case NoSession:
		{
			return TEXT("NoSession");
		}
		case Creating:
		{
			return TEXT("Creating");
		}
		case Pending:
		{
			return TEXT("Pending");
		}
		case Starting:
		{
			return TEXT("Starting");
		}
		case InProgress:
		{
			return TEXT("InProgress");
		}
		case Ending:
		{
			return TEXT("Ending");
		}
		case Ended:
		{
			return TEXT("Ended");
		}
		case Destroying:
		{
			return TEXT("Destroying");
		}
		}
		return TEXT("");
	}
} // namespace EOnlineSessionState

namespace EOnlineDataAdvertisementType
{
	const TCHAR* ToString(EOnlineDataAdvertisementType::Type EnumVal)
	{
		switch (EnumVal)
		{
		case DontAdvertise:
		{
			return TEXT("DontAdvertise");
		}
		case ViaPingOnly:
		{
			return TEXT("ViaPingOnly");
		}
		case ViaOnlineService:
		{
			return TEXT("OnlineService");
		}
		case ViaOnlineServiceAndPing:
		{
			return TEXT("OnlineServiceAndPing");
		}
		}
		return TEXT("");
	}
} // namespace EOnlineDataAdvertisementType

namespace EOnlineComparisonOp
{
	const TCHAR* ToString(EOnlineComparisonOp::Type EnumVal)
	{
		switch (EnumVal)
		{
		case Equals:
		{
			return TEXT("Equals");
		}
		case NotEquals:
		{
			return TEXT("NotEquals");
		}
		case GreaterThan:
		{
			return TEXT("GreaterThan");
		}
		case GreaterThanEquals:
		{
			return TEXT("GreaterThanEquals");
		}
		case LessThan:
		{
			return TEXT("LessThan");
		}
		case LessThanEquals:
		{
			return TEXT("LessThanEquals");
		}
		case Near:
		{
			return TEXT("Near");
		}
		case In:
		{
			return TEXT("In");
		}
		case NotIn:
		{
			return TEXT("NotIn");
		}
		}
		return TEXT("");
	}
} // namespace EOnlineComparisonOp

namespace EOnlineCachedResult
{
	const TCHAR* ToString(EOnlineCachedResult::Type EnumVal)
	{
		switch (EnumVal)
		{
		case Success:
		{
			return TEXT("Success");
		}
		case NotFound:
		{
			return TEXT("NotFound");
		}
		}
		return TEXT("");
	}
} // namespace EOnlineCachedResult

const TCHAR* LexToString(EFriendInvitePolicy EnumVal)
{
	switch (EnumVal)
	{
	case EFriendInvitePolicy::Public: return TEXT("PUBLIC");
	case EFriendInvitePolicy::Friends_of_Friends: return TEXT("FRIENDS_OF_FRIENDS");
	case EFriendInvitePolicy::Private: return TEXT("PRIVATE");
	default: return TEXT("Invalid");
	}
}

void LexFromString(EFriendInvitePolicy& Value, const TCHAR* String)
{
	if (FCString::Stricmp(String, TEXT("PUBLIC")) == 0)
	{
		Value = EFriendInvitePolicy::Public;
	}
	else if (FCString::Stricmp(String, TEXT("FRIENDS_OF_FRIENDS")) == 0)
	{
		Value = EFriendInvitePolicy::Friends_of_Friends;
	}
	else if (FCString::Stricmp(String, TEXT("PRIVATE")) == 0)
	{
		Value = EFriendInvitePolicy::Private;
	}
	else
	{
		Value = EFriendInvitePolicy::InvalidOrMax;
	}
}

bool IOnlinePlatformData::Compare(const IOnlinePlatformData& Other) const
{
	return (GetSize() == Other.GetSize()) &&
		(FMemory::Memcmp(GetBytes(), Other.GetBytes(), GetSize()) == 0);
}

namespace EInviteStatus
{
	const TCHAR* ToString(EInviteStatus::Type EnumVal)
	{
		switch (EnumVal)
		{
		case Unknown:
		{
			return TEXT("Unknown");
		}
		case Accepted:
		{
			return TEXT("Accepted");
		}
		case PendingInbound:
		{
			return TEXT("PendingInbound");
		}
		case PendingOutbound:
		{
			return TEXT("PendingOutbound");
		}
		case Blocked:
		{
			return TEXT("Blocked");
		}
		case Suggested:
		{
			return TEXT("Suggested");
		}
		}
		return TEXT("");
	}
} // namespace EInviteStatus

const TCHAR* ToString(EOnlineSharingCategory CategoryType)
{
	switch (CategoryType)
	{
	case EOnlineSharingCategory::None:
	{
		return TEXT("Category undefined");
	}
	case EOnlineSharingCategory::ReadPosts:
	{
		return TEXT("ReadPosts");
	}
	case EOnlineSharingCategory::Friends:
	{
		return TEXT("Friends");
	}
	case EOnlineSharingCategory::Mailbox:
	{
		return TEXT("Mailbox");
	}
	case EOnlineSharingCategory::OnlineStatus:
	{
		return TEXT("Online Status");
	}
	case EOnlineSharingCategory::ProfileInfo:
	{
		return TEXT("Profile Information");
	}
	case EOnlineSharingCategory::LocationInfo:
	{
		return TEXT("Location Information");
	}
	case EOnlineSharingCategory::SubmitPosts:
	{
		return TEXT("SubmitPosts");
	}
	case EOnlineSharingCategory::ManageFriends:
	{
		return TEXT("ManageFriends");
	}
	case EOnlineSharingCategory::AccountAdmin:
	{
		return TEXT("Account Admin");
	}
	case EOnlineSharingCategory::Events:
	{
		return TEXT("Events");
	}
	}
	return TEXT("");
}

const TCHAR* ToString(EOnlineStatusUpdatePrivacy PrivacyType)
{
	switch (PrivacyType)
	{
	case EOnlineStatusUpdatePrivacy::OnlyMe:
		return TEXT("Only Me");
	case EOnlineStatusUpdatePrivacy::OnlyFriends:
		return TEXT("Only Friends");
	case EOnlineStatusUpdatePrivacy::Everyone:
		return TEXT("Everyone");
	}
	return TEXT("");
}

uint32 GetTypeHash(const FOnlinePartyId& Value)
{
	return CityHash32(reinterpret_cast<const char*>(Value.GetBytes()), Value.GetSize());
}

void ParseOnlineSubsystemConfigPairs(TArrayView<const FString> InEntries, TArray<TPair<FString, FString>>& OutPairs)
{
	OutPairs.Reserve(InEntries.Num());
	// Takes on the pattern "(Key=Value)"
	for (const FString& Entry : InEntries)
	{
		FString TrimmedConfigEntry = Entry.TrimStartAndEnd();
		FString KeyString;
		FString ValueString;

		if (TrimmedConfigEntry.Left(1) == TEXT("("))
		{
			TrimmedConfigEntry.RightChopInline(1, EAllowShrinking::No);
		}
		if (TrimmedConfigEntry.Right(1) == TEXT(")"))
		{
			TrimmedConfigEntry.LeftChopInline(1, EAllowShrinking::No);
		}
		if (TrimmedConfigEntry.Split(TEXT("="), &KeyString, &ValueString, ESearchCase::CaseSensitive))
		{
			KeyString.TrimStartAndEndInline();
			ValueString.TrimStartAndEndInline();
		}
		OutPairs.Emplace(MoveTemp(KeyString), MoveTemp(ValueString));
	}
}

FName NAME_Unset = TEXT("UNSET");
