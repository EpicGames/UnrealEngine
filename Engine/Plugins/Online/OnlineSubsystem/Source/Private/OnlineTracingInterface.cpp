// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/OnlineTracingInterface.h"
#include "Features/IModularFeatures.h"

namespace
{
	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName(TEXT("OnlineTracing"));
		return FeatureName;
	}
}

/**
 * Checks that the OnlineTracing feature is registered with IModularFeature
 *
 * @return FOnlineTracing singleton if feature registered, otherwise nullptr
 */
IOnlineTracing* IOnlineTracing::GetTracingHelper()
{
	if (IModularFeatures::Get().IsModularFeatureAvailable(GetModularFeatureName()))
	{
		return &IModularFeatures::Get().GetModularFeature<IOnlineTracing>(GetModularFeatureName());
	}
	return nullptr;
}

void IOnlineTracing::StartContext(FName ContextName)
{
	if (IOnlineTracing* OnlineTracingSingleton = GetTracingHelper())
	{
		OnlineTracingSingleton->StartContextImpl(ContextName);
	}
}

void IOnlineTracing::EndContext(FName ContextName)
{
	if (IOnlineTracing* OnlineTracingSingleton = GetTracingHelper())
	{
		OnlineTracingSingleton->EndContextImpl(ContextName);
	}
}
