// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/OnlineUserCloudInterface.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS
IOnlineUserCloud::IOnlineUserCloud() = default;
IOnlineUserCloud::~IOnlineUserCloud() = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
