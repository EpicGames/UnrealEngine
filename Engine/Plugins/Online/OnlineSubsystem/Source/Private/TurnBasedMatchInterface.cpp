// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/TurnBasedMatchInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TurnBasedMatchInterface)

UTurnBasedMatchInterface::UTurnBasedMatchInterface(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

