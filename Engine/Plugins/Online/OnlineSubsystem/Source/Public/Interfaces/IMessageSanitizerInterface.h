// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

struct FBlockedQueryResult
{
	FBlockedQueryResult(bool InIsBlocked=false, bool InIsBlockedNonFriends=false, const FString& InUserId = FString())
		: bIsBlocked(InIsBlocked)
		, bIsBlockedNonFriends(InIsBlockedNonFriends)
		, UserId(InUserId)
	{}

	/** Is this user blocked */
	bool bIsBlocked;
	/** Is this user blocked for non-friends */
	bool bIsBlockedNonFriends;
	/** Platform specific unique id */
	FString UserId;
};

DECLARE_DELEGATE_TwoParams(FOnMessageProcessed, bool /*bSuccess*/, const FString& /*SanitizedMessage*/);
DECLARE_DELEGATE_TwoParams(FOnMessageArrayProcessed, bool /*bSuccess*/, const TArray<FString>& /*SanitizedMessages*/);
DECLARE_DELEGATE_OneParam(FOnQueryUserBlockedResponse, const FBlockedQueryResult& /** QueryResult */);

class IMessageSanitizer : public TSharedFromThis<IMessageSanitizer, ESPMode::ThreadSafe>
{
protected:
	IMessageSanitizer() {};

public:
	virtual ~IMessageSanitizer() = default;
	virtual void SanitizeDisplayName(const FString& DisplayName, const FOnMessageProcessed& CompletionDelegate) = 0;
	virtual void SanitizeDisplayNames(const TArray<FString>& DisplayNames, const FOnMessageArrayProcessed& CompletionDelegate) = 0;

	/**
	 * Query for a blocked user status between a local and remote user
	 *
	 * @param LocalUserNum local user making the query
	 * @param FromUserId platform specific user id of the remote user
	 * @param FromPlatform platform for remote user
	 * @param CompletionDelegate delegate to fire on completion
	 */
	virtual void QueryBlockedUser(int32 LocalUserNum, const FString& FromUserId, const FString& FromPlatform, const FOnQueryUserBlockedResponse& CompletionDelegate) = 0;

	/** Invalidate all previously queried blocked users state */
	virtual void ResetBlockedUserCache() = 0;

	const FString MessageArraySeparator = TEXT("\u241e");
};

typedef TSharedPtr<IMessageSanitizer, ESPMode::ThreadSafe> IMessageSanitizerPtr;
