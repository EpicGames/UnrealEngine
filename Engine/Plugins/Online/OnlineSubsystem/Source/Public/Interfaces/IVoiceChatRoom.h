// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Features/IModularFeature.h"

struct FOnlineError;

typedef uint64 VoiceChatRoomId;

#define INVALID_VOICE_CHATROOM -1

/**
 * Payload for a room entered callback
 */
struct FVoiceChatRoomEntered
{
public:

	FVoiceChatRoomEntered(VoiceChatRoomId InRoomId)
		: RoomId(InRoomId)
	{}
	FVoiceChatRoomEntered() = delete;
	~FVoiceChatRoomEntered() = default;

	/** Room entered */
	VoiceChatRoomId RoomId;
};

/**
 * Payload for a room exited callback
 */
struct FVoiceChatRoomExited
{
public:

	FVoiceChatRoomExited(VoiceChatRoomId InRoomId)
		: RoomId(InRoomId)
	{}
	FVoiceChatRoomExited() = delete;
	~FVoiceChatRoomExited() = default;

	/** Room exited */
	VoiceChatRoomId RoomId;
};

DECLARE_DELEGATE_TwoParams(FOnVoiceChatRoomEntered, const FVoiceChatRoomEntered& /*RoomData*/, const FOnlineError& /*Error*/);
DECLARE_DELEGATE_TwoParams(FOnVoiceChatRoomExited, const FVoiceChatRoomExited& /*RoomData*/, const FOnlineError& /*Error*/);
DECLARE_DELEGATE_OneParam(FOnShowVoiceChatUI, const FOnlineError& /*Error*/);

enum class EVoiceChatRoomState : uint8
{
	/** Default, invalid room state */
	Invalid,
	/** Entering chat room */
	Entering,
	/** Room successfully entered */
	Entered,
	/** Exiting chat room, will be removed shortly */
	Leaving
};

inline const TCHAR* LexToString(EVoiceChatRoomState InState)
{
	switch (InState)
	{
		case EVoiceChatRoomState::Invalid:
			return TEXT("Invalid");
			break;
		case EVoiceChatRoomState::Entering:
			return TEXT("Entering");
			break;
		case EVoiceChatRoomState::Entered:
			return TEXT("Entered");
			break;
		case EVoiceChatRoomState::Leaving:
			return TEXT("Leaving");
			break;
	}

	return TEXT("");
}

/**
 * Representation of a user inside of a voice chat room
 */
class FVoiceChatRoomMember
{

public:

	FVoiceChatRoomMember() = default;
	virtual ~FVoiceChatRoomMember() = default;

	/** @return debug information about this chat room suitable for output */
	virtual FString ToDebugString() const = 0;
};

/**
 * Basic information about a voice chat room in various possible states
 */
class FVoiceChatRoomInfo
{
public:

	FVoiceChatRoomInfo() = default;
	virtual ~FVoiceChatRoomInfo() = default;

	/** @return the room id for this chat room */
	virtual VoiceChatRoomId GetRoomId() const = 0;
	/** @return the state this chat room is in */
	virtual EVoiceChatRoomState GetState() const = 0;
	/** @return all currently known members of this chat room */
	virtual void GetMembers(TArray<TSharedRef<FVoiceChatRoomMember>>& OutMembers) const = 0;
	/** @return debug information about this chat room suitable for output */
	virtual FString ToDebugString() const = 0;
};

class IVoiceChatRoom : public IModularFeature
{
public:

	IVoiceChatRoom() = default;
	virtual ~IVoiceChatRoom() = default;

	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName(TEXT("VoiceChatRoom"));
		return FeatureName;
	}

	/**
	 * Enter a voice chat room
	 *
	 * @param InRoomId 64 bit room id to enter
	 * @param OnChatRoomEntered delegate to fire when request is complete
	 */
	virtual void EnterRoom(VoiceChatRoomId InRoomId, const FOnVoiceChatRoomEntered& OnChatRoomEntered) = 0;
	/**
	 * Switch between voice chat rooms
	 *
	 * @param InRoomId 64 bit room id to switch to, leaving previous room
	 * @param OnChatRoomEntered delegate to fire when request is complete
	 */
	virtual void SwitchRoom(VoiceChatRoomId InRoomId, const FOnVoiceChatRoomEntered& OnChatRoomEntered) = 0;
	/**
	 * Exit a voice chat room
	 *
	 * @param InRoomId 64 bit room id to exit
	 * @param OnChatRoomExited delegate to fire when request is complete
	 */
	virtual void ExitRoom(VoiceChatRoomId InRoomId, const FOnVoiceChatRoomExited& OnChatRoomExited) = 0;
	/**
	 * Show or hide the UI related to the voice chat system
	 *
	 * @param bShow true to show, false to hide the UI
	 * @param OnShowChatUI delegate to fire when request is complete
	 */
	virtual void ShowUI(bool bShow, const FOnShowVoiceChatUI& OnShowChatUI) = 0;
	/**
	 * Get all known voice chat rooms
	 *
	 * @param OutRooms array to fill with known chat rooms
	 */
	virtual void GetRooms(TArray<TSharedRef<FVoiceChatRoomInfo>>& OutRooms) const = 0;
	/**
	 * Set a display name for the chat room, if applicable
	 *
	 * @param InDisplayName desired display name for chat room UI
	 */
	virtual void SetDisplayName(const FString& InDisplayName) = 0;
};
