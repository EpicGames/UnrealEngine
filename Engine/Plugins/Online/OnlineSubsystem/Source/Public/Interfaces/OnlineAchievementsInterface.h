// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/DateTime.h"
#include "OnlineDelegateMacros.h"
#include "OnlineStats.h"

ONLINESUBSYSTEM_API DECLARE_LOG_CATEGORY_EXTERN(LogOnlineAchievements, Log, All);

#define UE_LOG_ONLINE_ACHIEVEMENTS(Verbosity, Format, ...) \
{ \
	UE_LOGF(LogOnlineAchievements, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

#define UE_CLOG_ONLINE_ACHIEVEMENTS(Conditional, Verbosity, Format, ...) \
{ \
	UE_CLOGF(Conditional, LogOnlineAchievements, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

/**
 * Delegate fired when achievements have been written to the server
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnAchievementsWritten, const FUniqueNetId&, bool );
typedef FOnAchievementsWritten::FDelegate FOnAchievementsWrittenDelegate;


/**
 * Delegate fired when an achievement has been unlocked
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnAchievementUnlocked, const FUniqueNetId&, const FString& );
typedef FOnAchievementUnlocked::FDelegate FOnAchievementUnlockedDelegate;


/**
 * Delegate fired when an achievement has been queried
 */
DECLARE_DELEGATE_TwoParams(FOnQueryAchievementsCompleteDelegate, const FUniqueNetId&, const bool);	

/**
*	FOnlineAchievement - Interface class for accessing the common achievement information
 */
struct FOnlineAchievement
{
	/** The id of the achievement */
	FString Id;

	/** The progress towards completing this achievement: 0.0-100.0 */
	double Progress;

	/** Returns debugging string to print out achievement info */
	FString ToDebugString() const
	{
		return FString::Printf( TEXT("Id='%s', Progress=%f"),
			*Id,
			Progress
			);
	}
};


/**
*	FOnlineAchievementDesc - Interface class for accessing the common achievement description information
 */
struct FOnlineAchievementDesc
{
	/** The localized title of the achievement */
	FText Title;

	/** The localized locked description of the achievement */
	FText LockedDesc;

	/** The localized unlocked description of the achievement */
	FText UnlockedDesc;

	/** Flag for whether the achievement is hidden */
	bool bIsHidden;

	/** The date/time the achievement was unlocked */
	FDateTime UnlockTime;

	/** Returns debugging string to print out achievement info */
	FString ToDebugString() const
	{
		return FString::Printf( TEXT("Title='%s', LockedDesc='%s', UnlockedDesc='%s', bIsHidden=%s, UnlockTime=%s"),
			*Title.ToString(),
			*LockedDesc.ToString(),
			*UnlockedDesc.ToString(),
			bIsHidden ? TEXT("true") : TEXT("false"),
			*UnlockTime.ToString()
			);
	}
};


/**
 *	IOnlineAchievements - Interface class for achievements
 */
class IOnlineAchievements
{
public:
	virtual ~IOnlineAchievements() = default;

	/**
	 * Write the achievements provided to the server
	 *
	 * @param PlayerId - The uid of the player we are writing achievements for
	 * @param WriteObject - The stats holder containing the achievements we are writing.
	 * @param Delegate - The delegate to call when the write has completed or failed.
	 */
	virtual void WriteAchievements(const FUniqueNetId& PlayerId, FOnlineAchievementsWriteRef& WriteObject, const FOnAchievementsWrittenDelegate& Delegate = FOnAchievementsWrittenDelegate()) = 0;

	/**
	 * Read achievement ids and progress from the server
	 * You must use QueryAchievementDescriptions to get more information about the achievements
	 * @see FOnlineAchievement
	 * @see QueryAchievementDescriptions
	 *
	 * @param PlayerId the id of the player we are reading achievements for
	 * @param Delegate the delegate to trigger when the query is complete
	 *
	 * @return Whether we have kicked off a read attempt
	 */
	virtual void QueryAchievements(const FUniqueNetId& PlayerId, const FOnQueryAchievementsCompleteDelegate& Delegate = FOnQueryAchievementsCompleteDelegate()) = 0;
	
	/**
	 * Read achievement descriptions from the server for displaying achievements in game
	 * @see FOnlineAchievementDesc
	 *
	 * @param PlayerId the id of the player we are reading achievements for
	 * @param Delegate the delegate to trigger when the query is complete
	 *
	 * @return Whether we have kicked off a read attempt
	 */
	virtual void QueryAchievementDescriptions( const FUniqueNetId& PlayerId, const FOnQueryAchievementsCompleteDelegate& Delegate = FOnQueryAchievementsCompleteDelegate() ) = 0;

	/**
	 * Get an achievement object which was previously synced from the server
	 *
	 * @param PlayerId - The uid of the player we are reading achievements for
	 * @param AchievementId - The id of the achievement we are looking up
	 * @param OutAchievement - The achievement object we are searching for
	 *
	 * @return Whether achievements were obtained
	 */
	virtual EOnlineCachedResult::Type GetCachedAchievement( const FUniqueNetId& PlayerId, const FString& AchievementId, FOnlineAchievement& OutAchievement) = 0;
	
	/**
	 * Get all the achievement objects for the specified player
	 *
	 * @param PlayerId - The uid of the player we are reading achievements for
	 * @param OutAchievements - The collection of achievements obtained from the server for the given player
	 *
	 * @return Whether achievements were obtained
	 */
	virtual EOnlineCachedResult::Type GetCachedAchievements(const FUniqueNetId& PlayerId, TArray<FOnlineAchievement>& OutAchievements) = 0;
	
	/**
	 * Get all the achievement description object for the specified achievement id
	 *
	 * @param AchievementId - The id of the achievement we are searching for data of
	 * @param OutAchievementDesc - The description object for the achievement id we seek
	 *
	 * @return Whether achievements were obtained
	 */
	virtual EOnlineCachedResult::Type GetCachedAchievementDescription(const FString& AchievementId, FOnlineAchievementDesc& OutAchievementDesc) = 0;

#if !UE_BUILD_SHIPPING
	/**
	 * Resets achievements for a given player
	 *
	 * @param PlayerId - The uid of the player
	 *
	 * @return Whether we kicked off the clear request
	 */
	virtual bool ResetAchievements( const FUniqueNetId& PlayerId ) = 0;
#endif // !UE_BUILD_SHIPPING

	/**
	 * Delegate fired when an achievement on the server was unlocked
	 */
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnAchievementUnlocked, const FUniqueNetId&, const FString&);
};
