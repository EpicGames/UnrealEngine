// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "OnlineDelegateMacros.h"
#include "Online/CoreOnlineFwd.h"

struct FDateTime;

ONLINESUBSYSTEM_API DECLARE_LOG_CATEGORY_EXTERN(LogOnlineChat, Log, All);

#define UE_LOG_ONLINE_CHAT(Verbosity, Format, ...) \
{ \
	UE_LOGF(LogOnlineChat, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

#define UE_CLOG_ONLINE_CHAT(Conditional, Verbosity, Format, ...) \
{ \
	UE_CLOGF(Conditional, LogOnlineChat, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

/**
 * Id of a chat room
 */
typedef FString FChatRoomId;

/**
 * Info for a joined/created chat room
 */
class FChatRoomInfo
{
public:
	virtual ~FChatRoomInfo() = default;

	virtual const FChatRoomId& GetRoomId() const = 0;
	virtual const FUniqueNetIdRef& GetOwnerId() const = 0;
	virtual const FString& GetSubject() const = 0;
	virtual bool IsPrivate() const = 0;
	virtual bool IsJoined() const = 0;
	virtual const class FChatRoomConfig& GetRoomConfig() const = 0;
	virtual FString ToDebugString() const = 0;
	virtual void SetChatInfo(const TSharedRef<class FJsonObject>& JsonInfo) = 0;
};

/**
 * Configuration for creating/joining a chat room
 */
class FChatRoomConfig
{
public:
	FChatRoomConfig()
		: bRejoinOnDisconnect(false)
		, bPasswordRequired(false)
		, bAnnounceMembers(true)
	{}

	/** Should this room be rejoined on disconnection */
	bool bRejoinOnDisconnect;
	/** Is there a password required to join the room (owner only) */
	bool bPasswordRequired;
	/** Password to join the room (owner only) */
	FString Password;
	/**
	 * Should we announce members joining and leaving?
	 * This may incur a performance cost if members need to be validated.
	 */
	bool bAnnounceMembers;

	FString ToDebugString() const
	{
		return FString::Printf(TEXT("bRejoin: %d bPassReqd: %d Pass: %s bAnnounceMembers: %d"), bRejoinOnDisconnect, bPasswordRequired, *Password, bAnnounceMembers);
	}

private:
	// Below are unused, move to public when hooking up to functionality
	bool bMembersOnly;
	bool bHidden;
	bool bPersistent;
	bool bAllowMemberInvites;
	bool bLoggingEnabled;
	int32 MessageHistory;
	int32 MaxMembers;
	FString PubSubNode;
};

/**
 * Member of a chat room
 */
class FChatRoomMember
{
public:
	virtual ~FChatRoomMember() = default;

	virtual const FUniqueNetIdRef& GetUserId() const = 0;
	virtual const FString& GetNickname() const = 0;
};

/**
* Chat message received from user/room
*/
class FChatMessage
{
public:
	virtual ~FChatMessage() = default;

	/** @return the user id the message is from */
	virtual const FUniqueNetIdRef& GetUserId() const = 0;
	/** @return the nickname of the user the message is from */
	virtual const FString& GetNickname() const = 0;
	/** @return the body of the message */
	virtual const FString& GetBody() const = 0;
	/** @return the timestamp for the message */
	virtual const FDateTime& GetTimestamp() const = 0;
};

/**
* Delegate used when creating a new chat room
*
* @param UserId the user that made the request
* @param RoomId room that was requested
* @param bWasSuccessful true if the async action completed without error, false if there was an error
* @param Error string representing the error condition
*/
DECLARE_MULTICAST_DELEGATE_FourParams(FOnChatRoomCreated, const FUniqueNetId& /*UserId*/, const FChatRoomId& /*RoomId*/, bool /*bWasSuccessful*/, const FString& /*Error*/);
typedef FOnChatRoomCreated::FDelegate FOnChatRoomCreatedDelegate;

/**
* Delegate used when configuring a chat room
*
* @param UserId the user that made the request
* @param RoomId room that was requested
* @param bWasSuccessful true if the async action completed without error, false if there was an error
* @param Error string representing the error condition
*/
DECLARE_MULTICAST_DELEGATE_FourParams(FOnChatRoomConfigured, const FUniqueNetId& /*UserId*/, const FChatRoomId& /*RoomId*/, bool /*bWasSuccessful*/, const FString& /*Error*/);
typedef FOnChatRoomConfigured::FDelegate FOnChatRoomConfiguredDelegate;

/**
* Delegate used when joining a public chat room
*
* @param UserId the user that made the request
* @param RoomId room that was requested
* @param bWasSuccessful true if the async action completed without error, false if there was an error
* @param Error string representing the error condition
*/
DECLARE_MULTICAST_DELEGATE_FourParams(FOnChatRoomJoinPublic, const FUniqueNetId& /*UserId*/, const FChatRoomId& /*RoomId*/, bool /*bWasSuccessful*/, const FString& /*Error*/);
typedef FOnChatRoomJoinPublic::FDelegate FOnChatRoomJoinPublicDelegate;

/**
 * Delegate used when joining a private chat room
 *
 * @param UserId the user that made the request
 * @param RoomId room that was requested
 * @param bWasSuccessful true if the async action completed without error, false if there was an error
 * @param Error string representing the error condition
 */
DECLARE_MULTICAST_DELEGATE_FourParams(FOnChatRoomJoinPrivate, const FUniqueNetId& /*UserId*/, const FChatRoomId& /*RoomId*/, bool /*bWasSuccessful*/, const FString& /*Error*/);
typedef FOnChatRoomJoinPrivate::FDelegate FOnChatRoomJoinPrivateDelegate;

/**
 * Delegate used when exiting a chat room
 *
 * @param UserId the user that made the request
 * @param RoomId room that was requested
 * @param bWasSuccessful true if the async action completed without error, false if there was an error
 * @param Error string representing the error condition
 */
DECLARE_MULTICAST_DELEGATE_FourParams(FOnChatRoomExit, const FUniqueNetId& /*UserId*/, const FChatRoomId& /*RoomId*/, bool /*bWasSuccessful*/, const FString& /*Error*/);
typedef FOnChatRoomExit::FDelegate FOnChatRoomExitDelegate;

/**
 * Delegate used when another chat room member enters/joins
 *
 * @param UserId user currently in the room
 * @param RoomId room that member is in
 * @param MemberId the user that entered
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnChatRoomMemberJoin, const FUniqueNetId& /*UserId*/, const FChatRoomId& /*RoomId*/, const FUniqueNetId& /*MemberId*/);
typedef FOnChatRoomMemberJoin::FDelegate FOnChatRoomMemberJoinDelegate;

/**
 * Delegate used when another chat room member exits
 *
 * @param UserId user currently in the room
 * @param RoomId room that member is in
 * @param MemberId the user that exited
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnChatRoomMemberExit, const FUniqueNetId& /*UserId*/, const FChatRoomId& /*RoomId*/, const FUniqueNetId& /*MemberId*/);
typedef FOnChatRoomMemberExit::FDelegate FOnChatRoomMemberExitDelegate;

/**
 * Delegate used when another chat room member is updated
 *
 * @param UserId user currently in the room
 * @param RoomId room that member is in
 * @param MemberId the user that updated
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnChatRoomMemberUpdate, const FUniqueNetId& /*UserId*/, const FChatRoomId& /*RoomId*/, const FUniqueNetId& /*MemberId*/);
typedef FOnChatRoomMemberUpdate::FDelegate FOnChatRoomMemberUpdateDelegate;

/**
* Delegate used when a chat message is sent to a chat room
*
* @param UserId the user that made the request
* @param RoomId room that was requested
* @param bWasSuccessful true if the async action completed without error, false if there was an error
* @param Error string representing the error condition
*/
DECLARE_MULTICAST_DELEGATE_FourParams(FOnChatRoomMessageSent, const FUniqueNetId& /*UserId*/, const FChatRoomId& /*RoomId*/, bool /*bWasSuccessful*/, const FString& /*Error*/);
typedef FOnChatRoomMessageSent::FDelegate FOnChatRoomMessageSentDelegate;

/**
 * Delegate used when a chat message is received from a chat room
 *
 * @param UserId user currently in the room
 * @param RoomId room that member is in
 * @param ChatMessage the message that was received
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnChatRoomMessageReceived, const FUniqueNetId& /*UserId*/, const FChatRoomId& /*RoomId*/, const TSharedRef<FChatMessage>& /*ChatMessage*/);
typedef FOnChatRoomMessageReceived::FDelegate FOnChatRoomMessageReceivedDelegate;

/**
* Delegate used when a private chat message is sent to another user
*
* @param UserId the user that made the request
* @param RecipientId the user the message was sent to
* @param bWasSuccessful true if the async action completed without error, false if there was an error
* @param Error string representing the error condition
*/
DECLARE_MULTICAST_DELEGATE_FourParams(FOnChatPrivateMessageSent, const FUniqueNetId& /*UserId*/, const FUniqueNetId& /*RecipientId*/, bool /*bWasSuccessful*/, const FString& /*Error*/);
typedef FOnChatPrivateMessageSent::FDelegate FOnChatPrivateMessageSentDelegate;

/**
 * Delegate used when a private chat message is received from another user
 *
 * @param UserId user that received the message
 * @param ChatMessage the message that was received
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnChatPrivateMessageReceived, const FUniqueNetId& /*UserId*/, const TSharedRef<FChatMessage>& /*ChatMessage*/);
typedef FOnChatPrivateMessageReceived::FDelegate FOnChatPrivateMessageReceivedDelegate;

/**
 * Interface class for user-user and user-room chat
 */
class IOnlineChat
{
public:
	virtual ~IOnlineChat() = default;

	/**
	 * Kick off request for creating a chat room with a provided configuration
	 * 
	 * @param UserId id of user that is creating the room
	 * @param RoomId name of room to create
	 * @param Nickname display name for the chat room. Name must be unique and is reserved for duration of join
	 * @param RoomConfig configuration for the room
	 *
	 * @return if successfully started the async operation
	 */
	virtual bool CreateRoom(const FUniqueNetId& UserId, const FChatRoomId& RoomId, const FString& Nickname, const FChatRoomConfig& ChatRoomConfig) = 0;

	/**
	 * Kick off request for configuring a chat room with a provided configuration
	 *
	 * @param UserId id of user that is creating the room
	 * @param RoomId name of room to create
	 * @param RoomConfig configuration for the room
	 *
	 * @return if successfully started the async operation
	 */
	virtual bool ConfigureRoom(const FUniqueNetId& UserId, const FChatRoomId& RoomId, const FChatRoomConfig& ChatRoomConfig) = 0;
	
	/**
	 * Kick off request for joining a public chat room
	 * 
	 * @param UserId id of user that is joining
	 * @param RoomId name of room to join
	 * @param Nickname display name for the chat room. Name must be unique and is reserved for duration of join
	 * @param ChatRoomConfig configuration parameters needed to join
	 *
	 * @return if successfully started the async operation
	 */
	virtual bool JoinPublicRoom(const FUniqueNetId& UserId, const FChatRoomId& RoomId, const FString& Nickname, const FChatRoomConfig& ChatRoomConfig) = 0;

	/**
	 * Kick off request for joining a private chat room
	 *
	 * @param UserId id of user that is joining
	 * @param RoomId name of room to join
	 * @param Nickname display name for the chat room. Name must be unique and is reserved for duration of join
	 * @param ChatRoomConfig configuration parameters needed to join
	 *
	 * @return if successfully started the async operation
	 */
	virtual bool JoinPrivateRoom(const FUniqueNetId& UserId, const FChatRoomId& RoomId, const FString& Nickname, const FChatRoomConfig& ChatRoomConfig) = 0;

	/**
	 * Kick off request for exiting a previously joined chat room
	 * 
	 * @param UserId id of user that is exiting
	 * @param RoomId name of room to exit
	 *
	 * @return if successfully started the async operation
	 */
	virtual bool ExitRoom(const FUniqueNetId& UserId, const FChatRoomId& RoomId) = 0;

	/**
	 * Kick off request for sending a chat message to a joined chat room
	 * 
	 * @param UserId id of user that is sending the message
	 * @param RoomId name of room to send message to
	 * @param MsgBody plain text of message body
	 *
	 * @return if successfully started the async operation
	 */
	virtual bool SendRoomChat(const FUniqueNetId& UserId, const FChatRoomId& RoomId, const FString& MsgBody) = 0;

	/**
	 * Kick off request for sending a chat message privately between users
	 * 
	 * @param UserId id of user that is sending the message
	 * @param RecipientId id of user to send the chat to
	 * @param MsgBody plain text of message body
	 *
	 * @return if successfully started the async operation
	 */
	virtual bool SendPrivateChat(const FUniqueNetId& UserId, const FUniqueNetId& RecipientId, const FString& MsgBody) = 0;

	/**
	 * Determine if chat is allowed for a given user
	 * 
	 * @param UserId id of user that is sending the message
	 * @param RecipientId id of user to send the chat to
	 *
	 * @return if chat is allowed
	 */
	virtual bool IsChatAllowed(const FUniqueNetId& UserId, const FUniqueNetId& RecipientId) const = 0;


	/**
	 * Get cached list of rooms that have been joined
	 * 
	 * @param UserId id of user to find
	 * @param OutRooms [out] list of room ids or empty if not found
	 */
	virtual void GetJoinedRooms(const FUniqueNetId& UserId, TArray<FChatRoomId>& OutRooms) = 0;

	/**
	 * Get cached room info for a room 
	 * 
	 * @param UserId id of user to find
	 * @param RoomId id of room to find
	 *
	 * @return RoomInfo information about a chat room or NULL if not found
	 */
	virtual TSharedPtr<FChatRoomInfo> GetRoomInfo(const FUniqueNetId& UserId, const FChatRoomId& RoomId) = 0;

	/**
	 * Get cached list of members currently joined in a chat room 
	 * 
	 * @param UserId id of user to find
	 * @param RoomId id of room to find
	 * @param OutRoomMembers [out] list of current members in the chat room
	 *
	 * @return true if found
	 */
	virtual bool GetMembers(const FUniqueNetId& UserId, const FChatRoomId& RoomId, TArray< TSharedRef<FChatRoomMember> >& OutMembers) = 0;

	/**
	 * Get cached member currently joined in a chat room 
	 * 
	 * @param UserId id of user to find
	 * @param RoomId id of room to find
	 * @param MemberId id of member to find
	 *
	 * @return member in room or NULL if not found
	 */
	virtual TSharedPtr<FChatRoomMember> GetMember(const FUniqueNetId& UserId, const FChatRoomId& RoomId, const FUniqueNetId& MemberId) = 0;

	/**
	 * Get cached list of chat messages for a currently joined chat room 
	 * 
	 * @param UserId id of user to find
	 * @param RoomId id of room to find
	 * @param NumMessages max number of messages to fetch from history (-1 for all cached messages)
	 * @param OutMessages [out] list of messages from the chat room history
	 *
	 * @return true if found
	 */
	virtual bool GetLastMessages(const FUniqueNetId& UserId, const FChatRoomId& RoomId, int32 NumMessages, TArray< TSharedRef<FChatMessage> >& OutMessages) = 0;

	/** 
	 * Check if a message is from the local user
	 * 
	 * @param UserId local user id
	 * @param Message message to check
	 * @param bIncludeExternalInstances relevant when multiple logins on the same account name are possible.  Use false if we want to restrict to the current (this) instance of the game, true if we want to include instances running in other processes
	 * 
	 * @return true if the message is from the local user, false if not
	 */
	virtual bool IsMessageFromLocalUser(const FUniqueNetId& UserId, const FChatMessage& Message, const bool bIncludeExternalInstances) = 0;

	/**
	 * Dump state information about chat rooms
	 */
	virtual void DumpChatState() const = 0;

	// delegate callbacks (see declarations above)
	DEFINE_ONLINE_DELEGATE_FOUR_PARAM(OnChatRoomCreated, const FUniqueNetId&, const FChatRoomId&, bool, const FString&);
	DEFINE_ONLINE_DELEGATE_FOUR_PARAM(OnChatRoomConfigured, const FUniqueNetId&, const FChatRoomId&, bool, const FString&);
	DEFINE_ONLINE_DELEGATE_FOUR_PARAM(OnChatRoomJoinPublic, const FUniqueNetId&, const FChatRoomId&, bool, const FString&);
	DEFINE_ONLINE_DELEGATE_FOUR_PARAM(OnChatRoomJoinPrivate, const FUniqueNetId&, const FChatRoomId&, bool, const FString&);
	DEFINE_ONLINE_DELEGATE_FOUR_PARAM(OnChatRoomExit, const FUniqueNetId&, const FChatRoomId&, bool, const FString&);
	DEFINE_ONLINE_DELEGATE_THREE_PARAM(OnChatRoomMemberJoin, const FUniqueNetId&, const FChatRoomId&, const FUniqueNetId&);
	DEFINE_ONLINE_DELEGATE_THREE_PARAM(OnChatRoomMemberExit, const FUniqueNetId&, const FChatRoomId&, const FUniqueNetId&);
	DEFINE_ONLINE_DELEGATE_THREE_PARAM(OnChatRoomMemberUpdate, const FUniqueNetId&, const FChatRoomId&, const FUniqueNetId&);
	DEFINE_ONLINE_DELEGATE_FOUR_PARAM(OnChatRoomMessageSent, const FUniqueNetId&, const FChatRoomId&, bool, const FString&);
	DEFINE_ONLINE_DELEGATE_THREE_PARAM(OnChatRoomMessageReceived, const FUniqueNetId&, const FChatRoomId&, const TSharedRef<FChatMessage>&);
	DEFINE_ONLINE_DELEGATE_FOUR_PARAM(OnChatPrivateMessageSent, const FUniqueNetId&, const FUniqueNetId&, bool, const FString&);
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnChatPrivateMessageReceived, const FUniqueNetId&, const TSharedRef<FChatMessage>&);
};
