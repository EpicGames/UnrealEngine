// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

class FUniqueNetId;
struct FOnlineError;

class FCodeLockInfo
{
public:
	FString CodeType;
	FString CodeId;
	FString CodeUseId;
	FString OfferId;
};

class FCodeEvaluationInfo
{
public:
	FString CodeType;
	FString CodeId;
	FString CodeStatus;
	FString OfferId;
};

/**
 * Delegate used when the a lock code request is completed
 *
 * @param FOnlineError The Result of the HttpRequest
 * @param TSharedRef<FCodeLockInfo> Data From a successful Request
 */
DECLARE_DELEGATE_TwoParams(FOnProductCodeLockedComplete, const FOnlineError& /*Result*/, const TSharedRef<FCodeLockInfo>&);

/**
 * Delegate used when the an evaluate code request is completed
 *
 * @param FOnlineError The Result of the HttpRequest
 */
DECLARE_DELEGATE_OneParam(FOnProductCodeUnlockedComplete, const FOnlineError& /*Result*/);

/**
 * Delegate used when the an evaluate code request is completed
 *
 * @param FOnlineError The Result of the HttpRequest
 * @param TSharedRef<FCodeEvaluationInfo> Data From a successful Request
 */
DECLARE_DELEGATE_TwoParams(FOnProductCodeEvaluateComplete, const FOnlineError& /*Result*/, const TSharedRef<FCodeEvaluationInfo>&);

/**
 *	IOnlineCodeRedemption - Interface for locking, unlocking, and evaluating codes. Use IOnlineFulfillment to redeem it.
 */
class IOnlineCodeRedemption
{
public:
	virtual ~IOnlineCodeRedemption() = default;

	/**
	 * Initiate the lock code process for reserving a product redemption
	 *
	 * @param CodeId The Redemption code to lock
	 * @param LockTimeoutSeconds The duration in seconds to keep the code locked for
	 * @param Delegate completion callback (guaranteed to be called)
	 */
	virtual void LockCode(const FString& CodeId, int32 LockTimeoutSeconds, const FOnProductCodeLockedComplete& Delegate) = 0;

	/**
	 * Initiate the Release code process to clear reservation on a product code
	 *
	 * @param CodeId The Redemption code to unlock
	 * @param CodeUseId The in use Code Id that was returned from LockCode
	 * @param Delegate completion callback (guaranteed to be called)
	 */
	virtual void UnlockCode(const FString& CodeId, const FString& CodeUseId, const FOnProductCodeUnlockedComplete& Delegate) = 0;

	/**
	 * Initiate the Evaluation Code process for checking if a code is valid and what product information it relates too
	 *
	 * @param UserId User initiating the request
	 * @param CodeId The Redemption code to evaluate
	 * @param Delegate completion callback (guaranteed to be called)
	 */
	virtual void EvaluateCode(const FUniqueNetId& UserId, const FString& CodeId, const FOnProductCodeEvaluateComplete& Delegate) = 0;
};
