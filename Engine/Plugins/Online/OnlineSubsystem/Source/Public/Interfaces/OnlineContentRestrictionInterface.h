// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineDelegateMacros.h"
#include "OnlineSubsystemTypes.h"

class FUniqueNetId;
struct FOnlineError;

/**
 * Delegate fired when content age restriction check is completed
 * 
 * @param LocalUserId the id of the player that this callback is intended for
 * @param bAgeRestricted the restriction result (is or is not restricted)
 * @param Result whether or not the process completed successfully 
 */
DECLARE_DELEGATE_ThreeParams(FOnQueryContentAgeRestictionComplete, const FUniqueNetId& /* LocalUserId */, const bool /* bAgeRestricted */, const FOnlineError& /* Result */);

/**
 *	IOnlineContentAgeRestriction - Interface class for managing a user's content age restriction status
 */
class IOnlineContentAgeRestriction
{
public:
	virtual ~IOnlineContentAgeRestriction() = default;

	/**
	 * Query the content age restriction status for an account
	 * 
	 * @param LocalUserId - Id of the player to check the content age restriction status for
	 * @param RatingBoards - Array of rating boards defined to query status with
	 * @param CompletionDelegate - Completion delegate called when the QueryContentAgeRestriction call is complete
	 */
	virtual void QueryContentAgeRestriction(const FUniqueNetId& LocalUserId, const TArray<FString>& RatingBoards, FOnQueryContentAgeRestictionComplete CompletionDelegate) = 0;
};