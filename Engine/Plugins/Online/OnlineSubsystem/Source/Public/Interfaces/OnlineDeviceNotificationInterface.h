// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

class FString;
class FUniqueNetId;

struct FOnlineError;

/**
 * Delegate used when the register device request is completed
 *
 * @param FOnlineError The Result of the Request
 */
DECLARE_DELEGATE_OneParam(FOnRegisterDeviceComplete, const FOnlineError& /*Result*/);

/**
 * Delegate used when the unregister device by channel type request is completed
 *
 * @param FOnlineError The Result of the Request
 */
DECLARE_DELEGATE_OneParam(FOnUnregisterDeviceByChannelTypeComplete, const FOnlineError& /*Result*/);

/**
 *	IOnlineDeviceNotification - Interface for communications service
 */
class IOnlineDeviceNotification
{
public:
	virtual ~IOnlineDeviceNotification() = default;

	/**
	 * Initiate the create/register device process
	 *
	 * @param LocalUserId user making the request
	 * @param InstanceId the device registration token generated on the device
	 * @param ChannelType the channel being registered (for example: fcm)
	 * @param Delegate completion callback (guaranteed to be called)
	 */
	virtual void RegisterDevice(const FUniqueNetId& LocalUserId, const FString& InstanceId, const FString& ChannelType, const FOnRegisterDeviceComplete& Delegate) = 0;

	/**
	 * Initiate the unregister device by channel type process
	 *
	 * @param LocalUserId user making the request
	 * @param InstanceId the device registration token generated on the device
	 * @param ChannelType the channel being unregistered (for example: fcm)
	 * @param Delegate completion callback (guaranteed to be called)
	*/
	virtual void UnregisterDeviceByChannelType(const FUniqueNetId& LocalUserId, const FString& InstanceId, const FString& ChannelType, const FOnUnregisterDeviceByChannelTypeComplete& Delegate) = 0;
};
