// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemTypes.h"
#include "OnlineDelegateMacros.h"

ONLINESUBSYSTEM_API DECLARE_LOG_CATEGORY_EXTERN(LogOnlineEntitlement, Log, All);

#define UE_LOG_ONLINE_ENTITLEMENT(Verbosity, Format, ...) \
{ \
	UE_LOGF(LogOnlineEntitlement, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

#define UE_CLOG_ONLINE_ENTITLEMENT(Conditional, Verbosity, Format, ...) \
{ \
	UE_CLOGF(Conditional, LogOnlineEntitlement, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

/** 
 * unique identifier for entitlements
 */
typedef FString FUniqueEntitlementId;

/**
 * Details of an entitlement
 */
struct FOnlineEntitlement
{
	/** Unique Entitlement Id associated with this entitlement */
	FUniqueEntitlementId Id;
	/** Display name for the entitlement */
	FString Name;
	/** Id for the item that this entitlement is associated with */
	FString ItemId;
	/**  Namespace of the entitlement */
	FString Namespace;
	/** True if the entitlement is a consumable */
	bool bIsConsumable;
	/** Number of uses still available for a consumable */
	int32 RemainingCount;
	/** Number of prior uses for a consumable */
	int32 ConsumedCount;
	/** When the entitlement started */
	FString StartDate;
	/** When the entitlement will expire */
	FString EndDate;
	/** Current Status of the entitlement e.g. Active, Subscribe, Expire ... */
	FString Status;

	FOnlineEntitlement()
		: bIsConsumable(false)
		, RemainingCount(1)
		, ConsumedCount(0)
	{}

	virtual ~FOnlineEntitlement() = default;

	/**
	 * @return Any additional data associated with the entitlement
 	 */
	virtual bool GetAttribute(const FString& AttrName, FString& OutAttrValue) const
	{
		return false;
	}

	/**
	 * Equality operator
	 */
	bool operator==(const FOnlineEntitlement& Other) const
	{
		return Other.Id == Id;
	}
};


/**
 * Delegate declaration for when entitlements are enumerated
 *
 * @param bWasSuccessful true if server was contacted and a valid result received
 * @param UserId of the user who was granted entitlements in this callback
 * @param Namespace optional namespace that was used to query. Empty means all entitlements were queried
 * @param Error string representing the error condition
 */
DECLARE_MULTICAST_DELEGATE_FourParams(FOnQueryEntitlementsComplete, bool /*bWasSuccessful*/, const FUniqueNetId& /*UserId*/, const FString& /*Namespace*/, const FString& /*Error*/);
typedef FOnQueryEntitlementsComplete::FDelegate FOnQueryEntitlementsCompleteDelegate;

/**
 * Interface for retrieving user entitlements
 */
class IOnlineEntitlements
{
public:
	virtual ~IOnlineEntitlements() = default;

	/**
	 * Checks for and retrieves a single cached entitlement for a user
	 *
	 * @param UserId the ID of the user to get this entitlement for
	 * @param EntitlementId the ID of the entitlement to retrieve
	 *
	 * @return entitlement entry if found, null otherwise
	 */
	virtual TSharedPtr<FOnlineEntitlement> GetEntitlement(const FUniqueNetId& UserId, const FUniqueEntitlementId& EntitlementId) = 0;

	/**
	 * Checks for and retrieves a single cached entitlement for a user
	 *
	 * @param UserId the ID of the user to get this entitlement for
	 * @param ItemId the ID of the item to retrieve an entitlement for
	 *
	 * @return entitlement entry if found, null otherwise
	 */
	virtual TSharedPtr<FOnlineEntitlement> GetItemEntitlement(const FUniqueNetId& UserId, const FString& ItemId) = 0;
	
	/**
	 * Gets the cached entitlement set for the requested user
	 *
	 * @param UserId the ID of the user to get entitlements for
	 * @param Namespace optional namespace to filter on
	 * @param OutUserEntitlements out parameter to copy the user's entitlements into
	 */
	virtual void GetAllEntitlements(const FUniqueNetId& UserId, const FString& Namespace, TArray<TSharedRef<FOnlineEntitlement>>& OutUserEntitlements) = 0;

	/**
	 * Contacts server and retrieves the list of the user's entitlements, caching them locally
	 *
	 * @param UserId the ID of the user to act on
	 * @param Namespace optional namespace to filter on
	 *
	 * @return true if the operation started successfully
	 */
	virtual bool QueryEntitlements(const FUniqueNetId& UserId, const FString& Namespace, const FPagedQuery& Page = FPagedQuery()) = 0;
	
	/**
	 * Delegate instanced called when enumerating entitlements has completed
	 *
	 * @param bWasSuccessful true if server was contacted and a valid result received
	 * @param UserId of the user who was granted entitlements in this callback
	 * @param Namespace optional namespace that was used to query. Empty means all entitlements were queried
	 * @param Error string representing the error condition
	 */
	DEFINE_ONLINE_DELEGATE_FOUR_PARAM(OnQueryEntitlementsComplete, bool /*bWasSuccessful*/, const FUniqueNetId& /*UserId*/, const FString& /*Namespace*/, const FString& /*Error*/);
};

typedef TSharedPtr<IOnlineEntitlements, ESPMode::ThreadSafe> IOnlineEntitlementsPtr;
