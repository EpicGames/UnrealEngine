// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineKeyValuePair.h"

ONLINESUBSYSTEM_API DECLARE_LOG_CATEGORY_EXTERN(LogOnlineEvents, Log, All);
#define UE_LOG_ONLINE_EVENTS(Verbosity, Format, ...) \
{ \
	UE_LOGF(LogOnlineEvents, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

#define UE_CLOG_ONLINE_EVENTS(Conditional, Verbosity, Format, ...) \
{ \
	UE_CLOGF(Conditional, LogOnlineEvents, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

class FUniqueNetId;

class FOnlineEventParms : public FOnlineKeyValuePairs<FString, FVariantData> {};

/**
 *	IOnlineEvents - Interface class for events
 */
class IOnlineEvents
{
public:

	/**
	 * Trigger an event by name
	 *
	 * @param PlayerId	- Player to trigger the event for
	 * @param EventName - Name of the event
	 * @param Parms		- The parameter list to be passed into the event
	 *
	 * @return Whether the event was successfully triggered
	 */
	virtual bool TriggerEvent( const FUniqueNetId& PlayerId, const TCHAR* EventName, const FOnlineEventParms& Parms ) = 0;

	/**
	 * Quick way to send a valid PlayerSessionId with every event, required for Xbox One
	 *
	 * @param PlayerId the unique id of the player this session is associated with
	 * @param PlayerSessionId A GUID unique to this player session
	 */
	virtual void SetPlayerSessionId(const FUniqueNetId& PlayerId, const FGuid& PlayerSessionId) = 0;
};
