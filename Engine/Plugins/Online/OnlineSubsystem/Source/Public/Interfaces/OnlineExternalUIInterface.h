// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineMessageInterface.h"
#include "OnlineError.h"

class FOnlineAccountCredentials;

ONLINESUBSYSTEM_API DECLARE_LOG_CATEGORY_EXTERN(LogOnlineExternalUI, Log, All);

#define UE_LOG_ONLINE_EXTERNALUI(Verbosity, Format, ...) \
{ \
	UE_LOGF(LogOnlineExternalUI, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

#define UE_CLOG_ONLINE_EXTERNALUI(Conditional, Verbosity, Format, ...) \
{ \
	UE_CLOGF(Conditional, LogOnlineExternalUI, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

/**
 * FLoginFlowResult codes
 */
#define LOGIN_TOKEN_FOUND FString()
#define LOGIN_ERROR_UNKNOWN					TEXT("com.epicgames.login.unknown")
#define LOGIN_ERROR_AUTH_FAILURE			TEXT("com.epicgames.login.auth_failure")
#define LOGIN_ERROR_NO_EXTERNAL_ACCOUNT		TEXT("com.epicgames.login.no_account_found_for_external_auth")
#define LOGIN_CANCELLED						TEXT("com.epicgames.login.canceled")
#define LOGIN_PAGELOADFAILED				TEXT("com.epicgames.login.pageloadfailed")
#define LOGIN_CEFLOADFAILED					TEXT("com.epicgames.login.cefloadfailed")

/**
 * Delegate called when the external UI is opened or closed
 *
 * @param bIsOpening state of the external UI
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnExternalUIChange, bool);
typedef FOnExternalUIChange::FDelegate FOnExternalUIChangeDelegate;


/**
 * A single login flow result
 */
struct FLoginFlowResult
{
	FLoginFlowResult()
		: Error(EOnlineErrorResult::Unknown)
		, NumericErrorCode(0)
	{ }

	/** Token returned by the login flow (platform dependent) */
	FString Token;
	/** Errors generated during login flow */
	FOnlineError Error;
	/** Numeric error code provided by the backend expected to correspond to error stored in ErrorCode */
	int32 NumericErrorCode;

	/** @return true if this result is usable for further login steps */
	bool IsValid() const { return !Token.IsEmpty(); }

	/** @return true if this result has reached a finished state */
	bool IsComplete() const
	{ 
		const bool bSuccess = Error.WasSuccessful() && !Token.IsEmpty();
		const bool bError = !Error.WasSuccessful() && NumericErrorCode;
		return bSuccess || bError;
	}

	FString ToDebugString() const
	{
#if UE_BUILD_SHIPPING
		return FString::Printf(TEXT("Token: [REDACTED] Code: %d Error: %s"), NumericErrorCode, *Error.ToLogString());
#else
		return FString::Printf(TEXT("Token: %s Code: %d Error: %s"), *Token, NumericErrorCode, *Error.ToLogString());
#endif
	}
};

/**
 * Delegate executed with a login flow has completed
 *
 * @Param Result the result of the login process
 */
DECLARE_DELEGATE_OneParam(FOnLoginFlowComplete, const FLoginFlowResult& /*Result*/);

/**
 * Delegate executed when a redirect URL request has been received by the internal browser window
 *
 * @param URL location of the next page
 *
 * @return the result of processing this URL.  Result.IsComplete() should return true if the login flow is expected to terminate 
 */
DECLARE_DELEGATE_RetVal_OneParam(FLoginFlowResult, FOnLoginRedirectURL, const FString& /*URL*/);

/**
 * Delegate executed by the requesting system when a supporting login flow browser window is asked to handle the request
 *
 * @param RequestedURL destination to fulfill the login request
 * @param OnLoginRedirect delegate to fire when URL redirects are detected
 * @param OnLoginFlowComplete delegate to fire when the login flow code has reached flow conclusion
 * @param bOutShouldContinueLogin true if the login flow code can proceed with the information given, false the caller should handle login failure
 */
DECLARE_MULTICAST_DELEGATE_FourParams(FOnLoginFlowUIRequired, const FString& /*RequestedURL*/, const FOnLoginRedirectURL& /*OnLoginRedirect*/, const FOnLoginFlowComplete& /*OnLoginFlowComplete*/, bool& /*bOutShouldContinueLogin*/);
typedef FOnLoginFlowUIRequired::FDelegate FOnLoginFlowUIRequiredDelegate;

//DECLARE_DELEGATE_OneParam(FOnCreateAccountFlowComplete, const FLoginFlowResult& /*Result*/);
//DECLARE_DELEGATE_RetVal_OneParam(FLoginFlowResult, FOnLoginRedirectURL, const FString& /*URL*/);

/**
 * Delegate executed by the requesting system when a supporting account creation flow browser window is asked to handle the request
 *
 * @param RequestedURL destination to fulfill the creation request
 * @param OnLoginRedirect delegate to fire when URL redirects are detected
 * @param OnLoginFlowComplete delegate to fire when the creation flow code has reached flow conclusion
 * @param bOutShouldContinueLogin true if the creation flow code can proceed with the information given, false the caller should handle creation failure
 */
DECLARE_MULTICAST_DELEGATE_FourParams(FOnCreateAccountFlowUIRequired, const FString& /*RequestedURL*/, const FOnLoginRedirectURL& /*OnLoginRedirect*/, const FOnLoginFlowComplete& /*OnLoginFlowComplete*/, bool& /*bOutShouldContinueLogin*/);
typedef FOnCreateAccountFlowUIRequired::FDelegate FOnCreateAccountFlowUIRequiredDelegate;

/**
 * Delegate executed when the external login UI has been closed.
 *
 * @param UniqueId The unique id of the user who signed in. Null if no user signed in.
 * @param ControllerIndex The controller index of the controller that activated the login UI.
 * @param Error any errors related to closing the UI
 */
DECLARE_DELEGATE_ThreeParams(FOnLoginUIClosedDelegate, FUniqueNetIdPtr /*UniqueId*/, const int /*ControllerIndex*/, const FOnlineError& /*Error*/);

/**
 * Delegate executed when the external account creation UI has been closed.
 *
 * @param ControllerIndex The controller index of the controller that activated the account creation UI.
 * @param AccountCredentials Account credentials created for use by login if successful, invalid otherwiseS
 * @param Error any errors related to closing the UI
 */
DECLARE_DELEGATE_ThreeParams(FOnAccountCreationUIClosedDelegate, const int /*ControllerIndex*/, const FOnlineAccountCredentials& /*AccountCredentials*/, const FOnlineError& /*Error*/);

/**
 * Delegate executed when the web url UI has been closed
 *
 * @param FinalUrl the url that was used as the final redirect before closing
 */
DECLARE_DELEGATE_OneParam(FOnShowWebUrlClosedDelegate, const FString& /*FinalUrl*/);

/**
 * Delegate executed when the store UI has been closed
 *
 * @param bPurchased true if a purchase occurred via the store ui
 */
DECLARE_DELEGATE_OneParam(FOnShowStoreUIClosedDelegate, bool /*bPurchased*/);

/**
 * Delegate executed when the send message UI has been closed
 *
 * @param bMessageSent if the user chose to send the message
 */
DECLARE_DELEGATE_OneParam(FOnShowSendMessageUIClosedDelegate, bool /*bMessageSent*/);

/** 
 * Delegate executed when the user profile UI has been closed. 
 */
DECLARE_DELEGATE(FOnProfileUIClosedDelegate);

/** Parameters used to show a web UI */
struct FShowWebUrlParams
{
	/** presented without a frame if embedded enabled */
	bool bEmbedded;
	/** Show the built in close button */
	bool bShowCloseButton;
	/** Show the built in background */
	bool bShowBackground;
	/** Hide the mouse cursor */
	bool bHideCursor;
	/** Rest cookies before invoking web browser */
	bool bResetCookies;
	/** x offset in pixels from top left */
	int32 OffsetX;
	/** y offset in pixels from top left */
	int32 OffsetY;
	/** x size in pixels */
	int32 SizeX;
	/** y size in pixels */
	int32 SizeY;
	/** if specified then restricted to only navigate within these domains */
	TArray<FString> AllowedDomains;
	/** portion of url for detecting callback.  Eg. "&code=", "redirect=", etc */
	FString CallbackPath;

	/**
	 * Constructor
	 */
	FShowWebUrlParams(bool InbEmbedded, int32 InOffsetX, int32 InOffsetY, int32 InSizeX, int32 InSizeY)
		: bEmbedded(InbEmbedded)
		, bShowCloseButton(false)
		, bShowBackground(false)
		, bHideCursor(false)
		, bResetCookies(false)
		, OffsetX(InOffsetX)
		, OffsetY(InOffsetY)
		, SizeX(InSizeX)
		, SizeY(InSizeY)
	{}

	/**
	 * Default Constructor
	 */
	FShowWebUrlParams()
		: bEmbedded(false)
		, bShowCloseButton(false)
		, bShowBackground(false)
		, bHideCursor(false)
		, bResetCookies(false)
		, OffsetX(0)
		, OffsetY(0)
		, SizeX(0)
		, SizeY(0)
	{}
};

typedef FString FUniqueOfferId;

struct FShowStoreParams
{
	/** Category filter for products to browse */
	FString Category;

	/** Product to show directly instead of the whole store */
	FUniqueOfferId ProductId;

	/** If the specific product id should be added to the user's cart automatically */
	bool bAddToCart;

	/**
	 * Constructor
	 */
	FShowStoreParams(const FString& InCategory = FString())
		: Category(InCategory)
		, bAddToCart(false)
	{}
};

struct FShowSendMessageParams
{
	FText DisplayTitle;
	/** Map of language to text so the recipient's platform OS can display the localized string */
	TMap<FString, FString> DisplayTitle_Loc;
	FText DisplayMessage;
	FText DisplayDetails;
	/** Map of language to text so the recipient's platform OS can display the localized string */
	TMap<FString, FString> DisplayDetails_Loc;
	TArray<uint8> DisplayThumbnail;
	FOnlineMessagePayload DataPayload;
	int32 MaxRecipients;

	/**
	 * Constructor
	 */
	FShowSendMessageParams()
		: MaxRecipients(1)
	{}
};

/**
 * Union of all the platform informational message types we handle (some may be handled by more than one platform)
 */
enum class EPlatformMessageType
{
	EmptyStore,
	ChatRestricted,
	UGCRestricted
};

/**
* Shop Purchasable types
*/
#define SHOP_PURCHASABLE_APPLICATION		TEXT("Application")
#define SHOP_PURCHASABLE_ADDON				TEXT("AddOnContent")
#define SHOP_PURCHASABLE_CONSUMABLE			TEXT("ConsumableContent")
#define SHOP_PURCHASABLE_SUBSCRIPTION		TEXT("SubscriptionContent")

/** 
 * Interface definition for the online services external UIs
 * Any online service that provides extra UI overlays will implement the relevant functions
 */
class IOnlineExternalUI
{
protected:
	IOnlineExternalUI() {};

public:
	virtual ~IOnlineExternalUI() = default;

	/**
	 * Displays the UI that prompts the user for their login credentials. Each
	 * platform handles the authentication of the user's data.
	 *
	 * @param ControllerIndex The controller that prompted showing the login UI. If the platform supports it,
	 * it will pair the signed-in user with this controller.
	 * @param bShowOnlineOnly whether to only display online enabled profiles or not
	 * @param bShowSkipButton On platforms that support it, display the "Skip" button
	 * @param Delegate The delegate to execute when the user closes the login UI.
	 *
	 * @return true if it was able to show the UI, false if it failed
	 */
	virtual bool ShowLoginUI(const int ControllerIndex, bool bShowOnlineOnly, bool bShowSkipButton, const FOnLoginUIClosedDelegate& Delegate = FOnLoginUIClosedDelegate()) = 0;

	/**
	 * Displays the UI that prompts the user to create an account
	 *
	 * @param ControllerIndex The controller that prompted showing the account creation UI. If the platform supports it,
	 * it will pair the signed-in user with this controller.
	 * @param Delegate The delegate to execute when the user closes the account creation UI.
	 *
	 * @return true if it was able to show the UI, false if it failed
	 */
	virtual bool ShowAccountCreationUI(const int ControllerIndex, const FOnAccountCreationUIClosedDelegate& Delegate = FOnAccountCreationUIClosedDelegate()) = 0;

	/**
	 * Displays the UI that shows a user's list of friends
	 *
	 * @param LocalUserNum the controller number of the associated user
	 *
	 * @return true if it was able to show the UI, false if it failed
	 */
	virtual bool ShowFriendsUI(int32 LocalUserNum) = 0;

	/**
	 *	Displays the UI that shows a user's list of friends to invite
	 *
	 * @param LocalUserNum the controller number of the associated user
	 * @param SessionName name of session associated with the invite
	 *
	 * @return true if it was able to show the UI, false if it failed
	 */
	virtual bool ShowInviteUI(int32 LocalUserNum, FName SessionName = NAME_GameSession) = 0;

	/**
	 *	Displays the UI that shows a user's list of achievements
	 *
	 * @param LocalUserNum the controller number of the associated user
	 *
	 * @return true if it was able to show the UI, false if it failed
	 */
	virtual bool ShowAchievementsUI(int32 LocalUserNum) = 0;

	/**
	 *	Displays the UI that shows a specific leaderboard
	 *
	 * @param LeaderboardName the name of the leaderboard to show
	 *
	 * @return true if it was able to show the UI, false if it failed
	 */
	virtual bool ShowLeaderboardUI(const FString& LeaderboardName) = 0;

	/**
	 * Displays a web page in the external UI
	 *
	 * @param WebURL fully formed web address (http://www.google.com)
	 * @param ShowParams configuration for the UI display
	 *
	 * @return true if it was able to show the UI, false if it failed
	 */
	virtual bool ShowWebURL(const FString& Url, const FShowWebUrlParams& ShowParams, const FOnShowWebUrlClosedDelegate& Delegate = FOnShowWebUrlClosedDelegate()) = 0;

	/**
	 * Closes the currently active web external UI
	 *
	 * @return true if it was able to show the UI, false if it failed
	 */
	virtual bool CloseWebURL() = 0;

	/**
	 * Displays a user's profile card.
	 *
	 * @param Requestor The user requesting the profile.
	 * @param Requestee The user for whom to show the profile.
	 *
	 * @return true if it was able to show the UI, false if it failed
	 */
	virtual bool ShowProfileUI(const FUniqueNetId& Requestor, const FUniqueNetId& Requestee, const FOnProfileUIClosedDelegate& Delegate = FOnProfileUIClosedDelegate()) = 0;

	/**
	 * Displays a system dialog to purchase user account upgrades.  e.g. PlaystationPlus, XboxLive GOLD, etc.
	 *
	 * @param UniqueID of the user to show the dialog for
	 *
	 * @return true if it was able to show the UI, false if it failed
	 */
	virtual bool ShowAccountUpgradeUI(const FUniqueNetId& UniqueId) = 0;

	/**
	 * Display the platform UI for browsing available products for purchase
	 *
	 * @param LocalUserNum the controller number of the associated user
	 * @param ShowParams configuration for the UI display
	 *
	 * @return true if it was able to show the UI, false if it failed
	 */
	virtual bool ShowStoreUI(int32 LocalUserNum, const FShowStoreParams& ShowParams, const FOnShowStoreUIClosedDelegate& Delegate = FOnShowStoreUIClosedDelegate()) = 0;

	/**
	 * Display the platform UI for sending a mailbox message to another user
	 *
	 * @param LocalUserNum the controller number of the associated user
	 * @param ShowParams configuration for the UI display
	 *
	 * @return true if it was able to show the UI, false if it failed
	 */
	virtual bool ShowSendMessageUI(int32 LocalUserNum, const FShowSendMessageParams& ShowParams, const FOnShowSendMessageUIClosedDelegate& Delegate = FOnShowSendMessageUIClosedDelegate()) = 0;

	/**
	 * Display the platform UI for sending a mailbox message to the specified user.
	 * Will attempt to automatically send the message if the platform allows for it.
	 *
	 * @param LocalUserNum the controller number of the associated user
	 * @param Recipient the user that should receive this message
	 * @param ShowParams configuration for the UI display
	 *
	 * @return true if it was able to send the message and show the UI, false if it failed
	 */
	virtual bool ShowSendMessageToUserUI(int32 LocalUserNum, const FUniqueNetId& Recipient, const FShowSendMessageParams& ShowParams, const FOnShowSendMessageUIClosedDelegate& Delegate = FOnShowSendMessageUIClosedDelegate())
	{
		UE_LOGF(LogOnlineExternalUI, Warning, "ShowSendMessageToUserUI is not implemented on this platform!");
		Delegate.ExecuteIfBound(false);
		return false;
	}

	/**
	 * Displays an informational system dialog.
	 *
	 * @param UserId Who to show this dialog for.
	 * @param MessageId Platform-specific identifier for the message box to display.
	 */
	virtual bool ShowPlatformMessageBox(const FUniqueNetId& UserId, EPlatformMessageType MessageType) { return false; }

	/**
	 * Tell the system that you've entered into a screen considered in-game store.
	 */
	virtual void ReportEnterInGameStoreUI() {}

	/**
	 * Tell the system that you've entered into a screen considered in-game store.
	 */
	virtual void ReportExitInGameStoreUI() {}

	/**
	 * Delegate called when the external UI is opened or closed
	 *
	 * @param bIsOpening state of the external UI
	 */
	DEFINE_ONLINE_DELEGATE_ONE_PARAM(OnExternalUIChange, bool);

	/**
	 * Delegate called when the online subsystem requires an external UI to handle login flow
	 */
	DEFINE_ONLINE_DELEGATE_FOUR_PARAM(OnLoginFlowUIRequired, const FString& /*RequestedURL*/, const FOnLoginRedirectURL& /*OnRedirectURL*/, const FOnLoginFlowComplete& /*OnLoginFlowComplete*/, bool& /*bOutShouldContinueLogin*/);
	/**
	 * Delegate called when the online subsystem requires an external UI to handle account creation
	 */
	DEFINE_ONLINE_DELEGATE_FOUR_PARAM(OnCreateAccountFlowUIRequired, const FString& /*RequestedURL*/, const FOnLoginRedirectURL& /*OnRedirectURL*/, const FOnLoginFlowComplete& /*OnLoginFlowComplete*/, bool& /*bOutShouldContinueLogin*/);
};

typedef TSharedPtr<IOnlineExternalUI, ESPMode::ThreadSafe> IOnlineExternalUIPtr;

