// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemTypes.h"
#include "OnlineDelegateMacros.h"

struct FOnlineError;
enum class EFriendInvitePolicy : uint8;

ONLINESUBSYSTEM_API DECLARE_LOG_CATEGORY_EXTERN(LogOnlineFriend, Log, All);

#define UE_LOG_ONLINE_FRIEND(Verbosity, Format, ...) \
{ \
	UE_LOGF(LogOnlineFriend, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

#define UE_CLOG_ONLINE_FRIEND(Conditional, Verbosity, Format, ...) \
{ \
	UE_CLOGF(Conditional, LogOnlineFriend, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

/** List of known friends list types */
namespace EFriendsLists
{
	enum Type
	{
		/** default friends list */
		Default,
		/** online players friends list */
		OnlinePlayers,
		/** list of players running the same title/game */
		InGamePlayers,
		/** list of players running the same title/game and in a session that has started */
		InGameAndSessionPlayers,
	};

	/** @return the stringified version of the enum passed in */
	inline const TCHAR* ToString(EFriendsLists::Type EnumVal)
	{
		switch (EnumVal)
		{
			case Default:
				return TEXT("default");
			case OnlinePlayers:
				return TEXT("onlinePlayers");
			case InGamePlayers:
				return TEXT("inGamePlayers");
			case InGameAndSessionPlayers:
				return TEXT("inGameAndSessionPlayers");
		}
		return TEXT("");
	}

	/** @return whether or not the passed string was successfully converted to a valid enum value */
	inline bool FromString(EFriendsLists::Type& OutEnum, const TCHAR* InString)
	{
		if (FCString::Stricmp(InString, TEXT("default")) == 0)
		{
			OutEnum = EFriendsLists::Type::Default;
		}
		else if (FCString::Stricmp(InString, TEXT("onlinePlayers")) == 0)
		{
			OutEnum = EFriendsLists::Type::OnlinePlayers;
		}
		else if (FCString::Stricmp(InString, TEXT("inGamePlayers")) == 0)
		{
			OutEnum = EFriendsLists::Type::InGamePlayers;
		}
		else if (FCString::Stricmp(InString, TEXT("inGameAndSessionPlayers")) == 0)
		{
			OutEnum = EFriendsLists::Type::InGameAndSessionPlayers;
		}
		else
		{
			checkNoEntry();
			return false;
		}

		return true;
	}
}

/*
 * Stores a generic list of settings for the Query/UpdateSettings calls
 */
struct FFriendSettings
{
	FFriendSettings() {}

	virtual ~FFriendSettings() = default;

	virtual bool GetSettingValue(const FString& SettingName, FString& Value) const
	{
		if (const FString* Result = SettingsMap.Find(SettingName))
		{
			Value = *Result;
			return true;
		}

		return false;
	}

	virtual void SetSettingValue(const FString& SettingName, const FString& Value)
	{
		SettingsMap.Add(SettingName, Value);
	}

	TMap<FString, FString> SettingsMap;
};

/**
 * Encounter type of recent player
 */
enum class ERecentPlayerEncounterType : uint8
{
	Default,
	Teammate,
	Opponent
};

/**
 * Stores information about a recent player
 */
struct FReportPlayedWithUser
{
	FReportPlayedWithUser() = delete;
	FReportPlayedWithUser(const FUniqueNetIdRef& InUserId, const FString& InPresenceStr, ERecentPlayerEncounterType InEncounterType=ERecentPlayerEncounterType::Default)
		: UserId(InUserId)
		, PresenceStr(InPresenceStr)
		, EncounterType(InEncounterType)
	{
	}

	/** UserId to report played with */
	FUniqueNetIdRef UserId;
	/** Optional presence string */
	FString PresenceStr;
	/** Encounter type of this user */
	ERecentPlayerEncounterType EncounterType;
};

/**
 * Delegate used in friends list change notifications
 */
DECLARE_MULTICAST_DELEGATE(FOnFriendsChange);
typedef FOnFriendsChange::FDelegate FOnFriendsChangeDelegate;

/**
 * Delegate used in outgoing invite list change notifications
 */
DECLARE_MULTICAST_DELEGATE(FOnOutgoingInviteSent);
typedef FOnOutgoingInviteSent::FDelegate FOnOutgoingInviteSentDelegate;

/**
 * Delegate used when the friends read request has completed
 *
 * @param LocalUserNum the controller number of the associated user that made the request
 * @param bWasSuccessful true if the async action completed without error, false if there was an error
 * @param ListName name of the friends list that was operated on
 * @param ErrorStr string representing the error condition
 */
DECLARE_DELEGATE_FourParams(FOnReadFriendsListComplete, int32 /*LocalUserNum*/, bool /*bWasSuccessful*/, const FString& /*ListName*/, const FString& /*ErrorStr*/);

/**
 * Delegate used when the friends list delete request has completed
 *
 * @param LocalUserNum the controller number of the associated user that made the request
 * @param bWasSuccessful true if the async action completed without error, false if there was an error
 * @param ListName name of the friends list that was operated on
 * @param ErrorStr string representing the error condition
 */
DECLARE_DELEGATE_FourParams(FOnDeleteFriendsListComplete, int32 /*LocalUserNum*/, bool /*bWasSuccessful*/, const FString& /*ListName*/, const FString& /*ErrorStr*/);

/**
 * Delegate used when an invite send request has completed
 *
 * @param LocalUserNum the controller number of the associated user that made the request
 * @param bWasSuccessful true if the async action completed without error, false if there was an error
 * @param FriendId player that was invited
 * @param ListName name of the friends list that was operated on
 * @param ErrorStr string representing the error condition
 */
DECLARE_DELEGATE_FiveParams(FOnSendInviteComplete, int32 /*LocalUserNum*/, bool /*bWasSuccessful*/, const FUniqueNetId& /*FriendId*/, const FString& /*ListName*/, const FString& /*ErrorStr*/);

/**
 * Delegate used when an invite accept request has completed
 *
 * @param LocalUserNum the controller number of the associated user that made the request
 * @param bWasSuccessful true if the async action completed without error, false if there was an error
 * @param FriendId player that invited us
 * @param ListName name of the friends list that was operated on
 * @param ErrorStr string representing the error condition
 */
DECLARE_DELEGATE_FiveParams(FOnAcceptInviteComplete, int32 /*LocalUserNum*/, bool /*bWasSuccessful*/, const FUniqueNetId& /*FriendId*/, const FString& /*ListName*/, const FString& /*ErrorStr*/);

/**
 * Delegate used when an invite reject request has completed
 *
 * @param LocalUserNum the controller number of the associated user that made the request
 * @param bWasSuccessful true if the async action completed without error, false if there was an error
 * @param FriendId player that invited us
 * @param ListName name of the friends list that was operated on
 * @param ErrorStr string representing the error condition
 */
DECLARE_MULTICAST_DELEGATE_FiveParams(FOnRejectInviteComplete, int32 /*LocalUserNum*/, bool /*bWasSuccessful*/, const FUniqueNetId& /*FriendId*/, const FString& /*ListName*/, const FString& /*ErrorStr*/);
typedef FOnRejectInviteComplete::FDelegate FOnRejectInviteCompleteDelegate;

/**
 * Delegate used when a set alias request has completed
 *
 * @param LocalUserNum the controller number of the associated user that made the request
 * @param FriendId player that we set alias for
 * @param ListName name of the friends list that was operated on
 * @param Error error information on failure
 */
DECLARE_DELEGATE_FourParams(FOnSetFriendAliasComplete, int32 /*LocalUserNum*/, const FUniqueNetId& /*FriendId*/, const FString& /*ListName*/, const FOnlineError& /*Error*/);

/**
 * Delegate used when a delete friend alias request has completed
 *
 * @param LocalUserNum the controller number of the associated user that made the request
 * @param FriendId player that we delete alias from
 * @param ListName name of the friends list that was operated on
 * @param Error error information on failure
 */
DECLARE_DELEGATE_FourParams(FOnDeleteFriendAliasComplete, int32 /*LocalUserNum*/, const FUniqueNetId& /*FriendId*/, const FString& /*ListName*/, const FOnlineError& /*Error*/);

/**
 * Delegate used when an friend delete request has completed
 *
 * @param LocalUserNum the controller number of the associated user that made the request
 * @param bWasSuccessful true if the async action completed without error, false if there was an error
 * @param FriendId player that was deleted
 * @param ListName name of the friends list that was operated on
 * @param ErrorStr string representing the error condition
 */
DECLARE_MULTICAST_DELEGATE_FiveParams(FOnDeleteFriendComplete, int32 /*LocalUserNum*/, bool /*bWasSuccessful*/, const FUniqueNetId& /*FriendId*/, const FString& /*ListName*/, const FString& /*ErrorStr*/);
typedef FOnDeleteFriendComplete::FDelegate FOnDeleteFriendCompleteDelegate;

/**
 * Delegate used when a block request has completed
 *
 * @param LocalUserNum the controller number of the associated user that made the request
 * @param bWasSuccessful true if the async action completed without error, false if there was an error
 * @param UniqueID Player blocked
 * @param ListName name of the friends list that was operated on
 * @param ErrorStr string representing the error condition
 */
DECLARE_MULTICAST_DELEGATE_FiveParams(FOnBlockedPlayerComplete, int32 /*LocalUserNum*/, bool /*bWasSuccessful*/, const FUniqueNetId& /*UniqueID*/, const FString& /*ListName*/, const FString& /*ErrorStr*/);
typedef FOnBlockedPlayerComplete::FDelegate FOnBlockedPlayerCompleteDelegate;

/**
 * Delegate used when an unblock request has completed
 *
 * @param LocalUserNum the controller number of the associated user that made the request
 * @param bWasSuccessful true if the async action completed without error, false if there was an error
 * @param UniqueID Player unblocked
 * @param ListName name of the friends list that was operated on
 * @param ErrorStr string representing the error condition
 */
DECLARE_MULTICAST_DELEGATE_FiveParams(FOnUnblockedPlayerComplete, int32 /*LocalUserNum*/, bool /*bWasSuccessful*/, const FUniqueNetId& /*UniqueID*/, const FString& /*ListName*/, const FString& /*ErrorStr*/);
typedef FOnUnblockedPlayerComplete::FDelegate FOnUnblockedPlayerCompleteDelegate;

/**
 * Delegate used in block list change notifications
 *
 * @param LocalUserNum the controller number of the associated user that made the request
 * @param ListName name of the friends list that was operated on
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnBlockListChange, int32 /*LocalUserNum*/, const FString& /*ListName*/);
typedef FOnBlockListChange::FDelegate FOnBlockListChangeDelegate;

/**
 * Delegate used when the query for recent players has completed
 *
 * @param UserId the id of the user that made the request
 * @param Namespace the recent players namespace
 * @param bWasSuccessful true if the async action completed without error, false if there was an error
 * @param ErrorStr string representing the error condition
 */
DECLARE_MULTICAST_DELEGATE_FourParams(FOnQueryRecentPlayersComplete, const FUniqueNetId& /*UserId*/, const FString& /*Namespace*/, bool /*bWasSuccessful*/, const FString& /*Error*/);
typedef FOnQueryRecentPlayersComplete::FDelegate FOnQueryRecentPlayersCompleteDelegate;

/**
 * Delegate used when adding a group of recent players has completed
 *
 * @param UserId the id of the user that made the request
 * @param Error error information on failure
 */
DECLARE_DELEGATE_TwoParams(FOnAddRecentPlayersComplete, const FUniqueNetId& /*UserId*/, const FOnlineError& /*Error*/);

/**
 * Delegate used when recent players have been added
 *
 * @param UserId the id of the user recieving new recent players
 * @param AddedPlayers players that were added to the recent players list
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnRecentPlayersAdded, const FUniqueNetId& /*UserId*/, const TArray<TSharedRef<FOnlineRecentPlayer>>& /*AddedPlayers*/);
typedef FOnRecentPlayersAdded::FDelegate FOnRecentPlayersAddedDelegate;

/**
 * Delegate used when the query for blocked players has completed
 *
 * @param UserId the id of the user that made the request
 * @param bWasSuccessful true if the async action completed without error, false if there was an error
 * @param ErrorStr string representing the error condition
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnQueryBlockedPlayersComplete, const FUniqueNetId& /*UserId*/, bool /*bWasSuccessful*/, const FString& /*Error*/);
typedef FOnQueryBlockedPlayersComplete::FDelegate FOnQueryBlockedPlayersCompleteDelegate;

/**
 * Delegate used when the query for friends settings has completed
 *
 * @param UserId the id of the user that made the request
 * @param bWasSuccessful true if the async action completed without error, false if there was an error
 * @param ErrorStr string representing the error condition
 */
DECLARE_DELEGATE_ThreeParams(FOnQueryFriendSettingsComplete, const FUniqueNetId&, bool, const FString&);

/**
* Delegate used when the query for friends settings has completed
*
* @param UserId the id of the user that made the request
* @param bWasSuccessful true if the async action completed without error, false if there was an error
* @param ErrorStr string representing the error condition
*/
DECLARE_DELEGATE_ThreeParams(FOnSetFriendSettingsComplete, const FUniqueNetId&, bool, const FString&);

/**
 * Delegate called when remote friend sends an invite
 *
 * @param UserId id of the local user that received the invite
 * @param FriendId remote friend id that sent the invite
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnInviteReceived, const FUniqueNetId& /*UserId*/, const FUniqueNetId& /*FriendId*/);
typedef FOnInviteReceived::FDelegate FOnInviteReceivedDelegate;

/**
 * Delegate called when a remote friend accepts an invite
 *
 * @param UserId id of the local user that had sent the invite
 * @param FriendId friend id that accepted the invite
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnInviteAccepted, const FUniqueNetId& /*UserId*/, const FUniqueNetId& /*FriendId*/);
typedef FOnInviteAccepted::FDelegate FOnInviteAcceptedDelegate;

/**
 * Delegate called when a remote friend rejects an invite
 *
 * @param UserId id of the local user that had sent the invite
 * @param FriendId friend id that rejected the invite
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnInviteRejected, const FUniqueNetId& /*UserId*/, const FUniqueNetId& /*FriendId*/);
typedef FOnInviteRejected::FDelegate FOnInviteRejectedDelegate;

/**
* Delegate called when a remote friend cancels/aborts an invite
*
* @param UserId id of the local user that had received the invite
* @param FriendId friend id that canceled that invite
*/
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnInviteAborted, const FUniqueNetId& /*UserId*/, const FUniqueNetId& /*FriendId*/);
typedef FOnInviteAborted::FDelegate FOnInviteAbortedDelegate;

/**
 * Delegate called when a remote friend removes user from friends list
 *
 * @param UserId id of the local user that had the friendship
 * @param FriendId friend id that removed themself from the friendship
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnFriendRemoved, const FUniqueNetId& /*UserId*/, const FUniqueNetId& /*FriendId*/);
typedef FOnFriendRemoved::FDelegate FOnFriendRemovedDelegate;

/**
* Delegate used when the friends settings are read 
*
* @param UserId id of the local user that requested settings operation
* @param bWasSuccessful true if the async action completed without error, false if there was an error
* @param bWasUpdate true if the operation was an "update" operation (vs a passive read)
* @param Settings settings retrieved 
* @param ErrorStr string representing the error condition
*/
DECLARE_DELEGATE_FiveParams(FOnSettingsOperationComplete, const FUniqueNetId& /*UserId*/, bool /*bWasSuccessful*/, bool /*bWasUpdate*/, const FFriendSettings& /*Settings*/, const FString& /*ErrorStr*/);

/**
* Delegate used when the friends settings are updated
*
* @param UserId id of the local user that had their settings updated
* @param bWasSuccessful true if the async action completed without error, false if there was an error
* @param bWasUpdate true if the operation was an "update" operation (vs a passive read)
* @param Settings settings updated
* @param ErrorStr string representing the error condition
*/
DECLARE_MULTICAST_DELEGATE_FiveParams(FOnFriendSettingsUpdated, const FUniqueNetId& /*UserId*/, bool /*bWasSuccessful*/, bool /*bWasUpdate*/, const FFriendSettings& /*Settings*/, const FString& /*ErrorStr*/);
typedef FOnFriendSettingsUpdated::FDelegate FOnFriendSettingsUpdatedDelegate;

/**
 * Interface definition for the online services friends services 
 * Friends services are anything related to the maintenance of friends and friends lists
 */
class IOnlineFriends
{
protected:
	IOnlineFriends() {};

public:
	virtual ~IOnlineFriends() = default;

	/**
     * Delegate used in friends list change notifications
     */
	DEFINE_ONLINE_PLAYER_DELEGATE(MAX_LOCAL_PLAYERS, OnFriendsChange);

	/**
	 * Delegate used in outgoing invite list change notifications
	 */
	DEFINE_ONLINE_PLAYER_DELEGATE(MAX_LOCAL_PLAYERS, OnOutgoingInviteSent);

	/**
	 * Delegate called when remote friend sends an invite
	 *
	 * @param UserId id of the local user that received the invite
	 * @param FriendId remote friend id that sent the invite
	 */
	 DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnInviteReceived, const FUniqueNetId& /*UserId*/, const FUniqueNetId& /*FriendId*/);

	/**
	 * Delegate called when a remote friend accepts an invite
	 *
	 * @param UserId id of the local user that had sent the invite
	 * @param FriendId friend id that accepted the invite
	 */
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnInviteAccepted, const FUniqueNetId& /*UserId*/, const FUniqueNetId& /*FriendId*/);

	/**
	 * Delegate called when a remote friend rejects an invite
	 *
	 * @param UserId id of the local user that had sent the invite
	 * @param FriendId friend id that rejected the invite
	 */
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnInviteRejected, const FUniqueNetId& /*UserId*/, const FUniqueNetId& /*FriendId*/);

	/**
	* Delegate called when a remote friend cancels/aborts an sent invite
	*
	* @param UserId id of the local user that had received the invite
	* @param FriendId friend id that canceled that invite
	*/
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnInviteAborted, const FUniqueNetId& /*UserId*/, const FUniqueNetId& /*FriendId*/);

	/**
	 * Delegate called when a remote friend removes user from friends list
	 *
	 * @param UserId id of the local user that had the friendship
	 * @param FriendId friend id that removed themself from the friendship
	 */
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnFriendRemoved, const FUniqueNetId& /*UserId*/, const FUniqueNetId& /*FriendId*/);

	/**
	 * Starts an async task that reads the named friends list for the player 
	 *
	 * @param LocalUserNum the user to read the friends list of
	 * @param ListName name of the friends list to read
	 *
	 * @return true if the read request was started successfully, false otherwise
	 */
	virtual bool ReadFriendsList(int32 LocalUserNum, const FString& ListName, const FOnReadFriendsListComplete& Delegate = FOnReadFriendsListComplete()) = 0;

	/**
	 * Starts an async task that deletes the named friends list for the player 
	 *
	 * @param LocalUserNum the user to delete the friends list for
	 * @param ListName name of the friends list to delete
	 *
	 * @return true if the delete request was started successfully, false otherwise
	 */
	virtual bool DeleteFriendsList(int32 LocalUserNum, const FString& ListName, const FOnDeleteFriendsListComplete& Delegate = FOnDeleteFriendsListComplete()) = 0;

	/**
	 * Starts an async task that sends an invite to another player. 
	 *
	 * @param LocalUserNum the user that is sending the invite
	 * @param FriendId player that is receiving the invite
	 * @param ListName name of the friends list to invite to
	 *
	 * @return true if the request was started successfully, false otherwise
	 */
	virtual bool SendInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnSendInviteComplete& Delegate = FOnSendInviteComplete()) = 0;

	/**
	 * Starts an async task that accepts an invite from another player. 
	 *
	 * @param LocalUserNum the user that is accepting the invite
	 * @param FriendId player that had sent the pending invite
	 * @param ListName name of the friends list to operate on
	 *
	 * @return true if the request was started successfully, false otherwise
	 */
	virtual bool AcceptInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnAcceptInviteComplete& Delegate = FOnAcceptInviteComplete()) = 0;

	/**
	 * Starts an async task that rejects an invite from another player. 
	 *
	 * @param LocalUserNum the user that is rejecting the invite
	 * @param FriendId player that had sent the pending invite
	 * @param ListName name of the friends list to operate on
	 *
	 * @return true if the request was started successfully, false otherwise
	 */
 	virtual bool RejectInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName) = 0;

	/**
	 * Starts an async task that sets an alias a to friend.
	 *
	 * @param LocalUserNum the user that is setting the alias
	 * @param FriendId player that will have the alias associated
	 * @param ListName name of the friends list to operate on
	 * @param Alias the alias associated to the player
	 *
	 */
	virtual void SetFriendAlias(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FString& Alias, const FOnSetFriendAliasComplete& Delegate = FOnSetFriendAliasComplete()) = 0;

	/**
	 * Starts an async task that delete an alias from a friend.
	 *
	 * @param LocalUserNum the user that is setting the alias
	 * @param FriendId player that will have the alias deleted from
	 * @param ListName name of the friends list to operate on
	 *
	 */
	virtual void DeleteFriendAlias(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnDeleteFriendAliasComplete& Delegate = FOnDeleteFriendAliasComplete()) = 0;

	/**
	 * Delegate used when an invite reject request has completed
	 *
	 * @param LocalUserNum the controller number of the associated user that made the request
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 * @param FriendId player that invited us
	 * @param ListName name of the friends list that was operated on
	 * @param ErrorStr string representing the error condition
	 */
	DEFINE_ONLINE_PLAYER_DELEGATE_FOUR_PARAM(MAX_LOCAL_PLAYERS, OnRejectInviteComplete, bool /*bWasSuccessful*/, const FUniqueNetId& /*FriendId*/, const FString& /*ListName*/, const FString& /*ErrorStr*/);

	/**
	 * Starts an async task that deletes a friend from the named friends list
	 *
	 * @param LocalUserNum the user that is making the request
	 * @param FriendId player that will be deleted
	 * @param ListName name of the friends list to operate on
	 *
	 * @return true if the request was started successfully, false otherwise
	 */
 	virtual bool DeleteFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName) = 0;

	/**
	 * Delegate used when an friend delete request has completed
	 *
	 * @param LocalUserNum the controller number of the associated user that made the request
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 * @param FriendId player that was deleted
	 * @param ListName name of the friends list that was operated on
	 * @param ErrorStr string representing the error condition
	 */
	DEFINE_ONLINE_PLAYER_DELEGATE_FOUR_PARAM(MAX_LOCAL_PLAYERS, OnDeleteFriendComplete, bool /*bWasSuccessful*/, const FUniqueNetId& /*FriendId*/, const FString& /*ListName*/, const FString& /*ErrorStr*/);

	/**
	 * Delegate used when a block player request has completed
	 *
	 * @param LocalUserNum the controller number of the associated user that made the request
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 * @param UniqueId player that was blocked
	 * @param ListName name of the friends list that was operated on
	 * @param ErrorStr string representing the error condition
	 */
	DEFINE_ONLINE_PLAYER_DELEGATE_FOUR_PARAM(MAX_LOCAL_PLAYERS, OnBlockedPlayerComplete, bool /*bWasSuccessful*/, const FUniqueNetId& /*UniqueId*/, const FString& /*ListName*/, const FString& /*ErrorStr*/);

	/**
	 * Delegate used when an unblock player request has completed
	 *
	 * @param LocalUserNum the controller number of the associated user that made the request
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 * @param UniqueId player that was unblocked
	 * @param ListName name of the friends list that was operated on
	 * @param ErrorStr string representing the error condition
	 */
	DEFINE_ONLINE_PLAYER_DELEGATE_FOUR_PARAM(MAX_LOCAL_PLAYERS, OnUnblockedPlayerComplete, bool /*bWasSuccessful*/, const FUniqueNetId& /*UniqueId*/, const FString& /*ListName*/, const FString& /*ErrorStr*/);

	/**
     * Delegate used in block list change notifications
	 *
	 * @param ListName name of friends list that was operated on
     */
	DEFINE_ONLINE_PLAYER_DELEGATE_ONE_PARAM(MAX_LOCAL_PLAYERS, OnBlockListChange, const FString& /*ListName*/);

	/**
	 * Copies the list of friends for the player previously retrieved from the online service
	 *
	 * @param LocalUserNum the user to read the friends list of
	 * @param ListName name of the friends list to read
	 * @param OutFriends [out] array that receives the copied data
	 *
	 * @return true if friends list was found
	 */
	virtual bool GetFriendsList(int32 LocalUserNum, const FString& ListName, TArray< TSharedRef<FOnlineFriend> >& OutFriends) = 0;

	/**
	 * Get the cached friend entry if found
	 *
	 * @param LocalUserNum the user to read the friends list of
	 * @param ListName name of the friends list to read
	 * @param OutFriends [out] array that receives the copied data
	 *
	 * @return null ptr if not found
	 */
	virtual TSharedPtr<FOnlineFriend> GetFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName) = 0;
 	
	/**
	 * Checks that a unique player id is part of the specified user's friends list
	 *
	 * @param LocalUserNum the controller number of the associated user that made the request
	 * @param FriendId the id of the player being checked for friendship
	 * @param ListName name of the friends list to read
	 *
	 * @return true if friends list was found and the friend was valid
	 */
	virtual bool IsFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName) = 0;

	/**
	 * Add a list of players to a user's recent players list
	 *
	 * @param UserId user to query recent players for
	 * @param InRecentPlayers list of players to report
	 * @param Namespace the recent players namespace to list players under
	 * @param InCompletionDelegate delegate to fire on completion
	 */
	virtual void AddRecentPlayers(const FUniqueNetId& UserId, const TArray<FReportPlayedWithUser>& InRecentPlayers, const FString& ListName, const FOnAddRecentPlayersComplete& InCompletionDelegate) { check(false); }

	/**
	 * Query for recent players of the current user
	 *
	 * @param UserId user to query recent players for
	 * @param Namespace the recent players namespace to retrieve
	 *
	 * @return true if query was started
	 */
	virtual bool QueryRecentPlayers(const FUniqueNetId& UserId, const FString& Namespace) = 0;

	/**
	 * Delegate used when the query for recent players has completed
	 *
	 * @param UserId the id of the user that made the request
	 * @param Namespace the recent players namespace 
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 * @param Error string representing the error condition
	 */
	DEFINE_ONLINE_DELEGATE_FOUR_PARAM(OnQueryRecentPlayersComplete, const FUniqueNetId& /*UserId*/, const FString& /*Namespace*/, bool /*bWasSuccessful*/, const FString& /*Error*/);

	/**
	 * Delegate used when the query for blocked players has completed
	 *
	 * @param UserId the id of the user that made the request
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 * @param Error string representing the error condition
	 */
	DEFINE_ONLINE_DELEGATE_THREE_PARAM(OnQueryBlockedPlayersComplete, const FUniqueNetId& /*UserId*/, bool /*bWasSuccessful*/, const FString& /*Error*/);

	/**
	 * Copies the cached list of recent players for a given user
	 *
	 * @param UserId user to retrieve recent players for
	 * @param Namespace the recent players namespace to retrieve (if empty retrieve all namespaces)
	 * @param OutRecentPlayers [out] array that receives the copied data
	 *
	 * @return true if recent players list was found for the given user
	 */
	virtual bool GetRecentPlayers(const FUniqueNetId& UserId, const FString& Namespace, TArray< TSharedRef<FOnlineRecentPlayer> >& OutRecentPlayers) = 0;

	/**
	 * Dump state information about blocked players
	 */
	virtual void DumpRecentPlayers() const = 0;
	
	/**
	 * Delegate used when recent players have been added
	 *
	 * @param UserId the id of the user recieving new recent players
	 * @param AddedPlayers players that were added to the recent players list
	 */
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnRecentPlayersAdded, const FUniqueNetId& /*UserId*/, const TArray<TSharedRef<FOnlineRecentPlayer>>& /*AddedPlayers*/);

	/**
	 * Block a player
	 *
	 * @param LocalUserNum The user to check for
	 * @param PlayerId The player to block
	 *
	 * @return true if query was started
	 */
	virtual bool BlockPlayer(int32 LocalUserNum, const FUniqueNetId& PlayerId) = 0;

	/**
	 * Unblock a player
	 *
	 * @param LocalUserNum The user to check for
	 * @param PlayerId The player to unblock
	 *
	 * @return true if query was started
	 */
	virtual bool UnblockPlayer(int32 LocalUserNum, const FUniqueNetId& PlayerId) = 0;

	/**
	 * Query for blocked players
	 *
	 * @param UserId user to query blocked players for
	 *
	 * @return true if query was started
	 */
	virtual bool QueryBlockedPlayers(const FUniqueNetId& UserId) = 0;

	/**
	 * Get the list of blocked players
	 *
	 * @param UserId user to retrieve blocked players for
	 * @param OuBlockedPlayers [out] array that receives the copied data
	 *
	 * @return true if blocked players list was found for the given user
	 */
	virtual bool GetBlockedPlayers(const FUniqueNetId& UserId, TArray< TSharedRef<FOnlineBlockedPlayer> >& OutBlockedPlayers) = 0;

	/**
	 * Dump state information about blocked players
	 */
	virtual void DumpBlockedPlayers() const = 0;

	/**
	* Delegate used when the friends settings are updated
	*
	* @param UserId id of the local user that had their settings updated
	* @param bWasSuccessful true if the async action completed without error, false if there was an error
	* @param bWasUpdate true if the operation was an "update" operation (vs a passive read)
	* @param Settings settings updated
	* @param ErrorStr string representing the error condition
	*/
	DEFINE_ONLINE_DELEGATE_FIVE_PARAM(OnFriendSettingsUpdated, const FUniqueNetId& /*UserId*/, bool /*bWasSuccessful*/, bool /*bWasUpdate*/, const FFriendSettings& /*Settings*/, const FString& /*ErrorStr*/);

	/**
	 * Query the current friend settings
	 *
	 * @param LocalUserId user to retrieve friend settings for
	 * @param Delegate Delegate to call when operation has been completed
	 *
	 */
	virtual void QueryFriendSettings(const FUniqueNetId& LocalUserId, FOnSettingsOperationComplete Delegate) { check(false) }

	/**
	 * Update the current friend settings
	 *
	 * @param LocalUserId user to retrieve friend settings for
	 * @param FriendInvitePolicy invite policy to change designated user to
	 * @param bAffectsExistingInvites boolean to determine if existing friend invites are affected by this change
	 * @param Delegate Delegate to call when operation has been completed
	 *
	 */
	virtual void UpdateFriendInvitePolicySettings(const FUniqueNetId& LocalUserId, const EFriendInvitePolicy FriendInvitePolicy, const bool bAffectsExistingInvites, FOnSettingsOperationComplete Delegate) { check(false) }

	/**
	 * Queries the settings we have stored for a third party source
	 *
	 * @param UserId user to retrieve settings for
	 * @param The source the settings relate to i.e. steam
	 *
	 * @return true if query was started
	 */
	virtual bool QueryFriendSettings(const FUniqueNetId& UserId, const FString& Source, const FOnQueryFriendSettingsComplete& Delegate = FOnQueryFriendSettingsComplete()) { check(false); return false; }

	/**
	 * Gets the cached information we have stored from a third party source
	 *
	 * @param UserId user to retrieve settings for
	 * @param OutSettings [out] Map that receives the copied data
	 *
	 * @return true if blocked players list was found for the given user
	 */
	virtual bool GetFriendSettings(const FUniqueNetId& UserId, TMap<FString, TSharedRef<FOnlineFriendSettingsSourceData> >& OutSettings) { check(false); return false; }

	/**
	 * Gets the FriendInvitePolicy info that defines who can send friend invites to the user. (Public = Anyone, Friends_of_Friends = Only mutual friends, Private = No one)
	 *
	 * @param LocalUserId user to retrieve friendInvitePolicy settings for
	 *
	 * @return the permission setting from the UserState
	 */
	virtual EFriendInvitePolicy GetFriendInvitePolicy(const FUniqueNetId& UserId) const  { return EFriendInvitePolicy::InvalidOrMax; }

	/**
	 * Set information we want to store for a third party source
	 *
	 * @param UserId user to retrieve settings for
	 * @param The source the settings relate to i.e. steam
	 * @param bNeverShowAgain A bool for if we should stop showing call outs for this source, (currently the only setting we store)
	 *
	 * @return true if query was started
	 */
	virtual bool SetFriendSettings(const FUniqueNetId& UserId, const FString& Source, bool bNeverShowAgain, const FOnSetFriendSettingsComplete& Delegate = FOnSetFriendSettingsComplete()) { check(false); return false; }

};

typedef TSharedPtr<IOnlineFriends, ESPMode::ThreadSafe> IOnlineFriendsPtr;
