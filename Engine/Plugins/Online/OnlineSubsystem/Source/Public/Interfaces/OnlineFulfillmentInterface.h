// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlinePurchaseInterface.h"

/**
 * IOnlineFulfillment - Interface for redeeming codes.
 */
class IOnlineFulfillment
{
public:
	virtual ~IOnlineFulfillment() = default;

	/**
	 * Initiate the checkout process for obtaining offers via code redemption
	 *
	 * @param UserId user initiating the request
	 * @param RedeemCodeRequest info needed for the redeem request
	 * @param Delegate completion callback (guaranteed to be called)
	 */
	virtual void RedeemCode(const FUniqueNetId& UserId, const FRedeemCodeRequest& RedeemCodeRequest, const FOnPurchaseRedeemCodeComplete& Delegate) = 0;
};
