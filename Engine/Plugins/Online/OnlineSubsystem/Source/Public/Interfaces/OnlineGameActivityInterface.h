// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector.h"
#include "OnlineDelegateMacros.h"
#include "Online/CoreOnlineFwd.h"

class FOnlineEventParms;
template <typename OptionalType> struct TOptional;

class FUniqueNetId;
struct FOnlineError;
class FOnlineSessionSearchResult;

/**
 * Players location information for the activity service
 */
struct FOnlineActivityPlayerLocation
{
	/** Current reported zone*/
	FString ZoneId;
	/** Coordinates */
	FVector Coordinates;

	/**
	 * Equals operator for comparing FOnlineActivityPlayerLocation objects
	 */
	friend inline bool operator==(const FOnlineActivityPlayerLocation& Location1, const FOnlineActivityPlayerLocation& Location2)
	{
		return Location1.ZoneId == Location2.ZoneId && Location1.Coordinates == Location2.Coordinates;
	}
};

/**
 * Outcome representation of ending an activity.
 */
enum class EOnlineActivityOutcome
{
	/** Activity has been completed successfully */
	Completed,
	/** Activity attempt failed to be completed */
	Failed,
	/** The activity was cancelled */
	Cancelled
};

/** 
 * Task ids for in progress and completed activity tasks to be reset
 */
struct FOnlineActivityTasksToReset
{
	TArray<FString> InProgressTasks;
	TArray<FString> CompletedTasks;
};

/**
 * Information about a game activity
 */
struct FOnlineActivityInformation
{
	/** Activity Id */
	FString ActivityId;
	/** Activity Name */
	FString ActivityName;
	/** Is the activity hidden */
	bool bIsGameActivityHidden = false;
	/** game match id */
	FString GameMatchId;
};

/**
 * Requested friend's activity information
 */
struct FUsersOnlineActivities
{
	/** The requesting account id */
	FString AccountId;
	/** Activity information */
	TArray<FOnlineActivityInformation> Activities;
};

/** 
 * Multicast delegate fired when an activity request has happened
 *
 * @param LocalUserId the id of the player this callback is for
 * @param ActivityId the id of the activity for activation
 * @param SessionInfo the session search results for the the activity
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnGameActivityActivationRequested, const FUniqueNetId& /* LocalUserId */, const FString& /* ActivityId */, const FOnlineSessionSearchResult* /* SessionInfo */ );
typedef FOnGameActivityActivationRequested::FDelegate FOnGameActivityActivationRequestedDelegate;

/**
 * Delegate fired when a start activity call has completed
 *
 * @param LocalUserId the id of the player this callback is for
 * @param ActivityId the id of the activity that was started
 * @param Status of whether async action completed successfully or with error
 */
DECLARE_DELEGATE_ThreeParams(FOnStartActivityComplete, const FUniqueNetId& /* LocalUserId */, const FString& /* ActivityId */, const FOnlineError& /* Status */);

/**
 * Delegate fired when an end activity call has completed
 *
 * @param LocalUserId the id of the player this callback is for
 * @param ActivityId the id of the activity that was ended
 * @param Outcome the outcome of the activity
 * @param Status of whether async action completed successfully or with error
 */
DECLARE_DELEGATE_FourParams(FOnEndActivityComplete, const FUniqueNetId& /* LocalUserId */, const FString& /* ActivityId */, const EOnlineActivityOutcome& /* Outcome */, const FOnlineError& /* Status */);

/**
 * Delegate fired when the reset all active activities call has completed
 *
 * @param LocalUserId the id of the player this callback is for
 * @param Status of whether async action completed successfully or with error
 */
DECLARE_DELEGATE_TwoParams(FOnResetAllActiveActivitiesComplete, const FUniqueNetId& /* LocalUserId */, const FOnlineError& /* Status */);

/**
 * Delegate fired when the resume activity call has completed
 *
 * @param LocalUserId the id of the player this callback is for
 * @param ActivityId the id of the activity that was ended
 * @param Status of whether async action completed successfully or with error
 */
DECLARE_DELEGATE_ThreeParams(FOnResumeActivityComplete, const FUniqueNetId& /* LocalUserId */, const FString& /* ActivityId */, const FOnlineError& /* Status */);

/**
 * Delegate fired when a set activity availability call has completed
 *
 * @param LocalUserId the id of the player this callback is for
 * @param Status of whether async action completed successfully or with error
 */
DECLARE_DELEGATE_TwoParams(FOnSetActivityAvailabilityComplete, const FUniqueNetId& /* LocalUserId */, const FOnlineError& /* Status*/);

/**
 * Delegate fired when a set activity priority call has completed
 *
 * @param LocalUserId the id of the player this callback is for
 * @param Status of whether async action completed successfully or with error
 */
DECLARE_DELEGATE_TwoParams(FOnSetActivityPriorityComplete, const FUniqueNetId& /* LocalUserId */, const FOnlineError& /* Status */);

/**
 * Delegate fired when querying remote user activities call has completed
 *
 * @param LocalUserId the id of the player this callback is for
 * @param UserActivities returned activity array for requested users
 * @param Status of whether async action completed successfully or with error
 */
DECLARE_DELEGATE_ThreeParams(FOnQueryUserActivitiesComplete, const FUniqueNetId& /* LocalUserId */, const TArray<FUsersOnlineActivities>& /* UserActivities */, const FOnlineError& /* Status */);

/**
 *	IOnlineGameActivity - Interface class for managing a user's activity state
 */
class IOnlineGameActivity
{
public:
	virtual ~IOnlineGameActivity() = default;

	/**
	 * Start an activity
	 *
	 * @param LocalUserId - Id of the user starting the activity
	 * @param ActivityId - Id of the activity to start
	 * @param Parms - Additional data to include with the start activity request
	 * @param CompletionDelegate - Completion delegate called when StartActivity is complete
	 */
	virtual void StartActivity(const FUniqueNetId& LocalUserId, const FString& ActivityId, const FOnlineEventParms& Parms, FOnStartActivityComplete CompletionDelegate) = 0;

	/**
	 * End an activity 
	 *
	 * @param LocalUserId - Id of the player stopping the activity
	 * @param ActivityId - Task to end by activity ID
	 * @param ActivityOutcome - The outcome of the activity (completed, failed, or abandoned)
	 * @param Parms - Additional data to include with the stop activity request
	 * @param CompletionDelegate - Completion delegate called when the EndActivity call is complete
	 */
	virtual void EndActivity(const FUniqueNetId& LocalUserId, const FString& ActivityId, EOnlineActivityOutcome ActivityOutcome, const FOnlineEventParms& Parms, FOnEndActivityComplete CompletionDelegate) = 0;

	/** 
	 * resets all in-progress and completed activities.
	 *
	 * @param LocalUserId - Id of the player resetting their active activities
	 * @param CompletionDelegate - Completion delegate called when ResetAllActiveActivities call is complete
	 */
	virtual void ResetAllActiveActivities(const FUniqueNetId& LocalUserId, const FOnResetAllActiveActivitiesComplete& CompletionDelegate) = 0;

	/**
	 * Resume an activity.  Different from StartActivity in that resume continues from current 
	 * progress where StartActivity will set progress back to 0 before activating the activity
	 *
	 * @param LocalUserId - Id of the player resuming the activity
	 * @param ActivityId - Task to resume by activity ID
	 * @param TasksToReset - Optional argument to specify which in-progress and completed tasks of an activity to reset.  Leaving this empty will reset all tasks
	 * @param CompletionDelegate - Completion delegate called when the ResumeActivity call is complete
	 */
	virtual void ResumeActivity(const FUniqueNetId& LocalUserId, const FString& ActivityId, const TOptional<FOnlineActivityTasksToReset>& TasksToReset, const FOnResumeActivityComplete& CompletionDelegate) = 0;

	/** 
	 * Set an activity's availability 
	 *
	 * @param LocalUserId - Id of user setting the activity availability
	 * @param ActivityId - Id of the activity to set availability on
	 * @param bEnabled - Boolean to set availability of the specified activity
	 * @param CompletionDelegate - Completion delegate called when SetActivityAvailability call is complete
	 */
	virtual void SetActivityAvailability(const FUniqueNetId& LocalUserId, const FString& ActivityId, const bool bEnabled, FOnSetActivityAvailabilityComplete CompletionDelegate) = 0;

	/** 
	 * Set the activity priority 
	 *
	 * @param LocalUserId - Id of user setting the activity priority
	 * @param PrioritizedActivities - Array of activities with their activity id and priority
	 * @param CompletionDelegate - Completion delegate called when SetActivityPriority call is complete
	 */
	virtual void SetActivityPriority(const FUniqueNetId& LocalUserId, const TMap<FString, int32>& PrioritizedActivities, FOnSetActivityPriorityComplete CompletionDelegate) = 0;

	/**
	 * Query user's activities
	 *
	 * @param LocalUserId - Id of user requesting activities of other users
	 * @param FriendAccountIds - Array of friend ids to get activity information on
	 * @param MaxUserActivitiesReturned - Optional maximum number of activities to return per user
	 * @param CompletionDelegate - Completion delegate called when QueryUserActivities call is complete
	 */
	virtual bool QueryActivitiesForUsers(const FUniqueNetId& LocalUserId, const TArray<FUniqueNetIdRef>& UserAccounts, TOptional<int32> MaxUserActivitiesReturned, const FOnQueryUserActivitiesComplete& CompletionDelegate) = 0;

	/**
	 * Called when an activity is requested
	 *
	 * @param LocalUserId the id of the player this callback is for
	 * @param ActivityId the id of the activity for activation
	 * @param SessionInfo the session search results for the the activity
	 */
	DEFINE_ONLINE_DELEGATE_THREE_PARAM(OnGameActivityActivationRequested, const FUniqueNetId& /* LocalUserId */, const FString& /* ActivityId */, const FOnlineSessionSearchResult* /* SessionInfo */);

	/** 
	 *
	 * Update player location 
	 *
	 * Note:  The game is expected to call this function periodically. ZoneId/Coordinates are also used when
	 * starting and stopping an activity
	 *
	 * @param LocalUserId - Id of user calling update on the player location
	 * @param ActivityPlayerLocation - zone id and player map coordinates (X and Y values)
	 */
	virtual void UpdatePlayerLocation(const FUniqueNetId& LocalUserId, TOptional<FOnlineActivityPlayerLocation>& ActivityPlayerLocation) = 0;
};
