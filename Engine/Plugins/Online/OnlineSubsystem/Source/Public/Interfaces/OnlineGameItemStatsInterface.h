// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

struct FOnlineError;

class FUniqueNetId;

/**
 * Delegate fired when a item usage call has completed
 *
 * @param LocalUserId the id of the player this callback is for
 * @param Status of whether async action completed successfully or with error
 */
DECLARE_DELEGATE_TwoParams(FOnItemUsageComplete, const FUniqueNetId& /* LocalUserId */, const FOnlineError& /* Status */);

/**
 * Delegate fired when a item impact call has completed
 *
 * @param LocalUserId the id of the player this callback is for
 * @param Status of whether async action completed successfully or with error
 */
DECLARE_DELEGATE_TwoParams(FOnItemImpactComplete, const FUniqueNetId& /* LocalUserId */, const FOnlineError& /* Status */);

/**
 * Delegate fired when a item mitigation call has completed
 *
 * @param LocalUserId the id of the player this callback is for
 * @param Status of whether async action completed successfully or with error
 */
DECLARE_DELEGATE_TwoParams(FOnItemMitigationComplete, const FUniqueNetId& /* LocalUserId */, const FOnlineError& /* Status */);

/**
 * Delegate fired when a item availability change call has completed
 *
 * @param LocalUserId the id of the player this callback is for
 * @param Status of whether async action completed successfully or with error
 */
DECLARE_DELEGATE_TwoParams(FOnItemAvailabilityChangeComplete, const FUniqueNetId& /* LocalUserId */, const FOnlineError& /* Status */);

/**
 * Delegate fired when a item inventory change call has completed
 *
 * @param LocalUserId the id of the player this callback is for
 * @param Status of whether async action completed successfully or with error
 */
DECLARE_DELEGATE_TwoParams(FOnItemInventoryChangeComplete, const FUniqueNetId& /* LocalUserId */, const FOnlineError& /* Status */);

/**
 * Delegate fired when a item loadout change call has completed
 *
 * @param LocalUserId the id of the player this callback is for
 * @param Status of whether async action completed successfully or with error
 */
DECLARE_DELEGATE_TwoParams(FOnItemLoadoutChangeComplete, const FUniqueNetId& /* LocalUserId */, const FOnlineError& /* Status */);

class IOnlineGameItemStats
{
public:
	virtual ~IOnlineGameItemStats() = default;

	/**
	 * Updates items used and by who
	 *
	 * @param LocalUserId -  Id of the player to report the game stat for
	 * @param ItemUsedBy - Who the item was used by. This can be LocalUserId, another actor, or blank if no actor used the item.
	 * @param ItemsUsed - Array of item IDs that were used
	 * @param CompletionDelegate - Item usage completion delegate called when ItemUsage call is complete
	 */
	virtual void ItemUsage(const FUniqueNetId& LocalUserId, const FString& ItemUsedBy, const TArray<FString>& ItemsUsed, FOnItemUsageComplete CompletionDelegate) = 0;

	/**
	 * Items used to make impact and who the impact target is 
	 *
	 * @param LocalUserId - User Id of the player using the item
	 * @param TargetActors - Who was impacted by the item
	 * @param ImpactInitiatedBy - Who initiated the impact
	 * @param ItemsUsed - A string array of item IDs that were used
	 * @param CompletionDelegate - Item impact completion delegate called when ItemImpact call is complete
	 */
	virtual void ItemImpact(const FUniqueNetId& LocalUserId, const TArray<FString>& TargetActors, const FString& ImpactInitiatedBy, const TArray<FString>& ItemsUsed, FOnItemImpactComplete CompletionDelegate) = 0;

	/**
	 * Items the player effectively used to mitigate an item used by the game on the player 
	 *
	 * @param LocalUserId - User Id of the player using the item
	 * @param ItemsUsed - Who activated the mitigation
	 * @param ImpactItemsMitigated - Array of items that were mitigated
	 * @param ItemUsedBy - Array of item IDs that were used
	 * @param CompletionDelegate - Item mitigation completion delegate called when ItemMitigation call is complete
	 */
	virtual void ItemMitigation(const FUniqueNetId& LocalUserId, const TArray<FString>& ItemsUsed, const TArray<FString>& ImpactItemsMitigated, const FString& ItemUsedBy, FOnItemMitigationComplete CompletionDelegate) = 0;

	/**
	 * Updates the lists of available and unavailable items
	 *
	 * @param LocalUserId - User Id of the player
	 * @param AvailableItems - Array of available items
	 * @param UnavailableItems - Array of unavailable items 
	 * @param CompletionDelegate - Item availability change completion delegate called when ItemAvailabilityChange call is complete
	 */
	virtual void ItemAvailabilityChange(const FUniqueNetId& LocalUserId, const TArray<FString>& AvailableItems, const TArray<FString>& UnavailableItems, FOnItemAvailabilityChangeComplete CompletionDelegate) = 0;

	/**
	 * Updates the lists of available and unavailable items in the player's inventory
	 *
	 * @param LocalUserId - User Id of the player using the item
	 * @param ItemsToAdd - Array of available items
	 * @param ItemsToRemove - Array of unavailable items
	 * @param CompletionDelegate - Item inventory change completion delegate called when ItemInventoryChange call is complete
	 */
	virtual void ItemInventoryChange(const FUniqueNetId& LocalUserId, const TArray<FString>& ItemsToAdd, const TArray<FString>& ItemsToRemove, FOnItemInventoryChangeComplete CompletionDelegate) = 0;

	/**
	 * Updates the loadout of the player's configuration
	 *
	 * @param LocalUserId - User Id of the player using the item
	 * @param EquippedItems - Array of equipped items
	 * @param UnequippedItems - Array of unequipped items
	 * @param CompletionDelegate - Item loadout change completion delegate called when ItemLoadoutChange call is complete
	 */
	virtual void ItemLoadoutChange(const FUniqueNetId& LocalUserId, const TArray<FString>& EquippedItems, const TArray<FString>& UnequippedItems, FOnItemLoadoutChangeComplete CompletionDelegate) = 0;
};
