// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Online/CoreOnlineFwd.h"
#include "OnlineKeyValuePair.h"

struct FOnlineError;

#define GAME_MATCH_TYPE_COOPERATIVE TEXT("Cooperative")
#define GAME_MATCH_TYPE_COMPETITIVE TEXT("Competitive")

/** 
 * Options available for match grouping
 */
enum class EMatchGroupType
{
	/** Default option */
	Invalid,
	/** Team grouping */
	Teams,
	/** Non team grouping (solos) */
	NonTeams
};

/** 
 * Options available for updating game match status
 */
enum class EUpdateGameMatchStatus
{
	/** Default option */
	Invalid,
	/** Used for when the match state should be set to in progress */
	InProgress,
	/** Indicates the match was put paused */
	Paused,
	/** The match was aborted before completion */
	Aborted,
};

/** 
 * Options available when leaving a game match
 */
enum class ELeaveReason
{
	/** Default option */
	Invalid,
	/** Player has disconnected (IE timeout) */
	Disconnect,
	/** The match was completed */
	Finished,
	/** The player quit before the match was completed */
	Quit
};

/**
 * Data required to leave a game match
 */
struct FLeaveGameMatchPlayer
{
	/** Id of the player leaving the match */
	FUniqueNetIdPtr PlayerId;
	/** Reason for leaving the match */
	ELeaveReason LeaveReason = ELeaveReason::Invalid;
};

/** 
 * Player information needed to join a game match
 */
struct FJoinGameMatchPlayer
{
	/** Player account id*/
	FUniqueNetIdPtr PlayerId;
	/** Player name */
	FString PlayerName;
	/** The id of the team this player will belong to */
	FString TeamId;
};

/** 
 * Player information needed when creating a new game match
 */
struct FGameMatchPlayer
{
	/** Player account id*/
	FUniqueNetIdPtr PlayerId;
	/** Player name */
	FString PlayerName;
	/** Team name this player belongs to */
	FString TeamName;
	/** Should the player join the match on creation */
	bool bJoinMatch = false;
	/** Is the player an NPC? */
	bool bIsNpc = false;
};

/** 
 * Data needed to construct a team
 */
struct FGameMatchTeam
{
	/** Name of the team */
	FString TeamName;
	/** Id of the team */
	FString TeamId;
	/** Members belonging to the team */
	TArray<FUniqueNetIdRef> TeamMemberIds;
};

/** 
 * Player game match results
 */
struct FGameMatchPlayerResult
{
	/** Id of the player */
	FUniqueNetIdPtr PlayerId;
	/** Player's rank */
	int32 Rank = -1;
	/** Player's score */
	float Score = 0.0f;
};

/** 
 * Game match results for the team
 */
struct FGameMatchTeamResult
{
	/** Team id */
	FString TeamId;
	/** Team's rank */
	int32 Rank = -1;
	/** Team's score */
	double Score = 0.0;
	/** The members on the team */
	TArray<FGameMatchPlayerResult> MembersResult;
};

/** 
 * Coop game match results 
 */
enum class EMatchCoopResults
{
	/** Default option */
	Invalid,
	/** The players completed the match */
	Complete,
	/** The match has not yet been completed */
	NotComplete,
	/** The players did not successfully complete the match */
	Failed
};

/** 
 * Results of the competitive game match
 */
struct FGameMatchCompetitiveResults
{
	/** Per player results */
	TArray<FGameMatchPlayerResult> PlayersResult;
	/** Per team results */
	TArray<FGameMatchTeamResult> TeamsResult;
};

/** 
 * Overall game match results container to house coop and competitive game match data
 */
struct FGameMatchResult
{
	/** Results of a coop game match */
	EMatchCoopResults CoopResult = EMatchCoopResults::Invalid;
	/** Results of a competitive game match */
	FGameMatchCompetitiveResults CompetitiveResult;
};

/** 
 * Stats container 
  */
struct FGameMatchStatsData
{
	/** Key */
	FString StatsKey;
	/** Value */
	FVariantData StatsValue;
};

/** 
 * Per player stats
 */
struct FGameMatchPlayerStats
{
	/** Player id */
	FUniqueNetIdPtr PlayerId;
	/** The player's stats */
	TArray<FGameMatchStatsData> PlayerStats;
};

/** 
 * Complete team stats with players
 */
struct FGameMatchTeamStats
{
	/** Team id*/
	FString TeamId;
	/** Team stats */
	TArray<FGameMatchStatsData> TeamStats;
	/** Team member stats */
	TArray<FGameMatchPlayerStats> TeamMemberStats;
};

/** 
 * Complete team and player game match stats
 */
struct FGameMatchStats
{
	/** Per player game match stats */
	TArray<FGameMatchPlayerStats> PlayerStats;
	/** Per team game match stats */
	TArray<FGameMatchTeamStats> TeamStats;
};

/** 
 * Game match roster 
 */
struct FGameMatchRoster
{
	/** Array of players */
	TArray<FGameMatchPlayer> Players;
	/** Array of teams */
	TArray<FGameMatchTeam> Teams;
	/** End game match results */
	FGameMatchResult Results;
	/** Game match stats for players and teams*/
	FGameMatchStats Stats;
};

/** 
 * Top most container that contains all game match data
 */
struct FGameMatchesData
{
	/** Game activity id*/
	FString ActivityId;
	/** zone id*/
	FString ZoneId;
	/** The amount of time of no updates before the game match is automatically expired */
	TOptional<int32> InactivityExpirationTimeSeconds;

	/** The roster of the match */
	FGameMatchRoster MatchesRoster;
};

/** 
 * Final match report container.  Holds all information to close out the game match
 */
struct FFinalGameMatchReport
{
	/** Group type */
	EMatchGroupType GroupType = EMatchGroupType::Invalid;
	/** Match type */
	FString MatchType;
	/** Can leave game feedback? */
	bool bLeaveGameFeedback = false;
	/** Results of the game match */
	FGameMatchResult Results;
	/** Stats of the game match */
	FGameMatchStats Stats;
};

/**
 * Delegate fired when the CreateGameMatch call has completed
 *
 * @param LocalUserId the id of the player this callback is for
 * @param MatchId the id of the match that was created
 * @param Result of whether async action completed successfully or with error
 */
DECLARE_DELEGATE_ThreeParams(FOnCreateGameMatchComplete, const FUniqueNetId& /* LocalUserId */, const FString& /* MatchId */, const FOnlineError& /* Result */);

/**
 * Delegate fired when the JoinGameMatch call has completed
 *
 * @param LocalUserId the id of the player this callback is for
 * @param Result of whether async action completed successfully or with error
 */
DECLARE_DELEGATE_TwoParams(FOnJoinGameMatchComplete, const FUniqueNetId& /* LocalUserId*/, const FOnlineError& /* Result */);

/**
 * Delegate fired when the LeaveGameMatch match call has completed
 *
 * @param LocalUserId the id of the player this callback is for
 * @param PlayerNames the player names that left the match
 * @param Result of whether async action completed successfully or with error
 */
DECLARE_DELEGATE_ThreeParams(FOnLeaveGameMatchComplete, const FUniqueNetId& /* LocalUserId */, const TArray<FString>& /* PlayerNames */, const FOnlineError& /* Result */);

/**
 * Delegate fired when the UpdateGameMatchStatus call has completed
 *
 * @param LocalUserId the id of the player this callback is for
 * @param Status the new status set when the operation is successful
 * @param Result of whether async action completed successfully or with error
 */
DECLARE_DELEGATE_ThreeParams(FOnGameMatchStatusUpdateComplete, const FUniqueNetId& /* LocalUserId */, const EUpdateGameMatchStatus& /* Status */, const FOnlineError& /* Result */);

/**
 * Delegate fired when the ReportGameMatchResults call has completed
 *
 * @param LocalUserId the id of the player this callback is for
 * @param Result of whether async action completed successfully or with error
 */
DECLARE_DELEGATE_TwoParams(FOnGameMatchReportComplete, const FUniqueNetId& /* LocalUserId */, const FOnlineError& /* Result */);

/**
 * Delegate fired when the UpdateGameMatchStatus request has completed
 *
 * @param LocalUserId the id of the player this callback is for
 * @param Result of whether async action completed successfully or with error
 */
DECLARE_DELEGATE_TwoParams(FOnUpdateGameMatchDetailsComplete, const FUniqueNetId& /* LocalUserId */, const FOnlineError& /* Result  */);

/**
 * Delegate fired when the game match feedback call has completed
 *
 * @param LocalUserId the id of the player this callback is for
 * @param Result of whether async action completed successfully or with error
 */
DECLARE_DELEGATE_TwoParams(FOnGameMatchFeedbackComplete, const FUniqueNetId& /* LocalUserId */, const FOnlineError& /* Result */);

/**
 *	IOnlineGameMatches - Interface class for managing a user's game matches
 */
class IOnlineGameMatches
{
public:
	virtual ~IOnlineGameMatches() = default;

	/**
	 * Create a game match
	 *
	 * @param UserId - Id of the user creating the match
	 * @param MatchesData - Game match data required to start a new match
	 * @param CompletionDelegate - Completion delegate called when CreateGameMatch is complete
	 */
	virtual void CreateGameMatch(const FUniqueNetId& UserId, const FGameMatchesData& MatchesData, const FOnCreateGameMatchComplete& CompletionDelegate) = 0;

	/**
	 * Joins a player to an existing match
	 *
	 * @param UserId - Id of the user updating the match (not to be confused with the id of the player being added to the match)
	 * @param MatchId - The id of the match in which the new player will join
	 * @param GameMatchJoinPlayerData - Game match data required to add a player to the existing match
	 * @param CompletionDelegate - Completion delegate called when JoinGameMatch is complete
	 */
	virtual void JoinGameMatch(const FUniqueNetId& UserId, const FString& MatchId, const TArray<FJoinGameMatchPlayer>& GameMatchJoinPlayerData, const FOnJoinGameMatchComplete& CompletionDelegate) = 0;

	/**
	 * Removes a player from an existing match
	 *
	 * @param UserId - Id of the user updating the match (not to be confused with the id of the player being removed from the match)
	 * @param MatchId - The id of the match in which the player will be removed from
	 * @param GameMatchLeavePlayerData - Game match data required to remove a player from the existing match
	 * @param CompletionDelegate - Completion delegate called when LeaveGameMatch is complete
	 */
	virtual void LeaveGameMatch(const FUniqueNetId& UserId, const FString& MatchId, const TArray<FLeaveGameMatchPlayer>& GameMatchLeavePlayerData, const FOnLeaveGameMatchComplete& CompletionDelegate) = 0;

	/**
	 * Updates the status of a created match
	 *
	 * @param UserId - Id of the user updating the match state
	 * @param MatchId - The id of the match to update
	 * @param Status - The new status to update the match to
	 * @param CompletionDelegate - Completion delegate called when UpdateGameMatchStatus is complete
	 */
	virtual void UpdateGameMatchStatus(const FUniqueNetId& UserId, const FString& MatchId, const EUpdateGameMatchStatus& Status, const FOnGameMatchStatusUpdateComplete& CompletionDelegate) = 0;

	/**
	 * Updates game match information while match is still in progress
	 *
	 * @param UserId - Id of the user updating the match information
	 * @param MatchId - The id of the match to be updated
	 * @param MatchRoster - Information on players and teams that have changed during the match (such as scores, etc)
	 * @param CompletionDelegate - Completion delegate called when UpdateGameMatchDetails is complete
	 */
	virtual void UpdateGameMatchDetails(const FUniqueNetId& UserId, const FString& MatchId, const FGameMatchRoster& MatchRoster, const FOnUpdateGameMatchDetailsComplete& CompletionDelegate) = 0;

	/**
	 * Sends final data of the match and closes the match out
	 *
	 * @param UserId - Id of the user closing out the match
	 * @param MatchId - The id of the match to close
	 * @param FinalReport - Final data to send with the closing of the match (includes team and player data)
	 * @param CompletionDelegate - Completion delegate called when ReportGameMatchResults is complete
	 */
	virtual void ReportGameMatchResults(const FUniqueNetId& UserId, const FString& MatchId, const FFinalGameMatchReport& FinalReport, const FOnGameMatchReportComplete& CompletionDelegate) = 0;
};
