// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/DateTime.h"
#include "OnlineSubsystemTypes.h"

class Error;

/**
 * Data about the group that is used for display
 */
struct FGroupDisplayInfo
{
	FGroupDisplayInfo()
	{
	}

	FGroupDisplayInfo(const FGroupDisplayInfo& Other)
		: Name(Other.Name)
		, Description(Other.Description)
		, Motto(Other.Motto)
		, Language(Other.Language)
		, InviteOnly(Other.InviteOnly)
	{
	}

	/** The human readable name of this guild */
	FText Name;

	/** User entered guild description text */
	FText Description;

	/** A one-line motto or catch phrase for the group */
	FText Motto;

	/** The main language of the team */
	FString Language;

	/** Is the group PUBLIC (anyone can join) or not? */
	bool InviteOnly;
};

/**
 * This struct describes metadata about a group.
 */
class IGroupInfo
{

public:
	/** Get the group */
	virtual FUniqueNetIdRef GetGroupId() const = 0;

	/** Arbitrary namespace string used to filter group groups in some queries or client side. Usually this would be the game codename */
	virtual const FString& GetNamespace() const = 0;

	/** All fields in this struct are group admin/owner entered */
	virtual const FGroupDisplayInfo& GetDisplayInfo() const = 0;

	/** GUID of the user account that holds the owner role for this group (will only be one) */
	virtual FUniqueNetIdRef GetOwner() const = 0;

	/** The current size of the group */
	virtual uint32 GetSize() const = 0;

	/** When was this group created */
	virtual const FDateTime& GetCreatedAt() const = 0;

	/** When was this group last updated (according to the server) */
	virtual const FDateTime& GetLastUpdated() const = 0;
};

/**
 * An entry in a group member list.
 */
struct FGroupMember
{
	FUniqueNetIdPtr GetId() const { return AccountId; }

	FUniqueNetIdPtr AccountId;
	bool bAdmin;
	bool bIsOwner;
	FDateTime JoinedAt;
};

/**
 * An entry in a pending invite list.
 */
struct FGroupMemberInvite
{
	FUniqueNetIdPtr GetId() const { return AccountId; }

	FUniqueNetIdPtr AccountId;
	FUniqueNetIdPtr GroupHostId;
	FDateTime SentAt;
};

/**
 * An entry in a pending application list.
 */
struct FGroupMemberRequest
{
	FUniqueNetIdPtr GetId() const { return AccountId; }

	FUniqueNetIdPtr AccountId;
	FDateTime SentAt;
};

/**
 * An entry in a group denylist
 */
struct FGroupDenylistEntry
{
	FUniqueNetIdPtr GetId() const { return AccountId; }

	FUniqueNetIdPtr AccountId;
	bool bIsApplicant;
};

/**
 * An entry in a group member list.
 */
struct FUserMembershipEntry
{
	FUniqueNetIdPtr GetId() const { return AccountId; }

	/** AccountId of the member */
	FUniqueNetIdPtr AccountId;

	/** Does the user have admin rights for this group. */
	bool bAdmin;

	/** Is the user the owner of this group. */
	bool bIsOwner;

	/** When did the user join this group */
	FDateTime JoinedAt;

	/** Arbitrary namespace string used to filter group groups in some queries or client side. Usually this would be the game codename */
	FString Namespace;

	/** GroupId */
	FUniqueNetIdPtr GroupId;

	/** The display name of the group */
	FText Name;

	/** AccountId of the group owner */
	FUniqueNetIdPtr Owner;
};

/**
 * An entry in the list of user pending membership applications.
 */
struct FApplicationEntry
{
	FUniqueNetIdPtr GetId() const { return AccountId; }

	/** AccountId of the user who applied for group membership. */
	FUniqueNetIdPtr AccountId;

	/** Timestamp of when the application was sent. */
	FDateTime SentAt;

	/** GroupId of the group for which membership was applied. */
	FUniqueNetIdPtr GroupId;

	/** Namespace in which the application exists. */
	FString Namespace;

	/** Name of the group. */
	FText Name;
};

/**
* An entry in a user's list of groups to which they are invited.
*/
struct FInvitationEntry
{
	FUniqueNetIdPtr GetId() const { return AccountId; }

	/** AccountId of the user invited to group membership. */
	FUniqueNetIdPtr AccountId;

	/** AccountId of the group user that did the inviting. */
	FUniqueNetIdPtr GroupHostId;

	/** Timestamp of when the application was sent. */
	FDateTime SentAt;

	/** The GroupId for the group. */
	FUniqueNetIdPtr GroupId;

	/** The Namespace for the invitation. */
	FString Namespace;

	/** The Name of the group. */
	FText Name;
};

template <typename EntryType>
struct TGroupConfigEntry
{
	FUniqueNetIdPtr GetId() const { return AccountId; }

	/** Context AccountId. */
	FUniqueNetIdPtr AccountId;

	/** The max number of members in any group. */
	FString Key;

	/** The max number of groups in which a user may be a member. */
	EntryType Value;
};

typedef TGroupConfigEntry<int32> FGroupConfigEntryInt;
typedef TGroupConfigEntry<bool> FGroupConfigEntryBool;

template <typename EntryType>
struct IGroupUserCollection
{
	virtual const EntryType* GetEntry(const FUniqueNetId& EntryId) const =0;
	virtual EntryType* GetEntry(const FUniqueNetId& EntryId) =0;
	virtual FUniqueNetIdRef GetCollectionId() const =0;
	virtual void CopyEntries(TArray<EntryType>& Out) const =0;
};

/**
 * A structure for caching a list of people in the group and their roles
 */
typedef IGroupUserCollection<FGroupMember> IGroupRoster;

/**
 * A structure for caching a list of people that have been invited to join a group
 */
typedef IGroupUserCollection<FGroupMemberInvite> IGroupInvites;

/**
 * A structure for caching a list of people who have requested to join the group
 */
typedef IGroupUserCollection<FGroupMemberRequest> IGroupRequests;

/**
 * A structure for caching a list of people who have been banned from a group
 */
typedef IGroupUserCollection<FGroupDenylistEntry> IGroupDenylist;

/**
 * What groups does a particular user currently belong to and what roles do they fill
 */
typedef IGroupUserCollection<FUserMembershipEntry> IUserMembership;

/**
 * A list of pending membership applications for a given user.
 */
typedef IGroupUserCollection<FApplicationEntry> IApplications;

/**
* A list of pending membership invitations for a given user.
*/
typedef IGroupUserCollection<FInvitationEntry> IInvitations;

/**
* Configuration key->values (int)
*/
typedef TMap<FString, TSharedPtr<const FGroupConfigEntryInt>> IGroupConfigInt;

/**
* Configuration key->values (boolean)
*/
typedef TMap<FString, TSharedPtr<const FGroupConfigEntryBool>> IGroupConfigBool;

/**
 * Group search options
 */
struct FGroupSearchOptions
{
	/** case insensitive group name keyword for search group, should itself be a valid team name */
	FString Query;

	/** language filter */
	TOptional<FString> Language;

	/** tags filter */
	TArray<FString> Tags;

	/** minimal group size threshold - a size group should have to show up in results */
	TOptional<uint32> MinSize;

	/** Offset and size for the query. */
	TOptional<FPagedQuery> Paging;
};

enum class EGroupSortOrder
{
	CreatedDescending,
	SizeDescending,
	NameAscending,
	NameDescending,
};

/**
 * Structure to encapsulate the result of a call to the server. This only contains enough information to check for errors
 */
struct FGroupsResult
{
	int32 HttpStatus;
	FUniqueNetIdPtr PrimaryId;
	FString ErrorContent;

	inline bool DidSucceed() const { return (HttpStatus / 100) == 2; }

	FGroupsResult(int32 InHttpStatus = 0, FUniqueNetIdPtr InPrimaryId = nullptr)
		: HttpStatus(InHttpStatus)
		, PrimaryId(InPrimaryId)
		, ErrorContent()
	{
	}

	FGroupsResult(int32 InHttpStatus, const FString& Error, FUniqueNetIdPtr InPrimaryId = nullptr)
		: HttpStatus(InHttpStatus)
		, PrimaryId(InPrimaryId)
		, ErrorContent(Error)
	{
	}
};

/**
 * Generic delegate used for when an asynchronous query completes
 *
 * @param Result A structure containing information about whether the query succeeded or failed. 
 *        Use the GetCached* related function once this delegate has been called to retrieve the result payload.
 */
DECLARE_DELEGATE_OneParam(FOnGroupsRequestCompleted, FGroupsResult);

/**
 * Specialized result for FindGroups 
 */
struct FFindGroupsResult
{
	bool bDidSucceed;
	TArray< TSharedPtr<const IGroupInfo> > MatchingGroups;
	FPagedQuery Paging;
	FString ErrorContent;

	FFindGroupsResult()
		: bDidSucceed(false)
	{
	}
};
DECLARE_DELEGATE_OneParam(FOnFindGroupsCompleted, FFindGroupsResult);

/**
 * Interface definition for the online teams (groups) service
 * Groups are collections of users with an owner and a set of administrators
 */
class IOnlineGroups
{
public: // delegates

	/**
	 * Delegate fired when a group is updated. We only receive notifications for groups for which one 
	 * of the logged-in players is a member.
	 * 
	 * @param GroupId The GUID of the group.
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnGroupUpdated, const FUniqueNetId&);
	FOnGroupUpdated OnGroupUpdated;

public: // callable by all users

	/**
	 * Create a new group using the specified GroupInfo. FGroupsResult::GroupId
	 * can be used to identify the newly created group once the callback executes (FGroupsResult being a param to 
	 * the callback).
	 *
	 * @param ContextUserId The ID of the user whose credentials are being used to make this call
	 * @param GroupInfo The display info to use for the new group. Name, description, etc
	 * @param OnCompleted This callback is invoked after contacting the server. It is guaranteed to occur
	 *        (regardless of success/fail) and will not be called before this function returns.
	 */
	virtual void CreateGroup(const FUniqueNetId& ContextUserId, const FGroupDisplayInfo& GroupInfo, const FOnGroupsRequestCompleted& OnCompleted) = 0;

	/**
	 * Find all groups matching the specified search string
	 *
	 * @param ContextUserId The ID of the user whose credentials are being used to make this call
	 * @param SearchOptions A collections of search parameters passed to the search.
	 * @param OnCompleted Callback delegate which will receive the results of the search. Any GroupInfo from the search will 
	 *        also be available from GetCachedGroupInfo.
	 */
	virtual void FindGroups(const FUniqueNetId& ContextUserId, const FGroupSearchOptions& SearchOptions, const FOnFindGroupsCompleted& OnCompleted) = 0;

	/** 
	 * Ask the server for GroupInfo corresponding to the provided group ID. If this completes 
	 * successfully, you can call GetCachedGroupInfo to get the full group information. Even if the group info
	 * is currently cached, this function will contact the server (ostensibly to refresh the cache).
	 *
	 * @param ContextUserId The ID of the user whose credentials are being used to make this call
	 * @param GroupId The group's globally unique ID to query
	 * @param OnCompleted This callback is invoked after contacting the server. It is guaranteed to occur
	 *        (regardless of success/fail) and will not be called before this function returns.
	 */
	virtual void QueryGroupInfo(const FUniqueNetId& ContextUserId, const FUniqueNetId& GroupId, const FOnGroupsRequestCompleted& OnCompleted) = 0;

	/**
	* Ask the server if the given group name currently exists. If it completes successfully, you can call
	* GetCachedGroupNameExist to get the result. Even if the result is currently cached, this function will
	* contact the server (ostensibly to refresh the cache).
	*
	* @param ContextUserId The ID of the user whose credentials are being used to make this call
	* @param GroupName The group name to query
	* @param OnCompleted This callback is invoked after contacting the server. It is guaranteed to occur
	*        (regardless of success/fail) and will not be called before this function returns.
	*/
	virtual void QueryGroupNameExist(const FUniqueNetId& ContextUserId, const FString& GroupName, const FOnGroupsRequestCompleted& OnCompleted) = 0;

	/**
	 * Get the group info for a group that has been previously queried. The shared pointer will be 
	 * empty if the group has not been successfully queried (or if it was purged from the cache).
	 * From time to time, the cache may be cleaned up. Keep a shared pointer to groups you care about
	 * or be willing to re-query.
	 *
	 * @param ContextUserId The ID of the user whose credentials are being used to make this call
	 * @param GroupId The group's globally unique ID
	 * @return Shared pointer to the cached group info structure if one exists.
	 */
	virtual TSharedPtr<const IGroupInfo> GetCachedGroupInfo(const FUniqueNetId& ContextUserId, const FUniqueNetId& GroupId) = 0;

	/**
	 * Submit a request to join the specified group. This is done on behalf of the context
	 * user (provided when you request this IOnlineGroups interface) and also updates the their cached membership
	 * info. You can call GetCachedUserMembership afterwards to see if they are a full member or petitioner.
	 *
	 * @param ContextUserId The ID of the user whose credentials are being used to make this call
	 * @param GroupId The group's globally unique ID
	 * @param OnCompleted This callback is invoked after contacting the server. It is guaranteed to occur
	 *        (regardless of success/fail) and will not be called before this function returns.
	 */
	virtual void JoinGroup(const FUniqueNetId& ContextUserId, const FUniqueNetId& GroupId, const FOnGroupsRequestCompleted& OnCompleted) = 0;

	/**
	 * Tells the server to remove the context user from the specified group.
	 * Should always result in the user's role for that group becoming Unaffiliated. Owners cannot leave their group, use
	 * TransferGroup (followed by LeaveGroup) or DeleteGroup instead.
	 *
	 * @param ContextUserId The ID of the user whose credentials are being used to make this call
	 * @param GroupId The group's globally unique ID
	 * @param OnCompleted This callback is invoked after contacting the server. It is guaranteed to occur
	 *        (regardless of success/fail) and will not be called before this function returns.
	 */
	virtual void LeaveGroup(const FUniqueNetId& ContextUserId, const FUniqueNetId& GroupId, const FOnGroupsRequestCompleted& OnCompleted) = 0;

	/**
	* Cancel pending request to join the given group.
	*
	* @param ContextUserId The ID of the user whose credentials are being used to make this call
	* @param GroupId The group's globally unique ID
	* @param OnCompleted This callback is invoked after contacting the server. It is guaranteed to occur
	*        (regardless of success/fail) and will not be called before this function returns.
	*/
	virtual void CancelRequest(const FUniqueNetId& ContextUserId, const FUniqueNetId& GroupId, const FOnGroupsRequestCompleted& OnCompleted) = 0;

	/**
	 * Accept a pending invite to join a group. This is done on behalf of the context
	 * user (provided when you request this IOnlineGroups interface) and also updates the their cached membership
	 * info. You can call GetCachedUserMembership afterwards to see if they are a full member or petitioner.
	 *
	 * @param ContextUserId The ID of the user whose credentials are being used to make this call
	 * @param GroupId The group's globally unique ID
	 * @param OnCompleted This callback is invoked after contacting the server. It is guaranteed to occur
	 *        (regardless of success/fail) and will not be called before this function returns.
	 */
	virtual void AcceptInvite(const FUniqueNetId& ContextUserId, const FUniqueNetId& GroupId, const FOnGroupsRequestCompleted& OnCompleted) = 0;

	/**
	 * Decline a pending invite to join a group. This is done on behalf of the context
	 * user (provided when you request this IOnlineGroups interface) and also updates the their cached membership
	 * info. You can call GetCachedUserMembership afterwards to see if they are a full member or petitioner.
	 *
	 * @param ContextUserId The ID of the user whose credentials are being used to make this call
	 * @param GroupId The group's globally unique ID
	 * @param OnCompleted This callback is invoked after contacting the server. It is guaranteed to occur
	 *        (regardless of success/fail) and will not be called before this function returns.
	 */
	virtual void DeclineInvite(const FUniqueNetId& ContextUserId, const FUniqueNetId& GroupId, const FOnGroupsRequestCompleted& OnCompleted) = 0;

	/**
	 * Get the list of members for a group and their role info.
	 * When the OnCompleted callback fires, if it succeeded you can use GetCachedGroupRoster to retrieve the membership 
	 * information.
	 *
	 * @param ContextUserId The ID of the user whose credentials are being used to make this call
	 * @param GroupId The group's globally unique ID
	 * @param OnCompleted This callback is invoked after contacting the server. It is guaranteed to occur
	 *        (regardless of success/fail) and will not be called before this function returns.
	 */
	virtual void QueryGroupRoster(const FUniqueNetId& ContextUserId, const FUniqueNetId& GroupId, const FOnGroupsRequestCompleted& OnCompleted) = 0;

	/**
	 * Get the cached Roster (membership) information for a group. If the information is not cached locally, call 
	 * QueryGroupRoster to request it from the server. If the context user is a member of this group, the cached roster
	 * information will include petitioners and invitees, otherwise it will not.
	 *
	 * @param ContextUserId The ID of the user whose credentials are being used to make this call
	 * @param GroupId The group's globally unique ID
	 * @return Shared pointer to the cached roster structure if one exists
	 */
	virtual TSharedPtr<const IGroupRoster> GetCachedGroupRoster(const FUniqueNetId& ContextUserId, const FUniqueNetId& GroupId) = 0;

	/**
	 * Queries the server for updated membership information for a particular user. This retrieves which groups they are members of.
	 * If the callback reports success, use GetCachedUserMembership to retrieve details.
	 *
	 * @param ContextUserId The ID of the user whose credentials are being used to make this call
	 * @param UserId The user to query for membership information.
	 * @param OnCompleted This callback is invoked after contacting the server. It is guaranteed to occur
	 *        (regardless of success/fail) and will not be called before this function returns.
	 */
	virtual void QueryUserMembership(const FUniqueNetId& ContextUserId, const FUniqueNetId& UserId, const FOnGroupsRequestCompleted& OnCompleted) = 0;

	/**
	 * Get cached user membership information (if it exists). This retrieves which groups they are members of.
	 * If UserId is the ID of the context user, then this list will include groups to which the user has been invited 
	 * (use JoinGroup to accept, LeaveGroup to decline) or groups they have petitioned but haven't received a 
	 * response for yet.
	 *
	 * @param ContextUserId The ID of the user whose credentials are being used to make this call
	 * @param UserId The ID of the user to query.
	 * @param OnCompleted This callback is invoked after contacting the server. It is guaranteed to occur
	 *        (regardless of success/fail) and will not be called before this function returns.
	 */
	virtual TSharedPtr<const IUserMembership> GetCachedUserMembership(const FUniqueNetId& ContextUserId, const FUniqueNetId& UserId) = 0;

	/**
	* Queries the server for a list of groups to which the user has applied for membership.
	* If the callback reports success, use GetCachedApplications to retrieve details.
	*
	* @param ContextUserId The ID of the user whose credentials are being used to make this call
	* @param UserId The user to query for pending membership application information.
	* @param OnCompleted This callback is invoked after contacting the server. It is guaranteed to occur
	*        (regardless of success/fail) and will not be called before this function returns.
	*/
	virtual void QueryOutgoingApplications(const FUniqueNetId& ContextUserId, const FUniqueNetId& UserId, const FOnGroupsRequestCompleted& OnCompleted) = 0;

	/**
	* Get cached pending application information (if it exists). This retrieves a list of membership
	* applications that the user has created (outgoing) or to which the user can respond as admin (incoming).
	*
	* @param ContextUserId The ID of the user whose credentials are being used to make this call
	* @param UserId The ID of the user to query.
	* @param OnCompleted This callback is invoked after contacting the server. It is guaranteed to occur
	*        (regardless of success/fail) and will not be called before this function returns.
	*/
	virtual TSharedPtr<const IApplications> GetCachedApplications(const FUniqueNetId& ContextUserId, const FUniqueNetId& UserId) = 0;

	/**
	* Queries the server for a list of group invitations the user has sent.
	* If the callback reports success, use GetCachedApplications to retrieve details.
	*
	* @param ContextUserId The ID of the user whose credentials are being used to make this call
	* @param UserId The user to query for pending membership invitation information.
	* @param OnCompleted This callback is invoked after contacting the server. It is guaranteed to occur
	*        (regardless of success/fail) and will not be called before this function returns.
	*/
	virtual void QueryOutgoingInvitations(const FUniqueNetId& ContextUserId, const FUniqueNetId& UserId, const FOnGroupsRequestCompleted& OnCompleted) = 0;

	/**
	* Queries the server for a list of groups to which the user has been invited.
	* If the callback reports success, use GetCachedApplications to retrieve details.
	*
	* @param ContextUserId The ID of the user whose credentials are being used to make this call
	* @param UserId The user to query for pending membership invitation information.
	* @param OnCompleted This callback is invoked after contacting the server. It is guaranteed to occur
	*        (regardless of success/fail) and will not be called before this function returns.
	*/
	virtual void QueryIncomingInvitations(const FUniqueNetId& ContextUserId, const FUniqueNetId& UserId, const FOnGroupsRequestCompleted& OnCompleted) = 0;

	/**
	* Get cached pending invitation information (if it exists). This retrieves a list of membership
	* invitations that the user's admin clans have created (outgoing) or to which the user can respond (incoming).
	*
	* @param ContextUserId The ID of the user whose credentials are being used to make this call
	* @param UserId The ID of the user to query.
	* @param OnCompleted This callback is invoked after contacting the server. It is guaranteed to occur
	*        (regardless of success/fail) and will not be called before this function returns.
	*/
	virtual TSharedPtr<const IInvitations> GetCachedInvitations(const FUniqueNetId& ContextUserId, const FUniqueNetId& UserId) = 0;

public: // can be called by group admins

	/**
	 * Update the user specified fields of the group such as name, description, etc. This call may fail if the name changes and 
	 * the new name is already in use within this namespace.
	 *
	 * @param ContextUserId The ID of the user whose credentials are being used to make this call
	 * @param GroupId The group's globally unique ID
	 * @param GroupInfo The new group display info structure.
	 * @param OnCompleted This callback is invoked after contacting the server. It is guaranteed to occur
	 *        (regardless of success/fail) and will not be called before this function returns.
	 */
	virtual void UpdateGroupInfo(const FUniqueNetId& ContextUserId, const FUniqueNetId& GroupId, const FGroupDisplayInfo& GroupInfo, const FOnGroupsRequestCompleted& OnCompleted) = 0;

	/**
	 * Accept a user request to join the group.
	 *
	 * @param ContextUserId The ID of the user whose credentials are being used to make this call
	 * @param GroupId The group's globally unique ID
	 * @param UserId The user to add to the group.
	 * @param OnCompleted This callback is invoked after contacting the server. It is guaranteed to occur
	 *        (regardless of success/fail) and will not be called before this function returns.
	 */
	virtual void AcceptUser(const FUniqueNetId& ContextUserId, const FUniqueNetId& GroupId, const FUniqueNetId& UserId, const FOnGroupsRequestCompleted& OnCompleted) = 0;

	/**
	 * Decline a user request to join the group.
	 *
	 * @param ContextUserId The ID of the user whose credentials are being used to make this call
	 * @param GroupId The group's globally unique ID
	 * @param UserId The user to add to the group.
	 * @param OnCompleted This callback is invoked after contacting the server. It is guaranteed to occur
	 *        (regardless of success/fail) and will not be called before this function returns.
	 */
	virtual void DeclineUser(const FUniqueNetId& ContextUserId, const FUniqueNetId& GroupId, const FUniqueNetId& UserId, const FOnGroupsRequestCompleted& OnCompleted) = 0;

	/**
	 * Invite a user to join the specified group.
	 *
	 * @param ContextUserId The ID of the user whose credentials are being used to make this call
	 * @param GroupId The group's globally unique ID
	 * @param UserId The user to add to the group.
	 * @param bAllowBlocked If true, if the user is on the group's denylist, the user will be unblocked before creating the invite. Otherwise adding a blocked user will fail.
	 * @param OnCompleted This callback is invoked after contacting the server. It is guaranteed to occur
	 *        (regardless of success/fail) and will not be called before this function returns.
	 */
	virtual void InviteUser(const FUniqueNetId& ContextUserId, const FUniqueNetId& GroupId, const FUniqueNetId& UserId, bool bAllowBlocked, const FOnGroupsRequestCompleted& OnCompleted) = 0;

	/**
	 * Invite a user to join the specified group.
	 *
	 * @param ContextUserId The ID of the user whose credentials are being used to make this call
	 * @param GroupId The group's globally unique ID
	 * @param UserId The user to add to the group.
	 * @param OnCompleted This callback is invoked after contacting the server. It is guaranteed to occur
	 *        (regardless of success/fail) and will not be called before this function returns.
	 */
	virtual void InviteUser(const FUniqueNetId& ContextUserId, const FUniqueNetId& GroupId, const FUniqueNetId& UserId, const FOnGroupsRequestCompleted& OnCompleted)
	{
		InviteUser(ContextUserId, GroupId, UserId, false, OnCompleted);
	}

	/**
	 * Cancels an invitation to join the group.
	 *
	 * @param ContextUserId The ID of the user whose credentials are being used to make this call
	 * @param GroupId The group's globally unique ID
	 * @param UserId The user whose invite to remove.
	 * @param OnCompleted This callback is invoked after contacting the server. It is guaranteed to occur
	 *        (regardless of success/fail) and will not be called before this function returns.
	 */
	virtual void CancelInvite(const FUniqueNetId& ContextUserId, const FUniqueNetId& GroupId, const FUniqueNetId& UserId, const FOnGroupsRequestCompleted& OnCompleted) = 0;

	/**
	 * Kick a user from the group OR decline a petition OR rescind an invitation (success always results in the user's role becoming Unaffiliated).
	 *
	 * @param ContextUserId The ID of the user whose credentials are being used to make this call
	 * @param GroupId The group's globally unique ID
	 * @param UserId The user to remove from the group.
	 * @param OnCompleted This callback is invoked after contacting the server. It is guaranteed to occur
	 *        (regardless of success/fail) and will not be called before this function returns.
	 */
	virtual void RemoveUser(const FUniqueNetId& ContextUserId, const FUniqueNetId& GroupId, const FUniqueNetId& UserId, const FOnGroupsRequestCompleted& OnCompleted) = 0;

	/**
	 * Promote a Member to an Admin within the specified group. Fails if the user's role is not Member.
	 *
	 * @param ContextUserId The ID of the user whose credentials are being used to make this call
	 * @param GroupId The group's globally unique ID
	 * @param UserId The member to promote.
	 * @param OnCompleted This callback is invoked after contacting the server. It is guaranteed to occur
	 *        (regardless of success/fail) and will not be called before this function returns.
	 */
	virtual void PromoteUser(const FUniqueNetId& ContextUserId, const FUniqueNetId& GroupId, const FUniqueNetId& UserId, const FOnGroupsRequestCompleted& OnCompleted) = 0;
	
	/**
	 * Demote an Admin to a Member within the specified group. Fails if the user's role is not Admin (Owner != Admin).
	 *
	 * @param ContextUserId The ID of the user whose credentials are being used to make this call
	 * @param GroupId The group's globally unique ID
	 * @param UserId The user to demote to regular member.
	 * @param OnCompleted This callback is invoked after contacting the server. It is guaranteed to occur
	 *        (regardless of success/fail) and will not be called before this function returns.
	 */
	virtual void DemoteUser(const FUniqueNetId& ContextUserId, const FUniqueNetId& GroupId, const FUniqueNetId& UserId, const FOnGroupsRequestCompleted& OnCompleted) = 0;

	/**
	 * Ban a user from joining the specified group.
	 *
	 * Any user but team owner can be denylisted including member, invitee, applicant and/or any other user unrelated to the team.
	 * Blocking follows "ignore" logic: blocked users shouldn't know for sure that they've been blocked.
	 * Blocked user will still be able to apply for membership and cancel pending application even while blocked.
	 *
	 * @param ContextUserId The ID of the user whose credentials are being used to make this call
	 * @param GroupId The group's globally unique ID
	 * @param UserId The user to denylist.
	 * @param OnCompleted This callback is invoked after contacting the server. It is guaranteed to occur
	 *        (regardless of success/fail) and will not be called before this function returns.
	 */
	virtual void BlockUser(const FUniqueNetId& ContextUserId, const FUniqueNetId& GroupId, const FUniqueNetId& UserId, const FOnGroupsRequestCompleted& OnCompleted) = 0;

	/**
	 * Remove a user from the group's denylist list.
	 *
	 * @param ContextUserId The ID of the user whose credentials are being used to make this call
	 * @param GroupId The group's globally unique ID
	 * @param UserId The user to allow back into the group.
	 * @param OnCompleted This callback is invoked after contacting the server. It is guaranteed to occur
	 *        (regardless of success/fail) and will not be called before this function returns.
	 */
	virtual void UnblockUser(const FUniqueNetId& ContextUserId, const FUniqueNetId& GroupId, const FUniqueNetId& UserId, const FOnGroupsRequestCompleted& OnCompleted) = 0;

	/**
	 * Get the list of outstanding invites to a group.
	 * When the OnCompleted callback fires, if it succeeded you can use GetCachedGroupInvites to retrieve the membership
	 * information.
	 *
	 * @param ContextUserId The ID of the user whose credentials are being used to make this call
	 * @param GroupId The group's globally unique ID
	 * @param OnCompleted This callback is invoked after contacting the server. It is guaranteed to occur
	 *        (regardless of success/fail) and will not be called before this function returns.
	 */
	virtual void QueryGroupInvites(const FUniqueNetId& ContextUserId, const FUniqueNetId& GroupId, const FOnGroupsRequestCompleted& OnCompleted) = 0;

	/**
	 * Get the cached list of outstanding invites to a group. If the information is not cached locally, call
	 * QueryGroupInvites to request it from the server.
	 *
	 * @param ContextUserId The ID of the user whose credentials are being used to make this call
	 * @param GroupId The group's globally unique ID
	 * @return Shared pointer to the cached roster structure if one exists
	 */
	virtual TSharedPtr<const IGroupInvites> GetCachedGroupInvites(const FUniqueNetId& ContextUserId, const FUniqueNetId& GroupId) = 0;

	/**
	 * Get the list of users requesting to becoming members of a group.
	 * When the OnCompleted callback fires, if it succeeded you can use GetCachedGroupRequests to retrieve the membership
	 * information.
	 *
	 * @param ContextUserId The ID of the user whose credentials are being used to make this call
	 * @param GroupId The group's globally unique ID
	 * @param OnCompleted This callback is invoked after contacting the server. It is guaranteed to occur
	 *        (regardless of success/fail) and will not be called before this function returns.
	 */
	virtual void QueryGroupRequests(const FUniqueNetId& ContextUserId, const FUniqueNetId& GroupId, const FOnGroupsRequestCompleted& OnCompleted) = 0;

	/**
	 * Get the cached list of  users requesting to becoming members of a group. If the information is not cached locally, call
	 * QueryGroupRequests to request it from the server.
	 *
	 * @param ContextUserId The ID of the user whose credentials are being used to make this call
	 * @param GroupId The group's globally unique ID
	 * @return Shared pointer to the cached roster structure if one exists
	 */
	virtual TSharedPtr<const IGroupRequests> GetCachedGroupRequests(const FUniqueNetId& ContextUserId, const FUniqueNetId& GroupId) = 0;

	/**
	 * Get the list of users banned from this group.
	 * When the OnCompleted callback fires, if it succeeded you can use GetCachedGroupDenylist to retrieve the
	 * information.
	 *
	 * @param ContextUserId The ID of the user whose credentials are being used to make this call
	 * @param GroupId The group's globally unique ID
	 * @param OnCompleted This callback is invoked after contacting the server. It is guaranteed to occur
	 *        (regardless of success/fail) and will not be called before this function returns.
	 */
	virtual void QueryGroupDenylist(const FUniqueNetId& ContextUserId, const FUniqueNetId& GroupId, const FOnGroupsRequestCompleted& OnCompleted) = 0;

	/**
	 * Get the cached list of users banned from this group. If the information is not cached locally, call
	 * QueryGroupDenylist to request it from the server.
	 *
	 * @param ContextUserId The ID of the user whose credentials are being used to make this call
	 * @param GroupId The group's globally unique ID
	 * @return Shared pointer to the cached roster structure if one exists
	 */
	virtual TSharedPtr<const IGroupDenylist> GetCachedGroupDenylist(const FUniqueNetId& ContextUserId, const FUniqueNetId& GroupId) = 0;

	/**
	* Queries the server for a list of membership applications that UserId can process (accept, reject, or block) as group admin.
	* If the callback reports success, use GetCachedApplications to retrieve details.
	*
	* @param ContextUserId The ID of the user whose credentials are being used to make this call
	* @param UserId The user to query for pending group membership application information.
	* @param OnCompleted This callback is invoked after contacting the server. It is guaranteed to occur
	*        (regardless of success/fail) and will not be called before this function returns.
	*/
	virtual void QueryIncomingApplications(const FUniqueNetId& ContextUserId, const FUniqueNetId& UserId, const FOnGroupsRequestCompleted& OnCompleted) = 0;

public: // configuration queries

	/**
	* Queries the system configuration for system-wide group max membership headcount.
	*/
	virtual void QueryConfigHeadcount(const FUniqueNetId& ContextUserId, const FOnGroupsRequestCompleted& OnCompleted) = 0;

	/**
	* Queries the system configuration for system-wide user max membership count.
	*/
	virtual void QueryConfigMembership(const FUniqueNetId& ContextUserId, const FOnGroupsRequestCompleted& OnCompleted) = 0;

	/**
	* Get the result of a previous configuration query.
	* Only the member queried for will be valid.
	*/
	virtual TSharedPtr<const FGroupConfigEntryInt> GetCachedConfigInt(const FString& Key) = 0;

	/**
	* Get the result of a previous configuration query.
	* Only the member queried for will be valid.
	*/
	virtual TSharedPtr<const FGroupConfigEntryBool> GetCachedConfigBool(const FString& Key) = 0;

public: // can be called by group owner only

	/**
	 * Promote an existing Admin to be the new Owner and simultaneously demotes the context user to Admin. Fails if the new owner
	 * is not currently an Admin of the group. Also fails if the new owner cannot own the group (due to owning too many other groups, as
	 * determined by the namespace config, perhaps).
	 *
	 * @param ContextUserId The ID of the user whose credentials are being used to make this call
	 * @param GroupId The group's globally unique ID
	 * @param UserId The user who should become the new group owner. This user must already be an Admin or the call fails.
	 * @param OnCompleted This callback is invoked after contacting the server. It is guaranteed to occur
	 *        (regardless of success/fail) and will not be called before this function returns.
	 */
	virtual void TransferGroup(const FUniqueNetId& ContextUserId, const FUniqueNetId& GroupId, const FUniqueNetId& NewOwnerId, const FOnGroupsRequestCompleted& OnCompleted) = 0;

	/**
	 * Delete a group entirely.
	 *
	 * @param ContextUserId The ID of the user whose credentials are being used to make this call
	 * @param GroupId The group's globally unique ID
	 * @param OnCompleted This callback is invoked after contacting the server. It is guaranteed to occur
	 *        (regardless of success/fail) and will not be called before this function returns.
	 */
	virtual void DeleteGroup(const FUniqueNetId& ContextUserId, const FUniqueNetId& GroupId, const FOnGroupsRequestCompleted& OnCompleted) = 0;

protected:
	IOnlineGroups() {}
public:
	virtual ~IOnlineGroups() = default;

	virtual void SetNamespace(const FString& Ns) = 0;
	virtual const FString& GetNamespace() const = 0;
};

typedef TSharedPtr<IOnlineGroups, ESPMode::ThreadSafe> IOnlineGroupsPtr;
