// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemTypes.h"
#include "OnlineDelegateMacros.h"

struct FOnlineError;



ONLINESUBSYSTEM_API DECLARE_LOG_CATEGORY_EXTERN(LogOnlineIdentity, Log, All);

#define UE_LOG_ONLINE_IDENTITY(Verbosity, Format, ...) \
{ \
	UE_LOGF(LogOnlineIdentity, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

#define UE_CLOG_ONLINE_IDENTITY(Conditional, Verbosity, Format, ...) \
{ \
	UE_CLOGF(Conditional, LogOnlineIdentity, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

/**
 * Account credentials needed to sign in to an online service
 */
class FOnlineAccountCredentials
{
public:

	/** Type of account. Needed to identity the auth method to use (epic, internal, facebook, etc) */
	FString Type;
	/** Id of the user logging in (email, display name, facebook id, etc) */
	FString Id;
	/** Credentials of the user logging in (password or auth token) */
	FString Token;

	/**
	 * Equality operator
	 */
	bool operator==(const FOnlineAccountCredentials& Other) const
	{
		return Other.Type == Type && 
			Other.Id == Id;
	}

	/**
	 * Constructor
	 */
	FOnlineAccountCredentials(const FString& InType, const FString& InId, const FString& InToken) :
		Type(InType),
		Id(InId),
		Token(InToken)
	{
	}

	/**
	 * Constructor
	 */
	FOnlineAccountCredentials()
	{
	}

	FString ToDebugString() const
	{
		return FString::Printf(TEXT("{Id: %s, Token: %s, Type: %s}"), *Id, OSS_REDACT(*Token), *Type);
	}
};

namespace EUserPrivileges
{
	enum Type
	{
		/** Whether the user can play at all, online or offline - may be age restricted */
		CanPlay,
		/** Whether the user can play in online modes */
		CanPlayOnline,
		/** Whether the user can use voice and text chat */
		CanCommunicateOnline,
		/** Whether the user can use content generated by other users */
		CanUseUserGeneratedContent,
		/** Whether the user can ever participate in cross-play due to age restrictions */
		CanUserCrossPlay
	};
}

/** Whether to show UI to resolve the privilege if there is no access when GetUserPrivilege */
enum class EShowPrivilegeResolveUI : uint8
{
	/** Use default behavior, which preserved the behaviour prior to the introduction of this. */
	Default,
	/** Show UI to resolve privilege */
	Show,
	/** Not show UI to resolve privilege */
	NotShow
};

/**
 * This struct will be broadcast with the FOnControllerPairingChanged event. This event is not broadcast on all OnlineIdentity platforms.
 * There will be a struct for the previous user on the controller and for the new user on the controller.
 */
struct FControllerPairingChangedUserInfo
{
	/**
	 * One of the users involved (the one who had the controller or the one who now has the controller).
	 * This may be nobody (e.g. if the controller was assigned to a User, and now is assigned to nobody)
	*/
	const FUniqueNetId& User;

	/**
	 * The number of controllers now associated with User. If User is nobody, this number will be 0
	*/
	int32 ControllersRemaining;

	FControllerPairingChangedUserInfo(const FUniqueNetId& InUser, int32 InControllersRemaining)
		: User(InUser)
		, ControllersRemaining(InControllersRemaining)
	{
	};

};

/**
 * Delegate called when a player logs in/out
 *
 * @param LocalUserNum the controller number of the associated user
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnLoginChanged, int32 /*LocalUserNum*/);
typedef FOnLoginChanged::FDelegate FOnLoginChangedDelegate;

/**
 * Delegate called when a player's status changes but doesn't change profiles
 *
 * @param LocalUserNum the controller number of the associated user
 * @param OldStatus the old login status for the user
 * @param NewStatus the new login status for the user
 * @param NewId the new id to associate with the user
 */
DECLARE_MULTICAST_DELEGATE_FourParams(FOnLoginStatusChanged, int32 /*LocalUserNum*/, ELoginStatus::Type /*OldStatus*/, ELoginStatus::Type /*NewStatus*/, const FUniqueNetId& /*NewId*/);
typedef FOnLoginStatusChanged::FDelegate FOnLoginStatusChangedDelegate;

/**
 * Delegate called when an auth session is about to expire and a new auth token is needed.
 * The code bound to it is expected to respond by getting a new auth token and use it to re-call Login to refresh auth and keep the player logged in.
 * 
 * param LocalUserNum the controller number of the user whose auth is about to expire
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnAuthAboutToExpire, int32 /*LocalUserNum*/);
typedef FOnAuthAboutToExpire::FDelegate FOnAuthAboutToExpireDelegate;

/**
 * Delegate called when a controller-user pairing changes
 *
 * @param LocalUserNum the logged-in user number of the user who owned the device whose pairing changed. If no previous owner, the user number of the new owner.
 * @param PreviousUser the user that used to be paired with this controller
 * @param NewUser the user that is currently paired with this controller
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnControllerPairingChanged, int /*LocalUserNum*/, FControllerPairingChangedUserInfo /*PreviousUser*/, FControllerPairingChangedUserInfo /*NewUser*/);
typedef FOnControllerPairingChanged::FDelegate FOnControllerPairingChangedDelegate;

/**
 * Called when user account login has completed after calling Login() or AutoLogin()
 *
 * @param LocalUserNum the controller number of the associated user
 * @param bWasSuccessful true if server was contacted and a valid result received
 * @param UserId the user id received from the server on successful login
 * @param Error string representing the error condition
 */
DECLARE_MULTICAST_DELEGATE_FourParams(FOnLoginComplete, int32 /*LocalUserNum*/, bool /*bWasSuccessful*/, const FUniqueNetId& /*UserId*/, const FString& /*Error*/);
typedef FOnLoginComplete::FDelegate FOnLoginCompleteDelegate;


/**
 * Delegate used in notifying the UI/game that the manual logout completed
 *
 * @param LocalUserNum the controller number of the associated user
 * @param bWasSuccessful whether the async call completed properly or not
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnLogoutComplete, int32 /*LocalUserNum*/, bool /*bWasSuccessful*/);
typedef FOnLogoutComplete::FDelegate FOnLogoutCompleteDelegate;

/**
 * Delegate called when logout requires login flow to cleanup
 *
 * @param LoginDomains the login domains to clean up
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnLoginFlowLogout, const TArray<FString>& /*LoginDomains*/);
typedef FOnLoginFlowLogout::FDelegate FOnLoginFlowLogoutDelegate;

/**
 * Delegate executed when we get a user privilege result.
 *
 * @param UserId The unique id of the user who was queried
 * @param OnlineError the result of the operation
 */
DECLARE_DELEGATE_TwoParams(FOnRevokeAuthTokenCompleteDelegate, const FUniqueNetId& /*UserId*/, const FOnlineError& /*OnlineError*/);

/**
 * A struct for the external account token in case we need per platform fields
 */
struct FExternalAuthToken
{
	TArray<uint8> TokenData;
	FString TokenString;

	inline bool HasTokenData() const
	{
		return TokenData.Num() > 0;
	}

	inline bool HasTokenString() const
	{
		return !TokenString.IsEmpty();
	}

	inline bool IsValid() const
	{
		return HasTokenData() || HasTokenString();
	}
};

/**
 * Interface for registration/authentication of user identities
 */
class IOnlineIdentity
{

protected:
	IOnlineIdentity() {};

public:

	enum class EPrivilegeResults : uint32
	{
		/** The user has the requested privilege */
		NoFailures				=	0,
		/** Patch required before the user can use the privilege */
		RequiredPatchAvailable	=	1 << 0,
		/** System update required before the user can use the privilege */
		RequiredSystemUpdate	=	1 << 1,
		/** Parental control failure usually */
		AgeRestrictionFailure	=	1 << 2,
		/** XboxLive Gold / PSPlus required but not available */
		AccountTypeFailure		=	1 << 3,
		/** Invalid user */
		UserNotFound			=	1 << 4,
		/** User must be logged in */
		UserNotLoggedIn			=	1 << 5,
		/** User restricted from chat */
		ChatRestriction			=	1 << 6,
		/** User restricted from User Generated Content */
		UGCRestriction			=	1 << 7,
		/** Platform failed for unknown reason and handles its own dialogs */
		GenericFailure			=	1 << 8,
		/** Online play is restricted */
		OnlinePlayRestricted	=	1 << 9,
		/** Check failed because network is unavailable */
		NetworkConnectionUnavailable =	1 << 10,
	};

	virtual ~IOnlineIdentity() = default;

	/**
	 * Delegate called when a player logs in/out
	 *
	 * @param LocalUserNum the player that logged in/out
	 */
	DEFINE_ONLINE_DELEGATE_ONE_PARAM(OnLoginChanged, int32 /*LocalUserNum*/);

	/**
	 * Delegate called when a player's login status changes but doesn't change identity
	 *
	 * @param LocalUserNum the controller number of the associated user
	 * @param OldStatus the previous status of the user
	 * @param NewStatus the new login status for the user
	 * @param NewId the new id to associate with the user
	 */
	DEFINE_ONLINE_PLAYER_DELEGATE_THREE_PARAM(MAX_LOCAL_PLAYERS, OnLoginStatusChanged, ELoginStatus::Type /*OldStatus*/, ELoginStatus::Type /*NewStatus*/, const FUniqueNetId& /*NewId*/);

	/**
	 * Delegate called when an auth session is about to expire and a new auth token is needed.
	 * The code bound to it is expected to respond by getting a new auth token and use it to re-call Login to refresh auth and keep the player logged in.
	 * 
	 * param LocalUserNum the controller number of the user whose auth is about to expire
	 */
	DEFINE_ONLINE_DELEGATE_ONE_PARAM(OnAuthAboutToExpire, int32 /*LocalUserNum*/);

	/**
	 * Delegate called when a controller-user pairing changes
	 *
	 * @param LocalUserNum the controller number whose pairing changed
	 * @param PreviousUser the previous user associated with the controller
	 * @param NewUser the new user associated with the controller
	 */
	DEFINE_ONLINE_DELEGATE_THREE_PARAM(OnControllerPairingChanged, int /*LocalUserNum*/, FControllerPairingChangedUserInfo /*PreviousUser*/, FControllerPairingChangedUserInfo /*NewUser*/);

	/**
	 * Login/Authenticate with user credentials.
	 * Will call OnLoginComplete() delegate when async task completes
	 *
	 * @param LocalUserNum the controller number of the associated user
	 * @param AccountCredentials user account credentials needed to sign in to the online service
	 *
	 * @return true if login task was started
	 */
	virtual bool Login(int32 LocalUserNum, const FOnlineAccountCredentials& AccountCredentials) = 0;

	/**
	 * Called when user account login has completed after calling Login() or AutoLogin()
	 *
	 * @param LocalUserNum the controller number of the associated user
	 * @param bWasSuccessful true if server was contacted and a valid result received
	 * @param UserId the user id received from the server on successful login
	 * @param Error string representing the error condition
	 */
	DEFINE_ONLINE_PLAYER_DELEGATE_THREE_PARAM(MAX_LOCAL_PLAYERS, OnLoginComplete, bool /*bWasSuccessful*/, const FUniqueNetId& /*UserId*/, const FString& /*Error*/);

	/**
	 * Signs the player out of the online service
	 * Will call OnLogoutComplete() delegate when async task completes
	 *
	 * @param LocalUserNum the controller number of the associated user
	 *
	 * @return true if logout task was started
	 */
	virtual bool Logout(int32 LocalUserNum) = 0;

	/**
	 * Delegate used in notifying the that manual logout completed
	 *
	 * @param LocalUserNum the controller number of the associated user
	 * @param bWasSuccessful whether the async call completed properly or not
	 */
	DEFINE_ONLINE_PLAYER_DELEGATE_ONE_PARAM(MAX_LOCAL_PLAYERS, OnLogoutComplete, bool /*bWasSuccessful*/);

	/**
	 * Delegate called when the online subsystem requires the login flow to logout and cleanup
	 *
	 * @param LoginDomains login domains to be cleaned up
	 */
	DEFINE_ONLINE_DELEGATE_ONE_PARAM(OnLoginFlowLogout, const TArray<FString>& /*LoginDomains*/);

	/**
	 * Logs the player into the online service using parameters passed on the
	 * command line. Expects -AUTH_LOGIN=<UserName> -AUTH_PASSWORD=<password>. If either
	 * are missing, the function returns false and doesn't start the login
	 * process
	 *
	 * @param LocalUserNum the controller number of the associated user
	 *
	 * @return true if the async call started ok, false otherwise
	 */
	virtual bool AutoLogin(int32 LocalUserNum) = 0;

	/**
	 * Obtain online account info for a user that has been registered
	 *
	 * @param UserId user to search for
	 *
	 * @return info about the user if found, NULL otherwise
	 */
	virtual TSharedPtr<FUserOnlineAccount> GetUserAccount(const FUniqueNetId& UserId) const = 0;

	/**
	 * Obtain list of all known/registered user accounts
	 *
	 * @return info about the users if found, NULL otherwise
	 */
	virtual TArray<TSharedPtr<FUserOnlineAccount> > GetAllUserAccounts() const = 0;

	/**
	 * Gets the platform specific unique id for the specified player
	 *
	 * @param LocalUserNum the controller number of the associated user
	 *
	 * @return Valid player id object if the call succeeded, NULL otherwise
	 */
	virtual FUniqueNetIdPtr GetUniquePlayerId(int32 LocalUserNum) const = 0;

	/**
	 * Gets the platform specific unique id for the sponsor of the specified player
	 *
	 * @param LocalUserNum the controller number of the associated user
	 *
	 * @return Valid player id object if the sponsor exists, NULL otherwise
	 */
	virtual FUniqueNetIdPtr GetSponsorUniquePlayerId(int32 LocalUserNum) const { return nullptr; }

	/**
	 * Create a unique id from binary data (used during replication)
	 * 
	 * @param Bytes opaque data from appropriate platform
	 * @param Size size of opaque data
	 *
	 * @return UniqueId from the given data, NULL otherwise
	 */
	virtual FUniqueNetIdPtr CreateUniquePlayerId(uint8* Bytes, int32 Size) = 0;

	/**
	 * Create a unique id from string
	 * 
	 * @param Str string holding textual representation of an Id
	 *
	 * @return UniqueId from the given data, NULL otherwise
	 */
	virtual FUniqueNetIdPtr CreateUniquePlayerId(const FString& Str) = 0;

	/**
	 * Fetches the login status for a given player
	 *
	 * @param LocalUserNum the controller number of the associated user
	 *
	 * @return the enum value of their status
	 */
	virtual ELoginStatus::Type GetLoginStatus(int32 LocalUserNum) const = 0;

	/**
	 * Fetches the login status for a given player
	 *
	 * @param UserId the unique net id of the associated user
	 *
	 * @return the enum value of their status
	 */
	virtual ELoginStatus::Type GetLoginStatus(const FUniqueNetId& UserId) const = 0;

	/**
	 * Reads the player's nick name from the online service
	 *
	 * @param LocalUserNum the controller number of the associated user
	 *
	 * @return a string containing the players nick name
	 */
	//@todo - move to user interface
	virtual FString GetPlayerNickname(int32 LocalUserNum) const = 0;

	/**
	 * Reads the player's nick name from the online service
	 *
	 * @param UserId the unique net of the associated user
	 *
	 * @return a string containing the players nick name
	 */
	//@todo - move to user interface
	virtual FString GetPlayerNickname(const FUniqueNetId& UserId) const = 0;

	/**
	 * Gets a user's platform specific authentication token to verify their identity
	 *
	 * @param LocalUserNum the controller number of the associated user
	 *
	 * @return String representing the authentication token
	 */
	//@todo - remove and use GetUserAccount instead
	virtual FString GetAuthToken(int32 LocalUserNum) const = 0;

	/**
	 * Revoke the user's registered auth token.
	 *
	 * @param LocalUserId the unique net of the associated user
	 * @param Delegate delegate to execute when the async task completes
	 */
	virtual void RevokeAuthToken(const FUniqueNetId& LocalUserId, const FOnRevokeAuthTokenCompleteDelegate& Delegate) = 0;

	/**
	 * Clear an auth token associated with the user, if one was cached.
	 *
	 * If an auth token is obtained for some other purpose than user account login, it would
	 * be ignored an replaced with another when an actual login attempt is made, wasting the
	 * previously obtained one.  Caching avoids that potential waste, but must be manually
	 * cleared client-side when the login completes due to the remote side has consuming it.
	 *
	 * @param UserId the unique net of the associated user
	 */
	virtual void ClearCachedAuthToken(const FUniqueNetId& UserId) {};

	/**
	 * Delegate executed when the user's auth ticket has arrived
	 *
	 * @param LocalUserNum the controller number of the associated user
	 * @param bWasSuccessful whether the ticket was successfully retrieved or not
	 * @param AuthToken the token to use with linked auth
	 */
	DECLARE_DELEGATE_ThreeParams(FOnGetLinkedAccountAuthTokenCompleteDelegate, int32 /*LocalUserNum*/, bool /*bWasSuccessful*/, const FExternalAuthToken& /*AuthToken*/);

	/**
	 * Gets a user's platform specific authentication ticket used to sign into linked account(s)
	 *
	 * @param LocalUserNum the controller number of the associated user
	 * @param TokenType the platform specific token type you are requesting.
	 * @param Delegate delegate to execute when the async task completes
	 */
	virtual void GetLinkedAccountAuthToken(int32 LocalUserNum, const FString& TokenType, const FOnGetLinkedAccountAuthTokenCompleteDelegate& Delegate) const
	{
		FExternalAuthToken EmptyToken;
		Delegate.ExecuteIfBound(LocalUserNum, false, EmptyToken);
	}

	/**
	 * Delegate executed when we get a user privilege result.
	 *
	 * @param LocalUserId The unique id of the user who was queried
	 * @param Privilege the privilege that was queried
	 * @param PrivilegeResult bitwise OR of any privilege failures. 0 is success.
	 */
	DECLARE_DELEGATE_ThreeParams(FOnGetUserPrivilegeCompleteDelegate, const FUniqueNetId& /*LocalUserId*/, EUserPrivileges::Type /*Privilege*/, uint32 /*PrivilegeResult*/);

	/**
	 * Gets the status of a user's privilege.
	 *
	 * @param LocalUserId the unique id of the user to query
	 * @param Privilege the privilege you want to know about
	 * @param Delegate delegate to execute when the async task completes
	 * @param ShowResolveUI whether to show UI to resolve if user don't have the privilege
	 */
	virtual void GetUserPrivilege(const FUniqueNetId& LocalUserId, EUserPrivileges::Type Privilege, const FOnGetUserPrivilegeCompleteDelegate& Delegate, EShowPrivilegeResolveUI ShowResolveUI = EShowPrivilegeResolveUI::Default) = 0;

	/**
	 * Converts from an online unique id to a PlatformUserId used by application code
	 *
	 * @param UniqueNetId The unique id to look up
	 *
	 * @return The corresponding id or PLATFORMID_NONE if not found
	 */
	virtual FPlatformUserId GetPlatformUserIdFromUniqueNetId(const FUniqueNetId& UniqueNetId) const = 0;

	/**
	 * Converts from a local user num used by the online system to a PlatformUserId used by application code
	 *
	 * @param LocalUserNum the controller number of the associated user
	 *
	 * @return The corresponding id or PLATFORMID_NONE if invalid
	 */
	ONLINESUBSYSTEM_API virtual FPlatformUserId GetPlatformUserIdFromLocalUserNum(int32 LocalUserNum) const;

	/**
	 * Converts from a PlatformUserId used by application code to a local user num used by the online system
	 *
	 * @param PlatformUserId The application-level platform user id
	 *
	 * @return The corresponding user number of INDEX_NONE if invalid
	 */
	ONLINESUBSYSTEM_API virtual int32 GetLocalUserNumFromPlatformUserId(FPlatformUserId PlatformUserId) const;

	/**
	 * Get the auth type associated with accounts for this platform
	 *
	 * @return The auth type associated with accounts for this platform
	 */
	virtual FString GetAuthType() const = 0;
};

typedef TSharedPtr<IOnlineIdentity, ESPMode::ThreadSafe> IOnlineIdentityPtr;

ONLINESUBSYSTEM_API FString ToDebugString(IOnlineIdentity::EPrivilegeResults PrivilegeResult);

ONLINESUBSYSTEM_API FString ToDebugString(EUserPrivileges::Type UserPrivilege);

ONLINESUBSYSTEM_API FString ToDebugString(const FControllerPairingChangedUserInfo& ControllerPairingChangedUserInfo);
