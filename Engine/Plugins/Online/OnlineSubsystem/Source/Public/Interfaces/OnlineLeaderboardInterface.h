// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineDelegateMacros.h"
#include "OnlineStats.h"

ONLINESUBSYSTEM_API DECLARE_LOG_CATEGORY_EXTERN(LogOnlineLeaderboard, Log, All);

#define UE_LOG_ONLINE_LEADERBOARD(Verbosity, Format, ...) \
{ \
	UE_LOGF(LogOnlineLeaderboard, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

#define UE_CLOG_ONLINE_LEADERBOARD(Conditional, Verbosity, Format, ...) \
{ \
	UE_CLOGF(Conditional, LogOnlineLeaderboard, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

/**
 * Delegate called when the last leaderboard read is complete
 *
 * @param bWasSuccessful was the leaderboard read successful or not
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnLeaderboardReadComplete, bool);
typedef FOnLeaderboardReadComplete::FDelegate FOnLeaderboardReadCompleteDelegate;

/**
 * Delegate called when the stats flush operation has completed
 *
 * @param SessionName the name of the session having stats flushed
 * @param bWasSuccessful true if the async action completed without error, false if there was an error
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnLeaderboardFlushComplete, const FName, bool);
typedef FOnLeaderboardFlushComplete::FDelegate FOnLeaderboardFlushCompleteDelegate;

/**
 * Interface definition for the online services leaderboard services 
 */
class IOnlineLeaderboards
{
protected:
	IOnlineLeaderboards() {};

public:
	virtual ~IOnlineLeaderboards() = default;

	/**
	 * Reads a set of stats for the specified list of players
	 *
	 * @param Players the array of unique ids to read stats for
	 * @param ReadObject holds the definitions of the tables to read the data from and
	 *		  results are copied into the specified object
	 *
	 * @return true if the call is successful, false otherwise
	 */
	virtual bool ReadLeaderboards(const TArray< FUniqueNetIdRef >& Players, FOnlineLeaderboardReadRef& ReadObject) = 0;

	/**
	 * Reads a player's stats and all of that player's friends stats for the
	 * specified set of stat views. This allows you to easily compare a player's
	 * stats to their friends.
	 *
	 * @param LocalUserNum the local player having their stats and friend's stats read for
	 * @param ReadObject holds the definitions of the tables to read the data from and
	 *		  results are copied into the specified object
	 *
	 * @return true if the call is successful, false otherwise
	 */
	virtual bool ReadLeaderboardsForFriends(int32 LocalUserNum, FOnlineLeaderboardReadRef& ReadObject) = 0;

	/**
	* Reads stats for a given rank in the leaderboard, as well as a range of stats above and below that rank.
	* For example, if Rank = 100 and Range = 10, entries for ranks 90-110 will be returned.
	*
	* @param Rank the rank for which stats should be returned
	* @Param Range the range above and below Rank to return as well
	* @param ReadObject holds the definitions of the tables to read the data from and
	*		  results are copied into the specified object
	*
	* @return true if the call is successful, false otherwise
	*/
	virtual bool ReadLeaderboardsAroundRank(int32 Rank, uint32 Range, FOnlineLeaderboardReadRef& ReadObject) = 0;

	/**
	* Reads stats for a given player in the leaderboard, as well as a range of stats above and below that user.
	* For example, if the player is 100th in the leaderboard and Range = 10, entries for ranks 90-110 will be returned.
	*
	* @param Player the player for which stats should be returned
	* @Param Range the range above and below the player's rank to return as well
	* @param ReadObject holds the definitions of the tables to read the data from and
	*		  results are copied into the specified object
	*
	* @return true if the call is successful, false otherwise
	*/
	virtual bool ReadLeaderboardsAroundUser(FUniqueNetIdRef Player, uint32 Range, FOnlineLeaderboardReadRef& ReadObject) = 0;

	/**
	 * Notifies the interested party that the last stats read has completed
	 *
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 */
	DEFINE_ONLINE_DELEGATE_ONE_PARAM(OnLeaderboardReadComplete, bool);

	/**
	 * Cleans up any platform specific allocated data contained in the stats data
	 *
	 * @param ReadObject the object to handle per platform clean up on
	 */
	virtual void FreeStats(FOnlineLeaderboardRead& ReadObject) = 0;

	/**
	 * Writes out the stats contained within the stats write object to the online
	 * subsystem's cache of stats data. Note the new data replaces the old. 
	 * Stats cannot be written without a session or the write request is ignored.
	 * No more than 5 stats views can be written to at a time or the write request
	 * is ignored.
	 *
	 * @param SessionName the name of the session to write stats for
	 * @param Player the player to write stats for
	 * @param WriteObject the object containing the information to write
	 *
	 * @return true if the call is successful, false otherwise
	 */
	virtual bool WriteLeaderboards(const FName& SessionName, const FUniqueNetId& Player, FOnlineLeaderboardWrite& WriteObject) = 0;

	/**
	 * Commits any changes in the online stats cache to the permanent storage
	 *
	 * @param SessionName the name of the session that stats are being flushed for
	 *
	 * @return true if the call is successful, false otherwise
	 */
	virtual bool FlushLeaderboards(const FName& SessionName) = 0;

	/**
	 * Delegate called when the stats flush operation has completed
	 *
	 * @param SessionName the name of the session having stats flushed
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 */
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnLeaderboardFlushComplete, const FName&, bool);

	/**
	 * Writes the score data for the match, used for rankings/ratings
	 *
	 * @param SessionName the name of the session to write scores for
	 * @param LeaderboardId the leaderboard to write the score information to
	 * @param PlayerScores the list of players, teams, and scores they earned
	 *
	 * @return true if the call is successful, false otherwise
	 */
	virtual bool WriteOnlinePlayerRatings(const FName& SessionName, int32 LeaderboardId, const TArray<FOnlinePlayerScore>& PlayerScores) = 0;

};

typedef TSharedPtr<IOnlineLeaderboards, ESPMode::ThreadSafe> IOnlineLeaderboardsPtr;
