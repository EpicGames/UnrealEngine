// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CoreOnline.h"
#include "OnlineKeyValuePair.h"
#include "OnlineDelegateMacros.h"

#define UE_API ONLINESUBSYSTEM_API

class FJsonObject;

/** 
 * unique identifier for messages 
 */
typedef FUniqueNetId FUniqueMessageId;
typedef FUniqueNetIdRef FUniqueMessageIdRef;
typedef FUniqueNetIdPtr FUniqueMessageIdPtr;
typedef FUniqueNetIdSet FUniqueMessageIdSet;

/**
 * Message payload that stores key value pairs for variant type data
 */
class FOnlineMessagePayload
{
public:
	/** Max size of buffer when serializing payloads */
	static const int32 MaxPayloadSize = 4 * 1024;

	/**
	 * Constructor
	 */
	FOnlineMessagePayload()
	{
	}

	/**
	 * Convert the key value data to byte array
	 *
	 * @param OutBytes [out] array of bytes to serialize to
	 */
	UE_API void ToBytes(TArray<uint8>& OutBytes) const;

	/**
	 * Convert byte array to key value data 
	 *
	 * @param InBytes array of bytes to serialize from
	 */
	UE_API void FromBytes(const TArray<uint8>& InBytes);

	/**
	 * Convert key/val properties to json
	 *
	 * @param OutJsonObject resulting json object
	 */
	UE_API void ToJson(FJsonObject& OutJsonObject) const;

	/**
	 * Convert key/val properties to json str
	 *
	 * @return resulting json string
	 */
	UE_API FString ToJsonStr() const;

	/**
	 * Convert json to key/val properties
	 *
	 * @param JsonObject json object to convert
	 */
	UE_API void FromJson(const FJsonObject& JsonObject);

	/**
	 * Convert json string to key/val properties
	 *
	 * @param JsonStr json string to convert
	 */
	UE_API void FromJsonStr(const FString& JsonStr);

	/**
	 * Find an attribute by name and get its value
	 *
	 * @param AttrName name of attribute entry to find
	 * @param OutAttrValue [out] attribute value to write to
	 *
	 * @return true if attribute was found
	 */
	UE_API bool GetAttribute(const FString& AttrName, FVariantData& OutAttrValue) const;

	/**
	 * Set an attribute value by name
	 *
	 * @param AttrName name of attribute entry to set
	 * @param AttrValue attribute value to set
	 */
	UE_API void SetAttribute(const FString& AttrName, const FVariantData& AttrValue);

private:

	/** key value attributes to store variant type data */
	FOnlineKeyValuePairs<FString, FVariantData> KeyValData;
};

/**
 * Message header obtained via EnumerateMessages
 * Represents an inbox message that can be downloaded
 */
class FOnlineMessageHeader
{
public:

	/**
	 * Constructor
	 */
	FOnlineMessageHeader(const FUniqueNetIdRef& InFromUserId, const FUniqueMessageIdRef& InMessageId)
		: FromUserId(InFromUserId)
		, MessageId(InMessageId)
	{
	}

	/** Unique id of user that sent the message */
	FUniqueNetIdRef FromUserId;
	/** Name of user that sent the message */
	FString FromName;
	/** Unique id of the message. Needed to download the message payload */
	FUniqueMessageIdRef MessageId;
	/** Type of message */
	FString Type;
	/** UTC timestamp when message was sent */
	FString TimeStamp;
};

/**
 * Downloaded message obtained via passing message id to ReadMessage
 */
class FOnlineMessage
{
public:
	/**
	 * Constructor
	 */
	FOnlineMessage(FUniqueMessageIdRef InMessageId)
		: MessageId(InMessageId)
	{
	}

	/** Unique id of the message */
	FUniqueMessageIdRef MessageId;
	/** Payload containing the body of the message */
	FOnlineMessagePayload Payload;
};

/**
 * Delegate used when the enumeration of message headers has completed
 *
 * @param LocalUserNum the controller number of the associated user that made the request
 * @param bWasSuccessful true if the async action completed without error, false if there was an error
 * @param ErrorStr string representing the error condition
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnEnumerateMessagesComplete, int32, bool, const FString&);
typedef FOnEnumerateMessagesComplete::FDelegate FOnEnumerateMessagesCompleteDelegate;

/**
 * Delegate used when downloading of message contents has completed
 *
 * @param LocalUserNum the controller number of the associated user that made the request
 * @param bWasSuccessful true if the async action completed without error, false if there was an error
 * @param MessageId unique id of the message downloaded
 * @param ErrorStr string representing the error condition
 */
DECLARE_MULTICAST_DELEGATE_FourParams(FOnReadMessageComplete, int32, bool, const FUniqueMessageId&, const FString&);
typedef FOnReadMessageComplete::FDelegate FOnReadMessageCompleteDelegate;

/**
 * Send a message from the currently logged in user to a list of recipients
 *
 * @param LocalUserNum the controller number of the associated user that made the request
 * @param RecipientIds unique ids of users to send this message to
 * @param MessageType string representing the type of message
 * @param Payload the body/content of the message
 * 
 * @return true if request was started
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnSendMessageComplete, int32, bool, const FString&);
typedef FOnSendMessageComplete::FDelegate FOnSendMessageCompleteDelegate;

/**
 * Delegate used when deleting a message has completed
 *
 * @param LocalUserNum the controller number of the associated user that made the request
 * @param bWasSuccessful true if the async action completed without error, false if there was an error
 * @param MessageId unique id of the message deleted
 * @param ErrorStr string representing the error condition
 */
DECLARE_MULTICAST_DELEGATE_FourParams(FOnDeleteMessageComplete, int32, bool, const FUniqueMessageId&, const FString&);
typedef FOnDeleteMessageComplete::FDelegate FOnDeleteMessageCompleteDelegate;

/**
 * Interface class for enumerating/sending/receiving messages between users
 */
class IOnlineMessage
{

public:

	virtual ~IOnlineMessage() = default;

	/**
	 * Enumerate list of available message headers from user's inbox
	 *
	 * @param LocalUserNum the controller number of the associated user that made the request
	 * 
	 * @return true if request was started
	 */
	virtual bool EnumerateMessages(int32 LocalUserNum) = 0;
	
	/**
	 * Delegate used when the enumeration of message headers has completed
	 *
	 * @param LocalUserNum the controller number of the associated user that made the request
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 * @param ErrorStr string representing the error condition
	 */
	DEFINE_ONLINE_PLAYER_DELEGATE_TWO_PARAM(MAX_LOCAL_PLAYERS, OnEnumerateMessagesComplete, bool, const FString&);

	/**
	 * Get the cached list of message headers for a user. Cache is populated by calling EnumerateMessages
	 *
	 * @param LocalUserNum the controller number of the associated user that made the request
	 * @param OutHeaders [out] array of message headers that were found
	 *
	 * @return true if cached entry was found
	 */
	virtual bool GetMessageHeaders(int32 LocalUserNum, TArray< TSharedRef<class FOnlineMessageHeader> >& OutHeaders) = 0;

	/**
	 * Clear the cached list of message headers
	 *
	 * @param LocalUserNum the controller number of the associated user that made the request
	 *
	 * @return true if messages were clear, false if there was a problem
	 */
	virtual bool ClearMessageHeaders(int32 LocalUserNum) = 0;

	/**
	 * Download a message and its payload from user's inbox
	 *
	 * @param LocalUserNum the controller number of the associated user that made the request
	 * @param MessageId unique id of the message to download. Obtained from message header list
	 * 
	 * @return true if request was started
	 */
	virtual bool ReadMessage(int32 LocalUserNum, const FUniqueMessageId& MessageId) = 0;
	
	/**
	 * Delegate used when downloading of message contents has completed
	 *
	 * @param LocalUserNum the controller number of the associated user that made the request
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 * @param MessageId unique id of the message downloaded
	 * @param ErrorStr string representing the error condition
	 */
	DEFINE_ONLINE_PLAYER_DELEGATE_THREE_PARAM(MAX_LOCAL_PLAYERS, OnReadMessageComplete, bool, const FUniqueMessageId&, const FString&);

	/**
	 * Get the cached message and its contents for a user. Cache is populated by calling ReadMessage with a message id
	 *
	 * @param LocalUserNum the controller number of the associated user that made the request
	 * @param MessageId unique id of the message downloaded to find
	 *
	 * @return pointer to message entry if found or NULL
	 */
	virtual TSharedPtr<class FOnlineMessage> GetMessage(int32 LocalUserNum, const FUniqueMessageId& MessageId) = 0;

	/**
	 * Clear the given cached message
	 *
	 * @param LocalUserNum the controller number of the associated user that made the request
	 *
	 * @return true if messages were clear, false if there was a problem
	 */
	virtual bool ClearMessage(int32 LocalUserNum, const FUniqueMessageId& MessageId) = 0;

	/**
	 * Clear all the cached messages
	 *
	 * @param LocalUserNum the controller number of the associated user that made the request
	 *
	 * @return true if messages were clear, false if there was a problem
	 */
	virtual bool ClearMessages(int32 LocalUserNum) = 0;

	/**
	 * Send a message from the currently logged in user to a list of recipients
	 *
	 * @param LocalUserNum the controller number of the associated user that made the request
	 * @param RecipientIds unique ids of users to send this message to
	 * @param MessageType string representing the type of message
	 * @param Payload the body/content of the message
	 * 
	 * @return true if request was started
	 */
	virtual bool SendMessage(int32 LocalUserNum, const TArray< FUniqueNetIdRef >& RecipientIds, const FString& MessageType, const FOnlineMessagePayload& Payload) = 0;
	
	/**
	 * Delegate used when sending of message has completed
	 *
	 * @param LocalUserNum the controller number of the associated user that made the request
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 * @param ErrorStr string representing the error condition
	 */
	DEFINE_ONLINE_PLAYER_DELEGATE_TWO_PARAM(MAX_LOCAL_PLAYERS, OnSendMessageComplete, bool, const FString&);

	/**
	 * Delete a message from currently logged in user's inbox
	 *
	 * @param LocalUserNum the controller number of the associated user that made the request
	 * @param MessageId unique id of the message to delete. Obtained from message header list
	 * 
	 * @return true if request was started
	 */
	virtual bool DeleteMessage(int32 LocalUserNum, const FUniqueMessageId& MessageId) = 0;

	/**
	 * Delegate used when deleting a message has completed
	 *
	 * @param LocalUserNum the controller number of the associated user that made the request
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 * @param MessageId unique id of the message deleted
	 * @param ErrorStr string representing the error condition
	 */
	DEFINE_ONLINE_PLAYER_DELEGATE_THREE_PARAM(MAX_LOCAL_PLAYERS, OnDeleteMessageComplete, bool, const FUniqueMessageId&, const FString&);
};

#undef UE_API
