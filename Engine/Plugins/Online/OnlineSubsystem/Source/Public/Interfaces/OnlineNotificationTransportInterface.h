// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemTypes.h"

class IOnlineSubsystem;
struct FOnlineNotification;

// abstract base class for messages of the type understood by the specific transport mechanism, eg. xmpp
class IOnlineNotificationTransportMessage
{

};


/**
* Interface for notification transport mechanisms
*/
class IOnlineNotificationTransport
{
protected:
	/** Constructor */
	IOnlineNotificationTransport(IOnlineSubsystem* InOnlineSubsystemInstance, FNotificationTransportId InTransportId)
		: OnlineSubsystemInstance(InOnlineSubsystemInstance),Id(InTransportId)
	{
	}

	virtual ~IOnlineNotificationTransport() = default;

	/** The OSS associated with this transport, used for accessing the notification handler and transport manager */
	IOnlineSubsystem* OnlineSubsystemInstance;

	/** Unique notification transport id associated with this transport */
	FNotificationTransportId Id;

public:

	const FNotificationTransportId& GetNotificationTransportId() const
	{
		return Id;
	}

	/**
	* Equality operator
	*/
	bool operator==(const IOnlineNotificationTransport& Other) const
	{
		return Other.GetNotificationTransportId() == Id;
	}

	virtual const IOnlineNotificationTransportMessage* Convert(const FOnlineNotification& Notification) = 0;

	virtual const FOnlineNotification& Convert(const IOnlineNotificationTransportMessage* TransportMessage) = 0;

	/** Send a notification out using this transport mechanism */
	virtual bool SendNotification(const FOnlineNotification& Notification) = 0;

	/** Receive a transport-specific notification in from this transport mechanism and pass along to be delivered */
	virtual bool ReceiveNotification(const IOnlineNotificationTransportMessage& TransportMessage) = 0;
};

typedef TSharedPtr<IOnlineNotificationTransport, ESPMode::ThreadSafe> IOnlineNotificationTransportPtr;

class IOnlineSubsystem;
class FWildcardString;
struct FOnlineNotification;

class FOnlineTransportTapHandle
{
	friend class FOnlinePostmasterMcpTransporter;
public:
	FOnlineTransportTapHandle() {}

	explicit operator bool() const
	{
		return Handle != -1;
	}

	friend bool operator==(FOnlineTransportTapHandle Lhs, FOnlineTransportTapHandle Rhs)
	{
		return Lhs.Handle == Rhs.Handle;
	}

	friend bool operator!=(FOnlineTransportTapHandle Lhs, FOnlineTransportTapHandle Rhs)
	{
		return Lhs.Handle != Rhs.Handle;
	}

private:
	int Handle = -1;
};

DECLARE_DELEGATE_OneParam(FOnTapStateChanged, bool /* bSubscribed */);

/** A pattern used to open a tap and associated event handlers */
struct FOnlineTransportTap
{
	FString AddressPattern;
	FOnTapStateChanged StateChangeHandler;
};
