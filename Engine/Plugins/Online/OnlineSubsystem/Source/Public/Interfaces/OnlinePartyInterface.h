// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineKeyValuePair.h"
#include "OnlineSubsystemTypes.h"
#include "OnlineDelegateMacros.h"
#include "OnlineError.h"

#define UE_API ONLINESUBSYSTEM_API

typedef FString FChatRoomId;
struct FOnlineError;

ONLINESUBSYSTEM_API DECLARE_LOG_CATEGORY_EXTERN(LogOnlineParty, Log, All);
#define UE_LOG_ONLINE_PARTY(Verbosity, Format, ...) \
{ \
	UE_LOGF(LogOnlineParty, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

#define UE_CLOG_ONLINE_PARTY(Conditional, Verbosity, Format, ...) \
{ \
	UE_CLOGF(Conditional, LogOnlineParty, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

#define F_PREFIX(TypeToPrefix) F##TypeToPrefix
#define PARTY_DECLARE_DELEGATETYPE(Type) typedef F##Type::FDelegate F##Type##Delegate

ONLINESUBSYSTEM_API extern FName DefaultPartyDataNamespace;

enum class EAcceptPartyInvitationCompletionResult : int8;
enum class ECreatePartyCompletionResult : int8;
enum class EJoinPartyCompletionResult : int8;
enum class EKickMemberCompletionResult : int8;
enum class ELeavePartyCompletionResult : int8;
enum class EPromoteMemberCompletionResult : int8;
enum class ERejectPartyInvitationCompletionResult : int8;
enum class ERequestPartyInvitationCompletionResult : int8;
enum class ESendPartyInvitationCompletionResult : int8;
enum class EUpdateConfigCompletionResult : int8;
enum class EInvitationResponse : uint8;
enum class ERequestToJoinPartyCompletionResult : int8;

struct FQueryPartyJoinabilityResult;

struct FAnalyticsEventAttribute;

enum class EMemberConnectionStatus : uint8
{
	Uninitialized,
	Disconnected,
	Initializing,
	Connected
};

/**
 * Party member user info returned by IOnlineParty interface
 */
class FOnlinePartyMember
	: public FOnlineUser
{
public:
	EMemberConnectionStatus MemberConnectionStatus = EMemberConnectionStatus::Uninitialized;
	EMemberConnectionStatus PreviousMemberConnectionStatus = EMemberConnectionStatus::Uninitialized;

	/**
	 * Event when a party member's attribute has changed
	 * @see FOnlineUser::GetUserAttribute
	 * @param ChangedUserId id associated with this notification
	 * @param Attribute attribute that changed
	 * @param NewValue the new value for the attribute
	 * @param PreviousValue the previous value for the attribute
	 */
	DECLARE_EVENT_FourParams(FOnlinePartyMember, FOnMemberAttributeChanged, const FUniqueNetId& /*ChangedUserId*/, const FString& /*Attribute*/, const FString& /*NewValue*/, const FString& /*OldValue*/);
	FOnMemberAttributeChanged& OnMemberAttributeChanged() const { return OnMemberAttributeChangedEvent; }

	/**
	 * Event when a party member's connection status has changed
	 * @param ChangedUserId - id associated with this notification
	 * @param NewMemberConnectionStatus - new member data status
	 * @param PreviousMemberConnectionStatus - previous member data status
	 */
	DECLARE_EVENT_ThreeParams(FOnlinePartyMember, FOnMemberConnectionStatusChanged, const FUniqueNetId& /*ChangedUserId*/, const EMemberConnectionStatus /*NewMemberConnectionStatus*/, const EMemberConnectionStatus /*PreviousMemberConnectionStatus*/);
	FOnMemberConnectionStatusChanged& OnMemberConnectionStatusChanged() const { return OnMemberConnectionStatusChangedEvent; }

	void SetMemberConnectionStatus(EMemberConnectionStatus NewMemberConnectionStatus)
	{
		if (NewMemberConnectionStatus != MemberConnectionStatus)
		{
			PreviousMemberConnectionStatus = MemberConnectionStatus;
			MemberConnectionStatus = NewMemberConnectionStatus;
			OnMemberConnectionStatusChangedEvent.Broadcast(*GetUserId(), MemberConnectionStatus, PreviousMemberConnectionStatus);
		}
	}

private:
	/** Event fired when connection status changes */
	mutable FOnMemberConnectionStatusChanged OnMemberConnectionStatusChangedEvent;
	/** Event fired when an attribute changes */
	mutable FOnMemberAttributeChanged OnMemberAttributeChangedEvent;
};

typedef TSharedRef<const FOnlinePartyMember> FOnlinePartyMemberConstRef;
typedef TSharedPtr<const FOnlinePartyMember> FOnlinePartyMemberConstPtr;

/**
 * Data associated with the entire party
 */
class FOnlinePartyData
	: public TSharedFromThis<FOnlinePartyData>
{
public:
	FOnlinePartyData() = default;
	virtual ~FOnlinePartyData() = default;

	FOnlinePartyData(const FOnlinePartyData&) = default;
	FOnlinePartyData& operator=(const FOnlinePartyData&) = default;

	FOnlinePartyData(FOnlinePartyData&&) = default;
	FOnlinePartyData& operator=(FOnlinePartyData&&) = default;

	/**
	 * Equality operator
	 *
	 * @param Other the FOnlinePartyData to compare against
	 * @return true if considered equal, false otherwise
	 */
	UE_API bool operator==(const FOnlinePartyData& Other) const;
	/**
	 * Inequality operator
	 *
	 * @param Other the FOnlinePartyData to compare against
	 * @return true if considered not equal, false otherwise
	 */
	UE_API bool operator!=(const FOnlinePartyData& Other) const;

	/**
	 * Get an attribute from the party data
	 *
	 * @param AttrName - key for the attribute
	 * @param OutAttrValue - [out] value for the attribute if found
	 *
	 * @return true if the attribute was found
	 */
	bool GetAttribute(const FString& AttrName, FVariantData& OutAttrValue) const
	{
		bool bResult = false;

		const FVariantData* FoundValuePtr = KeyValAttrs.Find(AttrName);
		if (FoundValuePtr != nullptr)
		{
			OutAttrValue = *FoundValuePtr;
			return true;
		}

		return bResult;
	}

	/**
	 * Set an attribute from the party data
	 *
	 * @param AttrName - key for the attribute
	 * @param AttrValue - value to set the attribute to
	 */
	inline void SetAttribute(const FString& AttrName, const FVariantData& AttrValue)
	{
		SetAttribute(CopyTemp(AttrName), CopyTemp(AttrValue));
	}

	/**
	 * Set an attribute from the party data
	 *
	 * @param AttrName - key for the attribute
	 * @param AttrValue - value to set the attribute to
	 */
	virtual void SetAttribute(FString&& AttrName, FVariantData&& AttrValue)
	{
		FVariantData& NewAttrValue = KeyValAttrs.FindOrAdd(AttrName);
		if (NewAttrValue != AttrValue)
		{
			NewAttrValue = MoveTemp(AttrValue);
			DirtyKeys.Emplace(MoveTemp(AttrName));
		}
	}

	/**
	 * Remove an attribute from the party data
	 *
	 * @param AttrName - key for the attribute
	 */
	inline void RemoveAttribute(const FString& AttrName)
	{
		return RemoveAttribute(CopyTemp(AttrName));
	}

	/**
	 * Remove an attribute from the party data
	 *
	 * @param AttrName - key for the attribute
	 */
	virtual void RemoveAttribute(FString&& AttrName)
	{
		if (KeyValAttrs.Remove(AttrName) > 0)
		{
			DirtyKeys.Emplace(MoveTemp(AttrName));
		}
	}

	/**
	 * Mark an attribute as dirty so it can be rebroadcasted
	 *
	 * @param AttrName - key for the attribute to mark dirty
	 */
	virtual void MarkAttributeDirty(FString&& AttrName)
	{
		DirtyKeys.Emplace(MoveTemp(AttrName));
	}

	/**
	 * Check if there are any dirty keys
	 *
	 * @return true if there are any dirty keys
	 */
	bool HasDirtyKeys() const
	{
		return DirtyKeys.Num() > 0;
	}

	/**
	 * Get the dirty and removed key-value attributes
	 *
	 * @param OutDirtyAttrs the dirty attributes
	 * @param OutRemovedAttrs the removed attributes
	 */
	UE_API void GetDirtyKeyValAttrs(FOnlineKeyValuePairs<FString, FVariantData>& OutDirtyAttrs, TArray<FString>& OutRemovedAttrs) const;

	/**
	 * Clear the attributes map
	 */
	virtual void ClearAttributes()
	{
		KeyValAttrs.Empty();
		DirtyKeys.Empty();
	}

	/** 
	 * Clear the dirty keys set, called after successfully sending an update of the dirty elements
	 */
	virtual void ClearDirty()
	{
		DirtyKeys.Empty();
	}

	/** 
	 * Increment the stat tracking variables on packet sent
	 * 
	 * @param PacketSize - size of the packet generated
	 * @param NumRecipients - number of recipients the packet was sent to
	 * @param bIncrementRevision - this packet was a dirty packet update so we should increment the revision
	 */
	void OnPacketSent(int32 PacketSize, int32 NumRecipients, bool bIncrementRevision) const
	{
		TotalPackets++;
		TotalBytes += PacketSize;
		TotalEffectiveBytes += PacketSize * NumRecipients;
		if (bIncrementRevision)
		{
			++RevisionCount;
		}
	}

	/** 
	 * Generate a JSON packet containing all key-value attributes
	 * 
	 * @param JsonString - [out] string containing the resulting JSON output
	 */
	UE_API void ToJsonFull(FString& JsonString) const;
	
	/** 
	 * Generate a JSON packet containing only the dirty key-value attributes for a delta update
	 *
	 * @param JsonString - [out] string containing the resulting JSON output
	 */
	UE_API void ToJsonDirty(FString& JsonString) const;

	/**
	 * Create a JSON object containing all key-value attributes
	 * @return a JSON object containing all key-value attributes
	 */
	UE_API TSharedRef<FJsonObject> GetAllAttributesAsJsonObject() const;

	/**
	 * Create a string representing a JSON object containing all key-value attributes
	 * @return a string representing a JSON object containing all key-value attributes
	 */
	UE_API FString GetAllAttributesAsJsonObjectString() const;

	/** 
	 * Update attributes from a JSON packet
	 *
	 * @param JsonString - string containing the JSON packet
	 */
	UE_API void FromJson(const FString& JsonString);

	/** Accessor functions for KeyValAttrs map */
	FOnlineKeyValuePairs<FString, FVariantData>& GetKeyValAttrs() { return KeyValAttrs; }
	const FOnlineKeyValuePairs<FString, FVariantData>& GetKeyValAttrs() const { return KeyValAttrs; }

	/** Stat tracking variables */
	/** Total number of bytes generated by calls to ToJsonFull and ToJsonDirty */
	mutable int32 TotalBytes = 0;
	/** Total number of bytes generated by calls to ToJsonFull and ToJsonDirty, multiplied by the number of recipients the packet was sent to */
	mutable int32 TotalEffectiveBytes = 0;
	/** Total number of packets generated by calls to ToJsonFull and ToJsonDirty */
	mutable int32 TotalPackets = 0;

	/** Id representing number of updates sent, useful for determining if a client has missed an update */
	mutable int32 RevisionCount = 0;

private:
	/** map of key/val attributes that represents the data */
	FOnlineKeyValuePairs<FString, FVariantData> KeyValAttrs;

	/** set of which fields are dirty and need to transmitted */
	TSet<FString> DirtyKeys;
};

typedef TSharedRef<FOnlinePartyData> FOnlinePartyDataRef;
typedef TSharedPtr<FOnlinePartyData> FOnlinePartyDataPtr;
typedef TSharedRef<const FOnlinePartyData> FOnlinePartyDataConstRef;
typedef TSharedPtr<const FOnlinePartyData> FOnlinePartyDataConstPtr;

/**
 * Info about a user requesting to join a party
 */
class IOnlinePartyUserPendingJoinRequestInfo
{
public:
	virtual ~IOnlinePartyUserPendingJoinRequestInfo() = default;

	/**
	 * Get the id of the user requesting to join
	 * @return the id of the user requesting to join
	 */
	virtual const FUniqueNetIdRef& GetUserId() const = 0;

	/**
	 * Get the display name of the user requesting to join
	 * @return the display name of the user requesting to join
	 */
	virtual const FString& GetDisplayName() const = 0;

	/**
	 * Get the platform of the user requesting to join
	 * @return the platform of the user requesting to join
	 */
	virtual const FString& GetPlatform() const = 0;

	/**
	 * Get the join data of the user requesting to join
	 * @return the join data of the user requesting to join
	 */
	virtual TSharedRef<const FOnlinePartyData> GetJoinData() const = 0;
};

typedef TSharedRef<const IOnlinePartyUserPendingJoinRequestInfo> IOnlinePartyUserPendingJoinRequestInfoConstRef;
typedef TSharedPtr<const IOnlinePartyUserPendingJoinRequestInfo> IOnlinePartyUserPendingJoinRequestInfoConstPtr;

/**
 * Info about a group of users requesting to join a party
 */
class IOnlinePartyPendingJoinRequestInfo
	: public TSharedFromThis<IOnlinePartyPendingJoinRequestInfo>
{
public:
	IOnlinePartyPendingJoinRequestInfo() = default;
	virtual ~IOnlinePartyPendingJoinRequestInfo() = default;

	/**
	 * Get the primary user's id
	 * @return id of the primary user of this join request
	 */
	virtual const FUniqueNetIdRef& GetSenderId() const
	{
		TArray<IOnlinePartyUserPendingJoinRequestInfoConstRef> Users;
		GetUsers(Users);
		check(Users.Num() > 0);
		return Users[0]->GetUserId();
	}

	/**
	 * Get the primary user's display name
	 * @return display name of the primary user of this join request
	 */
	virtual const FString& GetSenderDisplayName() const
	{
		TArray<IOnlinePartyUserPendingJoinRequestInfoConstRef> Users;
		GetUsers(Users);
		check(Users.Num() > 0);
		return Users[0]->GetDisplayName();
	}

	/**
	 * Get the primary user's platform
	 * @return platform of the primary user of this join request
	 */
	virtual const FString& GetSenderPlatform() const
	{
		TArray<IOnlinePartyUserPendingJoinRequestInfoConstRef> Users;
		GetUsers(Users);
		check(Users.Num() > 0);
		return Users[0]->GetPlatform();
	}

	/**
	 * Get the primary user's join data
	 * @return join data provided by the primary user for this join request
	 */
	virtual TSharedRef<const FOnlinePartyData> GetSenderJoinData() const
	{
		TArray<IOnlinePartyUserPendingJoinRequestInfoConstRef> Users;
		GetUsers(Users);
		check(Users.Num() > 0);
		return Users[0]->GetJoinData();
	}

	/**
	 * Get the list of users requesting to join
	 * @return array of users requesting to join
	 */
	virtual void GetUsers(TArray<IOnlinePartyUserPendingJoinRequestInfoConstRef>& OutUsers) const = 0;
};

typedef TSharedRef<const IOnlinePartyPendingJoinRequestInfo> IOnlinePartyPendingJoinRequestInfoConstRef;
typedef TSharedPtr<const IOnlinePartyPendingJoinRequestInfo> IOnlinePartyPendingJoinRequestInfoConstPtr;

/**
 * Info needed to join a party
 */
class IOnlinePartyJoinInfo
	: public TSharedFromThis<IOnlinePartyJoinInfo>
{
public:
	IOnlinePartyJoinInfo() {}
	virtual ~IOnlinePartyJoinInfo() = default;

	virtual bool IsValid() const = 0;

	/**
	 * @return party id of party associated with this join invite
	 */
	virtual TSharedRef<const FOnlinePartyId> GetPartyId() const = 0;

	/**
	 * @return party id of party associated with this join invite
	 */
	virtual FOnlinePartyTypeId GetPartyTypeId() const = 0;

	/**
	 * @return user id of where this join info came from
	 */
	virtual FUniqueNetIdRef GetSourceUserId() const = 0;

	/**
	 * @return user id of where this join info came from
	 */
	virtual const FString& GetSourceDisplayName() const = 0;

	/** 
	 * @return source platform string
	 */
	virtual const FString& GetSourcePlatform() const = 0;

	/**
	 * @return the PlatformData included in the join info
	 */
	virtual const FString& GetPlatformData() const = 0;

	/**
	 * @return true if the join info has some form of key(does not guarantee the validity of that key)
	 */
	virtual bool HasKey() const = 0;

	/**
	 * @return true if a password can be used to bypass generated access key
     */
	virtual bool HasPassword() const = 0;

	/**
	 * @return true if the party is known to be accepting members
     */
	virtual bool IsAcceptingMembers() const = 0;

	/**
	 * @return true if this is a party of one
	 */
	virtual bool IsPartyOfOne() const = 0;

	/**
	 * @return why the party is not accepting members
	 */
	virtual int32 GetNotAcceptingReason() const = 0;

	/**
	 * @return id of the client app associated with the sender of the party invite
	 */
	virtual const FString& GetAppId() const = 0;

	/**
	* @return id of the build associated with the sender of the party invite
	*/
	virtual const FString& GetBuildId() const = 0;

	/**
	 * @return whether or not the join info can be used to join
	 */
	virtual bool CanJoin() const = 0;

	/**
	 * @return whether or not the join info can be used to join with a password
	 */
	virtual bool CanJoinWithPassword() const = 0;

	/**
	 * @return whether or not the join info has the info to request an invite
	 */
	virtual bool CanRequestAnInvite() const = 0;

	UE_API virtual FString ToDebugString() const;

};

typedef TSharedRef<const IOnlinePartyJoinInfo> IOnlinePartyJoinInfoConstRef;
typedef TSharedPtr<const IOnlinePartyJoinInfo> IOnlinePartyJoinInfoConstPtr;

/**
 * Permissions for party features
 */
namespace PartySystemPermissions
{
	/**
	 * Who has permissions to perform party actions
	 */
	enum class EPermissionType : uint8
	{
		/** Noone has access to do that action */
		Noone,
		/** Available to the leader only */
		Leader,
		/** Available to the leader and friends of the leader only */
		Friends,
		/** Available to anyone */
		Anyone
	};
}

enum class EJoinRequestAction : uint8
{
	Manual,
	AutoApprove,
	AutoReject
};
/**
 * Options for configuring a new party or for updating an existing party
 */
struct FPartyConfiguration
	: public TSharedFromThis<FPartyConfiguration>
{
	FPartyConfiguration()
		: JoinRequestAction(EJoinRequestAction::Manual)
		, PresencePermissions(PartySystemPermissions::EPermissionType::Anyone)
		, InvitePermissions(PartySystemPermissions::EPermissionType::Leader)
		, bChatEnabled(true)
		, bShouldRemoveOnDisconnection(false)
		, bIsAcceptingMembers(false)
		, NotAcceptingMembersReason(0)
		, MaxMembers(0)
	{}

	/**
	 * Equality operator
	 *
	 * @param Other the FPartyConfiguration to compare against
	 * @return true if considered equal, false otherwise
	 */
	UE_API bool operator==(const FPartyConfiguration& Other) const;
	/**
	 * Inequality operator
	 *
	 * @param Other the FPartyConfiguration to compare against
	 * @return true if considered not equal, false otherwise
	 */
	UE_API bool operator!=(const FPartyConfiguration& Other) const;

	/** should publish info to presence */
	EJoinRequestAction JoinRequestAction;
	/** Permission for how the party can be  */
	PartySystemPermissions::EPermissionType PresencePermissions;
	/** Permission who can send invites */
	PartySystemPermissions::EPermissionType InvitePermissions;
	/** should have a muc room */
	bool bChatEnabled;
	/** should remove on disconnection */
	bool bShouldRemoveOnDisconnection;
	/** is accepting members */
	bool bIsAcceptingMembers;
	/** not accepting members reason */
	int32 NotAcceptingMembersReason;
	/** Maximum active members allowed. 0 means no maximum. */
	int32 MaxMembers;
	/** Human readable nickname */
	FString Nickname;
	/** Human readable description */
	FString Description;
	/** Human readable password for party. */
	FString Password;
};

typedef TSharedRef<const FPartyConfiguration> FPartyConfigurationConstRef;

enum class EPartyState : uint8
{
	None,
	CreatePending,
	JoinPending,
	LeavePending,
	Active,
	Disconnected,
	CleanUp
};

/**
 * Current state associated with a party
 */
class FOnlineParty
	: public TSharedFromThis<FOnlineParty>
{
	FOnlineParty() = delete;
protected:
		FOnlineParty(const TSharedRef<const FOnlinePartyId>& InPartyId, const FOnlinePartyTypeId InPartyTypeId)
		: PartyId(InPartyId)
		, PartyTypeId(InPartyTypeId)
		, State(EPartyState::None)
		, PreviousState(EPartyState::None)
	{}

public:
	virtual ~FOnlineParty() = default;

	/**
	 * Check if the local user has invite permissions in this party. Based on configuration permissions and party state.
	 *
	 * @param LocalUserId the local user's id
	 * @return true if the local user can invite, false if not
	 */
	virtual bool CanLocalUserInvite(const FUniqueNetId& LocalUserId) const = 0;

	/**
	 * Is this party joinable?
	 *
	 * @return true if this party is joinable, false if not
	 */
	virtual bool IsJoinable() const = 0;
	virtual void SetState(EPartyState InState)
	{
		PreviousState = State;
		State = InState;
	}

	/**
	 * Get the party's configuration
	 *
	 * @return the party's configuration
	 */
	virtual TSharedRef<const FPartyConfiguration> GetConfiguration() const = 0;

	/** Unique id of the party */
	TSharedRef<const FOnlinePartyId> PartyId;
	/** Type of party (e.g., Primary) */
	const FOnlinePartyTypeId PartyTypeId;
	/** Unique id of the leader */
	FUniqueNetIdPtr LeaderId;
	/** The current state of the party */
	EPartyState State;
	/** The previous state of the party */
	EPartyState PreviousState;
	/** id of chat room associated with the party */
	FChatRoomId RoomId;
};

typedef TSharedRef<const FOnlineParty> FOnlinePartyConstRef;
typedef TSharedPtr<const FOnlineParty> FOnlinePartyConstPtr;

enum class EMemberExitedReason : uint8
{
	Unknown,
	Left,
	Removed,
	Kicked
};

enum class EPartyInvitationRemovedReason : uint8
{
	/** Unknown or undefined reason */
	Unknown,
	/** User accepted the invitation */
	Accepted,
	/** User declined the invitation */
	Declined,
	/** ClearInvitations was called, the invitation should no longer be displayed */
	Cleared,
	/** Expired */
	Expired,
	/** Became invalid (for example, party was destroyed) */
	Invalidated,
};

/** Recipient information for SendInvitation */
struct FPartyInvitationRecipient
{
	/** Constructor */
	FPartyInvitationRecipient(const FUniqueNetIdRef& InId)
		: Id(InId)
	{}

	/** Constructor */
	FPartyInvitationRecipient(const FUniqueNetId& InId)
		: Id(InId.AsShared())
	{}

	/** Constructor */
	FPartyInvitationRecipient(const FUniqueNetId& InId, const FString& InMetaData)
		: Id(InId.AsShared())
		, PlatformData(InMetaData)
	{}

	/** Id of the user to send the invitation to */
	FUniqueNetIdRef Id;
	/** Additional data to provide context for the invitee */
	FString PlatformData;

	/** Get a string representation suitable for logging */
	FString ONLINESUBSYSTEM_API ToDebugString() const;
};

enum class EPartySystemState : uint8
{
	Initializing = 0,
	Initialized,
	RequestingShutdown,
	ShutDown,
};

enum class EPartyRequestToJoinRemovedReason : uint8
{
	/** Unknown or undefined reason */
	Unknown,
	/** Cancelled */
	Cancelled,
	/** Expired */
	Expired,
	/** Dismissed */
	Dismissed,
	/** Accepted */
	Accepted,
};

/**
 * Info about a request to join a local user's party
 */
class IOnlinePartyRequestToJoinInfo
	: public TSharedFromThis<IOnlinePartyRequestToJoinInfo>
{
public:
	IOnlinePartyRequestToJoinInfo() = default;
	virtual ~IOnlinePartyRequestToJoinInfo() = default;

	/**
	 * Get the id of the user requesting to join
	 * @return the id of the user requesting to join
	 */
	virtual const FUniqueNetIdPtr GetUserId() const = 0;

	/**
	 * Get the display name of the user requesting to join
	 * @return the display name of the user requesting to join
	 */
	virtual const FString& GetDisplayName() const = 0;

	/**
	 * Get the platform data associated with this request
	 * @return the platform data associated with this request
	 */
	virtual const FString& GetPlatformData() const = 0;

	/**
	 * Get the expiration time of the request to join
	 * @return the expiration time of the request to join
	 */
	virtual const FDateTime& GetExpirationTime() const = 0;

	/**
	 * Get the GetPartyTypeId of the request to join
	 * @return the GetPartyTypeId of the request to join
	 */
	virtual const FOnlinePartyTypeId GetPartyTypeId() const = 0;
};

typedef TSharedRef<const IOnlinePartyRequestToJoinInfo> IOnlinePartyRequestToJoinInfoConstRef;
typedef TSharedPtr<const IOnlinePartyRequestToJoinInfo> IOnlinePartyRequestToJoinInfoConstPtr;


///////////////////////////////////////////////////////////////////
// Completion delegates
///////////////////////////////////////////////////////////////////
/**
 * Restore parties async task completed callback
 *
 * @param LocalUserId id of user that initiated the request
 * @param Result Result of the operation
 */
DECLARE_DELEGATE_TwoParams(FOnRestorePartiesComplete, const FUniqueNetId& /*LocalUserId*/, const FOnlineError& /*Result*/);
/**
 * Restore invites async task completed callback
 *
 * @param LocalUserId id of user that initiated the request
 * @param Result Result of the operation
 */
DECLARE_DELEGATE_TwoParams(FOnRestoreInvitesComplete, const FUniqueNetId& /*LocalUserId*/, const FOnlineError& /*Result*/);
/**
 * Cleanup parties async task completed callback
 *
 * @param LocalUserId id of user that initiated the request
 * @param Result Result of the operation
 */
DECLARE_DELEGATE_TwoParams(FOnCleanupPartiesComplete, const FUniqueNetId& /*LocalUserId*/, const FOnlineError& /*Result*/);
/**
 * Party creation async task completed callback
 *
 * @param LocalUserId - id of user that initiated the request
 * @param PartyId - id associated with the party
 * @param Result - result of the operation
 */
DECLARE_DELEGATE_ThreeParams(FOnCreatePartyComplete, const FUniqueNetId& /*LocalUserId*/, const TSharedPtr<const FOnlinePartyId>& /*PartyId*/, const ECreatePartyCompletionResult /*Result*/);
/**
 * Party join async task completed callback
 *
 * @param LocalUserId - id of user that initiated the request
 * @param PartyId - id associated with the party
 * @param Result - result of the operation
 * @param NotApprovedReason - client defined value describing why you were not approved
 */
DECLARE_DELEGATE_FourParams(FOnJoinPartyComplete, const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const EJoinPartyCompletionResult /*Result*/, const int32 /*NotApprovedReason*/);
/**
 * Party query joinability async task completed callback
 *
 * @param LocalUserId - id of user that initiated the request
 * @param PartyId - id associated with the party
 * @param Result - result of the operation
 */
DECLARE_DELEGATE_ThreeParams(FOnQueryPartyJoinabilityCompleteEx, const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const FQueryPartyJoinabilityResult& /*Result*/);
/**
 * Party leave async task completed callback
 *
 * @param LocalUserId - id of user that initiated the request
 * @param PartyId - id associated with the party
 * @param Result - result of the operation
 */
DECLARE_DELEGATE_ThreeParams(FOnLeavePartyComplete, const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const ELeavePartyCompletionResult /*Result*/);
/**
 * Party update async task completed callback
 *
 * @param LocalUserId - id of user that initiated the request
 * @param PartyId - id associated with the party
 * @param Result - result of the operation
 */
DECLARE_DELEGATE_ThreeParams(FOnUpdatePartyComplete, const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const EUpdateConfigCompletionResult /*Result*/);
/**
 * Party update async task completed callback
 *
 * @param LocalUserId - id of user that initiated the request
 * @param PartyId - id associated with the party
 * @param Result - result of the operation
 */
DECLARE_DELEGATE_ThreeParams(FOnRequestPartyInvitationComplete, const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const ERequestPartyInvitationCompletionResult /*Result*/);
/**
 * Party invitation sent completed callback
 *
 * @param LocalUserId - id of user that initiated the request
 * @param PartyId - id associated with the party
 * @param RecipientId - user invite was sent to
 * @param Result - result of the send operation
 */
DECLARE_DELEGATE_FourParams(FOnSendPartyInvitationComplete, const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const FUniqueNetId& /*RecipientId*/, const ESendPartyInvitationCompletionResult /*Result*/);
/**
 * Party invitation cancel completed callback
 *
 * @param SenderUserId - id of user that initiated the request
 * @param PartyId - id associated with the party
 * @param TargetUserId - user invite was sent to
 * @param Result - result of the cancel operation
 */
DECLARE_DELEGATE_FourParams(FOnCancelPartyInvitationComplete, const FUniqueNetId& /*SenderUserId*/, const FOnlinePartyId& /*PartyId*/, const FUniqueNetId& /*TargetUserId*/, const FOnlineError& /*Result*/);
/**
 * Accepting an invite to a user to join party async task completed callback
 *
 * @param LocalUserId - id of user that initiated the request
 * @param PartyId - id associated with the party
 * @param Result - string with error info if any
 */
DECLARE_DELEGATE_ThreeParams(FOnAcceptPartyInvitationComplete, const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const EAcceptPartyInvitationCompletionResult /*Result*/);
/**
 * Rejecting an invite to a user to join party async task completed callback
 *
 * @param LocalUserId - id of user that initiated the request
 * @param bWasSuccessful - true if successfully sent invite
 * @param PartyId - id associated with the party
 * @param Result - string with error info if any
 */
DECLARE_DELEGATE_ThreeParams(FOnRejectPartyInvitationComplete, const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const ERejectPartyInvitationCompletionResult /*Result*/);
/**
 * Kicking a member of a party async task completed callback
 *
 * @param LocalUserId - id of user that initiated the request
 * @param PartyId - id associated with the party
 * @param MemberId - id of member being kicked
 * @param Result - string with error info if any
 */
DECLARE_DELEGATE_FourParams(FOnKickPartyMemberComplete, const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const FUniqueNetId& /*MemberId*/, const EKickMemberCompletionResult /*Result*/);
/**
 * Promoting a member of a party async task completed callback
 *
 * @param LocalUserId - id of user that initiated the request
 * @param PartyId - id associated with the party
 * @param MemberId - id of member being promoted to leader
 * @param Result - string with error info if any
 */
DECLARE_DELEGATE_FourParams(FOnPromotePartyMemberComplete, const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const FUniqueNetId& /*MemberId*/, const EPromoteMemberCompletionResult /*Result*/);
/**
 * Request to join party async task completed callback
 *
 * @param LocalUserId - id of user that initiated the request
 * @param PartyLeaderId - id of party leader receiving the request
 * @param ExpiresAt - if successful, the time the request expires
 * @param Result - result of the operation
 */
DECLARE_DELEGATE_FourParams(FOnRequestToJoinPartyComplete, const FUniqueNetId& /*LocalUserId*/, const FUniqueNetId& /*PartyLeaderId*/, const FDateTime& /*ExpiresAt*/, const ERequestToJoinPartyCompletionResult /*Result*/);





///////////////////////////////////////////////////////////////////
// Notification delegates
///////////////////////////////////////////////////////////////////

/**
* notification when a party is joined
* @param LocalUserId - id associated with this notification
* @param PartyId - id associated with the party
*/
DECLARE_MULTICAST_DELEGATE_TwoParams(F_PREFIX(OnPartyJoined), const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/);
PARTY_DECLARE_DELEGATETYPE(OnPartyJoined);

/**
 * notification when a party is joined
 * @param LocalUserId - id associated with this notification
 * @param PartyId - id associated with the party
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(F_PREFIX(OnPartyExited), const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/);
PARTY_DECLARE_DELEGATETYPE(OnPartyExited);

/**
 * Notification when a party's state has changed
 * @param LocalUserId - id associated with this notification
 * @param PartyId - id associated with the party
 * @param State - state of the party
 * @param PreviousState - previous state of the party
 */
DECLARE_MULTICAST_DELEGATE_FourParams(F_PREFIX(OnPartyStateChanged), const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, EPartyState /*State*/, EPartyState /*PreviousState*/);
PARTY_DECLARE_DELEGATETYPE(OnPartyStateChanged);

/**
 * Notification when player promotion is locked out.
 * @param LocalUserId - id associated with this notification
 * @param PartyId - id associated with the party
 * @param bLockoutState - if promotion is currently locked out
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(F_PREFIX(OnPartyPromotionLockoutChanged), const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const bool /*bLockoutState*/);
PARTY_DECLARE_DELEGATETYPE(OnPartyPromotionLockoutChanged);

/**
 * Notification when party config is updated
 * @param LocalUserId - id associated with this notification
 * @param PartyId - id associated with the party
 * @param PartyConfig - party whose config was updated
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(F_PREFIX(OnPartyConfigChanged), const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const FPartyConfiguration& /*PartyConfig*/);
PARTY_DECLARE_DELEGATETYPE(OnPartyConfigChanged);

/**
 * Notification when party data is updated
 * @param LocalUserId - id associated with this notification
 * @param PartyId - id associated with the party
 * @param Namespace - namespace the party data is associated with
 * @param PartyData - party data that was updated
 */
DECLARE_MULTICAST_DELEGATE_FourParams(F_PREFIX(OnPartyDataReceived), const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const FName& /*Namespace*/, const FOnlinePartyData& /*PartyData*/);
PARTY_DECLARE_DELEGATETYPE(OnPartyDataReceived);

/**
 * Notification when a member is promoted in a party
 * @param LocalUserId - id associated with this notification
 * @param PartyId - id associated with the party
 * @param NewLeaderId - id of member that was promoted
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(F_PREFIX(OnPartyMemberPromoted), const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const FUniqueNetId& /*NewLeaderId*/);
PARTY_DECLARE_DELEGATETYPE(OnPartyMemberPromoted);

/**
 * Notification when a member exits a party
 * @param LocalUserId - id associated with this notification
 * @param PartyId - id associated with the party
 * @param MemberId - id of member that joined
 * @param Reason - why the member was removed
 */
DECLARE_MULTICAST_DELEGATE_FourParams(F_PREFIX(OnPartyMemberExited), const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const FUniqueNetId& /*MemberId*/, const EMemberExitedReason /*Reason*/);
PARTY_DECLARE_DELEGATETYPE(OnPartyMemberExited);

/**
 * Notification when a member joins the party
 * @param LocalUserId - id associated with this notification
 * @param PartyId - id associated with the party
 * @param MemberId - id of member that joined
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(F_PREFIX(OnPartyMemberJoined), const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const FUniqueNetId& /*MemberId*/);
PARTY_DECLARE_DELEGATETYPE(OnPartyMemberJoined);

/**
 * Notification when party member data is updated
 * @param LocalUserId - id associated with this notification
 * @param PartyId - id associated with the party
 * @param MemberId - id of member that had updated data
 * @param Namespace - namespace the party data is associated with
 * @param PartyMemberData - party member data that was updated
 */
DECLARE_MULTICAST_DELEGATE_FiveParams(F_PREFIX(OnPartyMemberDataReceived), const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const FUniqueNetId& /*MemberId*/, const FName& /*Namespace*/, const FOnlinePartyData& /*PartyMemberData*/);
PARTY_DECLARE_DELEGATETYPE(OnPartyMemberDataReceived);

/**
 * Notification when an invite list has changed for a party the user is in
 * @param LocalUserId - user that is associated with this notification
 */
DECLARE_MULTICAST_DELEGATE_OneParam(F_PREFIX(OnPartyInvitesChanged), const FUniqueNetId& /*LocalUserId*/);
PARTY_DECLARE_DELEGATETYPE(OnPartyInvitesChanged);

/**
 * Notification when a request for an invite has been received
 * @param LocalUserId id associated with this notification
 * @param PartyId id associated with the party
 * @param SenderId id of user that sent the invite
 * @param RequestForId id of user that sender is requesting the invite for - invalid if the sender is requesting the invite
 */
DECLARE_MULTICAST_DELEGATE_FourParams(F_PREFIX(OnPartyInviteRequestReceived), const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const FUniqueNetId& /*SenderId*/, const FUniqueNetId& /*RequestForId*/);
PARTY_DECLARE_DELEGATETYPE(OnPartyInviteRequestReceived);

/**
 * Notification when a new invite is received
 * @param LocalUserId id associated with this notification
 * @param Invitation the invitation that was received
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(F_PREFIX(OnPartyInviteReceivedEx), const FUniqueNetId& /*LocalUserId*/, const IOnlinePartyJoinInfo& /*Invitation*/);
PARTY_DECLARE_DELEGATETYPE(OnPartyInviteReceivedEx);

/**
 * Notification when an invite has been removed
 * @param LocalUserId id associated with this notification
 * @param Invitation the invitation that was removed
 * @param Reason reason the invite has been removed
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(F_PREFIX(OnPartyInviteRemovedEx), const FUniqueNetId& /*LocalUserId*/, const IOnlinePartyJoinInfo& /*Invitation*/, EPartyInvitationRemovedReason /*Reason*/);
PARTY_DECLARE_DELEGATETYPE(OnPartyInviteRemovedEx);

/**
 * Notification when a new invite is received
 * @param LocalUserId id associated with this notification
 * @param PartyId id associated with the party
 * @param SenderId id of user that sent the invite
 * @param bWasAccepted whether or not the invite was accepted
 */
DECLARE_MULTICAST_DELEGATE_FourParams(F_PREFIX(OnPartyInviteResponseReceived), const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const FUniqueNetId& /*SenderId*/, const EInvitationResponse /*Response*/);
PARTY_DECLARE_DELEGATETYPE(OnPartyInviteResponseReceived);

/**
 * Notification when a new reservation request is received
 * @param LocalUserId - id associated with this notification
 * @param PartyId - id associated with the party
 * @param JoinRequestInfo - data about users that are joining
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(F_PREFIX(OnPartyJoinRequestReceived), const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const IOnlinePartyPendingJoinRequestInfo& /*JoinRequestInfo*/);
PARTY_DECLARE_DELEGATETYPE(OnPartyJoinRequestReceived);

/**
 * Notification when a player wants to know if the party is in a joinable state
 * @param LocalUserId - id associated with this notification
 * @param PartyId - id associated with the party
 * @param JoinRequestInfo - data about users that are joining
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(F_PREFIX(OnQueryPartyJoinabilityReceived), const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const IOnlinePartyPendingJoinRequestInfo& /*JoinRequestInfo*/);
PARTY_DECLARE_DELEGATETYPE(OnQueryPartyJoinabilityReceived);

/**
 * Request for the game to fill in data to be sent with the join request for the leader to make an informed decision based on the joiner's state
 * @param LocalUserId - id associated with this notification
 * @param PartyId - id associated with the party
 * @param PartyData - data for the game to populate
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(F_PREFIX(OnFillPartyJoinRequestData), const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, FOnlinePartyData& /*PartyData*/);
PARTY_DECLARE_DELEGATETYPE(OnFillPartyJoinRequestData);

/**
 * Notification when an analytics event needs to be recorded
 * @param LocalUserId - id associated with this notification
 * @param PartyId - id associated with the party
 * @param PartyData - data for the game to populate
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(F_PREFIX(OnPartyAnalyticsEvent), const FUniqueNetId& /*LocalUserId*/, const FString& /*EventName*/, const TArray<FAnalyticsEventAttribute>& /*Attributes*/);
PARTY_DECLARE_DELEGATETYPE(OnPartyAnalyticsEvent);

/**
* Notification of party system state change
* @param NewState - new state this partysystem is in
*/
DECLARE_MULTICAST_DELEGATE_OneParam(F_PREFIX(OnPartySystemStateChange), EPartySystemState /*NewState*/);
PARTY_DECLARE_DELEGATETYPE(OnPartySystemStateChange);

/**
* Notification when a new party join request is received
* @param LocalUserId - id associated with this notification
* @param PartyId - id associated with the party
* @param RequesterId - id of player who requested to join
* @param Request - information regarding the request
*/
DECLARE_MULTICAST_DELEGATE_FourParams(F_PREFIX(OnPartyRequestToJoinReceived), const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const FUniqueNetId& /*RequesterId*/, const IOnlinePartyRequestToJoinInfo& /*Request*/);
PARTY_DECLARE_DELEGATETYPE(OnPartyRequestToJoinReceived);

/**
 * Notification when a request to join has been removed
 * @param LocalUserId id associated with this notification
 * @param PartyId id associated with the party
 * @param RequesterId - id of player who requested to join
 * @param Request - information regarding the request
 * @param Reason reason the request has been removed
 */
DECLARE_MULTICAST_DELEGATE_FiveParams(F_PREFIX(OnPartyRequestToJoinRemoved), const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const FUniqueNetId& /*RequesterId*/, const IOnlinePartyRequestToJoinInfo& /*Request*/, EPartyRequestToJoinRemovedReason /*Reason*/);
PARTY_DECLARE_DELEGATETYPE(OnPartyRequestToJoinRemoved);

/**
 * Interface definition for the online party services 
 * Allows for forming a party and communicating with party members
 */
class IOnlinePartySystem
{
protected:
	IOnlinePartySystem() = default;

public:
	virtual ~IOnlinePartySystem() = default;

	/**
	 * Restore party memberships. Intended to be called once during login to restore state from other running instances.
	 *
	 * @param LocalUserId the user to restore the party membership for
	 * @param CompletionDelegate the delegate to trigger on completion
	 */
	virtual void RestoreParties(const FUniqueNetId& LocalUserId, const FOnRestorePartiesComplete& CompletionDelegate) = 0;

	/**
	 * Restore party invites. Intended to be called once during login to restore state from other running instances.
	 *
	 * @param LocalUserId the user to restore the pings for
	 * @param CompletionDelegate the delegate to trigger on completion
	 */
	virtual void RestoreInvites(const FUniqueNetId& LocalUserId, const FOnRestoreInvitesComplete& CompletionDelegate) = 0;
	
	/**
	 * Cleanup party state. This will cleanup the local party state and attempt to cleanup party memberships on an external service if possible.  Intended to be called for development purposes.
	 *
	 * @param LocalUserId the user to cleanup the parties for
	 * @param CompletionDelegate the delegate to trigger on completion
	 */
	virtual void CleanupParties(const FUniqueNetId& LocalUserId, const FOnCleanupPartiesComplete& CompletionDelegate) = 0;
	
	/**
	 * Create a new party
	 *
	 * @param LocalUserId - user making the request
	 * @param PartyConfig - configuration for the party (can be updated later)
	 * @param Delegate - called on completion
	 * @param UserRoomId - this forces the name of the room to be this value
	 *
	 * @return true if task was started
	 */
	virtual bool CreateParty(const FUniqueNetId& LocalUserId, const FOnlinePartyTypeId PartyTypeId, const FPartyConfiguration& PartyConfig, const FOnCreatePartyComplete& Delegate = FOnCreatePartyComplete()) = 0;

	/**
	 * Update an existing party with new configuration
	 *
	 * @param LocalUserId - user making the request
	 * @param PartyConfig - configuration for the party
	 * @param Delegate - called on completion
	 *
	 * @return true if task was started
	 */
	virtual bool UpdateParty(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, const FPartyConfiguration& PartyConfig, bool bShouldRegenerateReservationKey = false, const FOnUpdatePartyComplete& Delegate = FOnUpdatePartyComplete()) = 0;

	/**
	 * Join an existing party
	 *
	 * @param LocalUserId - user making the request
	 * @param OnlinePartyJoinInfo - join information containing data such as party id, leader id
	 * @param Delegate - called on completion
	 *
	 * @return true if task was started
	 */
	virtual bool JoinParty(const FUniqueNetId& LocalUserId, const IOnlinePartyJoinInfo& OnlinePartyJoinInfo, const FOnJoinPartyComplete& Delegate = FOnJoinPartyComplete()) = 0;

	/**
	* Request to join an existing party
	*
	* @param LocalUserId - user making the request
	* @param PartyTypeId - type id of the party you want join
	* @param Recipient - structure specifying the party leader receiving the request
	* @param Delegate - called on completion
	*/
	virtual void RequestToJoinParty(const FUniqueNetId& LocalUserId, const FOnlinePartyTypeId PartyTypeId, const FPartyInvitationRecipient& Recipient, const FOnRequestToJoinPartyComplete& Delegate = FOnRequestToJoinPartyComplete()) = 0;

	/**
	* Clear a received request to join your party
	*
	* @param LocalUserId - user clearing the request
	* @param PartyId - id of the party the request was made to
	* @param Sender - user who sent the request
	* @param Reason - reason why the request was cleared
	*/
	virtual void ClearRequestToJoinParty(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, const FUniqueNetId& Sender, EPartyRequestToJoinRemovedReason Reason) = 0;

	/**
	 * Query a party to check it's current joinability
	 * Intended to be used before a call to LeaveParty (to leave your existing party, which would then be followed by JoinParty)
	 * Note that the party's joinability can change from moment to moment so a successful response for this does not guarantee a successful JoinParty
	 *
	 * @param LocalUserId - user making the request
	 * @param OnlinePartyJoinInfo - join information containing data such as party id, leader id
	 * @param Delegate - called on completion
	 */
	virtual void QueryPartyJoinability(const FUniqueNetId& LocalUserId, const IOnlinePartyJoinInfo& OnlinePartyJoinInfo, const FOnQueryPartyJoinabilityCompleteEx& Delegate) = 0;

	/**
	 * Leave an existing party
	 * All existing party members notified of member leaving (see FOnPartyMemberLeft)
	 *
	 * @param LocalUserId - user making the request
	 * @param PartyId - id of an existing party
	 * @param bSynchronizeLeave - Whether we synchronize the leave with remote server/clients or only do a local cleanup
	 * @param Delegate - called on completion
	 *
	 * @return true if task was started
	 */
	virtual bool LeaveParty(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, bool bSynchronizeLeave, const FOnLeavePartyComplete& Delegate = FOnLeavePartyComplete()) = 0;

	/**
	* Approve a request to join a party
	*
	* @param LocalUserId - user making the request
	* @param PartyId - id of an existing party
	* @param RecipientId - id of the user being invited
	* @param bIsApproved - whether the join request was approved or not
	* @param DeniedResultCode - used when bIsApproved is false - client defined value to return when leader denies approval
	*
	* @return true if task was started
	*/
	virtual bool ApproveJoinRequest(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, const FUniqueNetId& RecipientId, bool bIsApproved, int32 DeniedResultCode = 0) = 0;

	/**
	 * Respond to a query joinability request.  This reflects the current party's joinability state and can change from moment to moment, and therefore does not guarantee a successful join.
	 *
	 * @param LocalUserId - user making the request
	 * @param PartyId - id of an existing party
	 * @param RecipientId - id of the user being invited
	 * @param bCanJoin - whether the player can attempt to join or not
	 * @param DeniedResultCode - used when bCanJoin is false - client defined value to return when leader denies approval
	 * @param PartyData - data to send back to the querying user
	 */
	virtual void RespondToQueryJoinability(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, const FUniqueNetId& RecipientId, bool bCanJoin, int32 DeniedResultCode, FOnlinePartyDataConstPtr PartyData) = 0;

	/**
	 * sends an invitation to a user that could not otherwise join a party
	 * if the player accepts the invite they will be sent the data needed to trigger a call to RequestReservation
	 *
	 * @param LocalUserId - user making the request
	 * @param PartyId - id of an existing party
	 * @param Recipient - structure specifying the recipient of the invitation
	 * @param Delegate - called on completion
	 *
	 * @return true if task was started
	 */
	virtual bool SendInvitation(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, const FPartyInvitationRecipient& Recipient, const FOnSendPartyInvitationComplete& Delegate = FOnSendPartyInvitationComplete()) = 0;

	/**
	 * Cancel an invitation sent to a user
	 *
	 * @param LocalUserId - user making the cancellation
	 * @param TargetUserId - previously invited user
	 * @param PartyId - id of an existing party
	 * @param Delegate - called on completion
	 */
	virtual void CancelInvitation(const FUniqueNetId& LocalUserId, const FUniqueNetId& TargetUserId, const FOnlinePartyId& PartyId, const FOnCancelPartyInvitationComplete& Delegate = FOnCancelPartyInvitationComplete()) = 0;

	/**
	 * Reject an invite to a party
	 *
	 * @param LocalUserId - user making the request
	 * @param SenderId - id of the sender
	 *
	 * @return true if task was started
	 */
	virtual bool RejectInvitation(const FUniqueNetId& LocalUserId, const FUniqueNetId& SenderId) = 0;

	/**
	 * Clear invitations from a user because the invitations were handled by the application
	 *
	 * @param LocalUserId - user making the request
	 * @param SenderId - id of the sender
	 * @param PartyId - optional, if specified will clear only the one invitation, if blank all invitations from the sender will be cleared
	 *
	 * @return true if task was started
	 */
	virtual void ClearInvitations(const FUniqueNetId& LocalUserId, const FUniqueNetId& SenderId, const FOnlinePartyId* PartyId = nullptr) = 0;

	/**
	 * Kick a user from an existing party
	 * Only admin can kick a party member
	 *
	 * @param LocalUserId - user making the request
	 * @param PartyId - id of an existing party
	 * @param MemberId - id of the user being kicked
	 * @param Delegate - called on completion
	 *
	 * @return true if task was started
	 */
	virtual bool KickMember(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, const FUniqueNetId& TargetMemberId, const FOnKickPartyMemberComplete& Delegate = FOnKickPartyMemberComplete()) = 0;

	/**
	 * Promote a user from an existing party to be admin
	 * All existing party members notified of promoted member (see FOnPartyMemberPromoted)
	 *
	 * @param LocalUserId - user making the request
	 * @param PartyId - id of an existing party
	 * @param MemberId - id of the user being promoted
	 * @param Delegate - called on completion
	 *
	 * @return true if task was started
	 */
	virtual bool PromoteMember(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, const FUniqueNetId& TargetMemberId, const FOnPromotePartyMemberComplete& Delegate = FOnPromotePartyMemberComplete()) = 0;

	/**
	 * Set party data and broadcast to all members
	 * Only current data can be set and no history of past party data is preserved
	 * Party members notified of new data (see FOnPartyDataReceived)
	 *
	 * @param LocalUserId - user making the request
	 * @param PartyId - id of an existing party
	 * @param Namespace - namespace for the data
	 * @param PartyData - data to send to all party members
	 * @param Delegate - called on completion
	 *
	 * @return true if task was started
	 */
	virtual bool UpdatePartyData(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, const FName& Namespace, const FOnlinePartyData& PartyData) = 0;

	/**
	 * Set party data for a single party member and broadcast to all members
	 * Only current data can be set and no history of past party member data is preserved
	 * Party members notified of new data (see FOnPartyMemberDataReceived)
	 *
	 * @param LocalUserId - user making the request
	 * @param PartyId - id of an existing party
	 * @param Namespace - namespace for the data
	 * @param PartyMemberData - member data to send to all party members
	 * @param Delegate - called on completion
	 *
	 * @return true if task was started
	 */
	virtual bool UpdatePartyMemberData(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, const FName& Namespace, const FOnlinePartyData& PartyMemberData) = 0;

	/**
	 * returns true if the user specified by MemberId is the leader of the party specified by PartyId
	 *
	 * @param LocalUserId - user making the request
	 * @param PartyId     - id of an existing party
	 * @param MemberId    - id of member to test
	 *
	 */
	virtual bool IsMemberLeader(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, const FUniqueNetId& MemberId) const = 0;

	/**
	 * returns the number of players in a given party
	 *
	 * @param LocalUserId - user making the request
	 * @param PartyId     - id of an existing party
	 *
	 */
	virtual uint32 GetPartyMemberCount(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId) const = 0;

	/**
	 * Get info associated with a party
	 *
	 * @param LocalUserId - user making the request
	 * @param PartyId     - id of an existing party
	 *
	 * @return party info or nullptr if not found
	 */
	virtual FOnlinePartyConstPtr GetParty(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId) const = 0;

	/**
	 * Get info associated with a party
	 *
	 * @param LocalUserId - user making the request
	 * @param PartyTypeId - type of an existing party
	 *
	 * @return party info or nullptr if not found
	 */
	virtual FOnlinePartyConstPtr GetParty(const FUniqueNetId& LocalUserId, const FOnlinePartyTypeId& PartyTypeId) const = 0;

	/**
	 * Get a party member by id
	 *
	 * @param LocalUserId - user making the request
	 * @param PartyId     - id of an existing party
	 * @param MemberId    - id of member to find
	 *
	 * @return party member info or nullptr if not found
	 */
	virtual FOnlinePartyMemberConstPtr GetPartyMember(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, const FUniqueNetId& MemberId) const = 0;

	/**
	 * Get current cached data associated with a party
	 * FOnPartyDataReceived notification called whenever this data changes
	 *
	 * @param LocalUserId - user making the request
	 * @param PartyId     - id of an existing party
	 * @param Namespace   - namespace of data
	 *
	 * @return party data or nullptr if not found
	 */
	virtual FOnlinePartyDataConstPtr GetPartyData(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, const FName& Namespace) const = 0;

	/**
	 * Get current cached data associated with a party member
	 * FOnPartyMemberDataReceived notification called whenever this data changes
	 *
	 * @param LocalUserId - user making the request
	 * @param PartyId     - id of an existing party
	 * @param MemberId    - id of member to find data for
	 * @param Namespace   - namespace of data
	 *
	 * @return party member data or nullptr if not found
	 */
	virtual FOnlinePartyDataConstPtr GetPartyMemberData(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, const FUniqueNetId& MemberId, const FName& Namespace) const = 0;

	/**
	 * Get the join info of the specified user and party type
	 *
	 * @param LocalUserId       - user making the request
	 * @param UserId            - user to check
	 * @param PartyTypeId       - type of party to query
	 *
	 * @return shared pointer to the join info if the user is advertising for that party type
	 */
	virtual IOnlinePartyJoinInfoConstPtr GetAdvertisedParty(const FUniqueNetId& LocalUserId, const FUniqueNetId& UserId, const FOnlinePartyTypeId PartyTypeId) const = 0;

	/**
	 * Get a list of currently joined parties for the user
	 *
	 * @param LocalUserId     - user making the request
	 * @param OutPartyIdArray - list of party ids joined by the current user
	 *
	 * @return true if entries found
	 */
	virtual bool GetJoinedParties(const FUniqueNetId& LocalUserId, TArray<TSharedRef<const FOnlinePartyId>>& OutPartyIdArray) const = 0;

	/**
	 * Get list of current party members
	 *
	 * @param LocalUserId          - user making the request
	 * @param PartyId              - id of an existing party
	 * @param OutPartyMembersArray - list of party members currently in the party
	 *
	 * @return true if entries found
	 */
	virtual bool GetPartyMembers(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, TArray<FOnlinePartyMemberConstRef>& OutPartyMembersArray) const = 0;

	/**
	 * Get a list of parties the user has been invited to
	 *
	 * @param LocalUserId            - user making the request
	 * @param OutPendingInvitesArray - list of party info needed to join the party
	 *
	 * @return true if entries found
	 */
	virtual bool GetPendingInvites(const FUniqueNetId& LocalUserId, TArray<IOnlinePartyJoinInfoConstRef>& OutPendingInvitesArray) const = 0;

	/**
	 * Get list of users requesting to join the party
	 *
	 * @param LocalUserId           - user making the request
	 * @param PartyId               - id of an existing party
	 * @param OutPendingUserIdArray - list of pending party members
	 *
	 * @return true if entries found
	 */
	virtual bool GetPendingJoinRequests(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, TArray<IOnlinePartyPendingJoinRequestInfoConstRef>& OutPendingJoinRequestArray) const = 0;

	/**
	 * Get list of users invited to a party that have not yet responded
	 *
	 * @param LocalUserId                - user making the request
	 * @param PartyId                    - id of an existing party
	 * @param OutPendingInvitedUserArray - list of user that have pending invites
	 *
	 * @return true if entries found
	 */
	virtual bool GetPendingInvitedUsers(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, TArray<FUniqueNetIdRef>& OutPendingInvitedUserArray) const = 0;

	/**
	 * Get a list of users requesting to join the party. These are not requests for a reservation, but permission (e.g. locked party).
	 *
	 * @param LocalUserId       - user making the request
	 * @param OutRequestsToJoin - list of requests to join
	 *
	 * @return true if entries found
	 */
	virtual bool GetPendingRequestsToJoin(const FUniqueNetId& LocalUserId, TArray<IOnlinePartyRequestToJoinInfoConstRef>& OutRequestsToJoin) const = 0;

	static const FOnlinePartyTypeId::TInternalType PrimaryPartyTypeIdValue = 0x11111111;
	/**
	 * @return party type id for the primary party - the primary party is the party that will be addressable via the social panel
	 */
	static const FOnlinePartyTypeId GetPrimaryPartyTypeId() { return FOnlinePartyTypeId(PrimaryPartyTypeIdValue); }

	/**
	 * @return party type id for the specified party
	 */
	static const FOnlinePartyTypeId MakePartyTypeId(const FOnlinePartyTypeId::TInternalType InTypeId) { ensure(InTypeId != PrimaryPartyTypeIdValue); return FOnlinePartyTypeId(InTypeId); }

	/**
	 * returns the json version of a join info for a current party
	 *
	 * @param LocalUserId       - user making the request
	 * @param PartyId           - party to make the json from
	 *
 	 */
	virtual FString MakeJoinInfoJson(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId) = 0;

	/**
	 * returns a valid join info object from a json blob
	 *
	 * @param JoinInfoJson       - json blob to convert
	 *
	 */
	virtual IOnlinePartyJoinInfoConstPtr MakeJoinInfoFromJson(const FString& JoinInfoJson) = 0;

	/**
	 * Creates a command line token from a IOnlinePartyJoinInfo object
	 *
	 * @param JoinInfo - the IOnlinePartyJoinInfo object to convert
	 *
	 * return the new IOnlinePartyJoinInfo object
	 */
	virtual FString MakeTokenFromJoinInfo(const IOnlinePartyJoinInfo& JoinInfo) const = 0;

	/**
	 * returns the join info for current party
	 *
	 * @param LocalUserId       - user making the request
	 * @param PartyId           - party to make the json from
	 *
 	 */
	virtual IOnlinePartyJoinInfoConstPtr MakeJoinInfo(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId) = 0;

	/**
	 * Creates a IOnlinePartyJoinInfo object from a command line token
	 *
	 * @param Token - the token string
	 *
	 * return the new IOnlinePartyJoinInfo object
	 */
	virtual IOnlinePartyJoinInfoConstPtr MakeJoinInfoFromToken(const FString& Token) const = 0;

	/**
	 * Checks to see if there is a pending command line invite and consumes it
	 *
	 * return the pending IOnlinePartyJoinInfo object
	 */
	virtual IOnlinePartyJoinInfoConstPtr ConsumePendingCommandLineInvite() = 0;

	/**
	 * List of all subscribe-able notifications
	 *
	 * OnPartyJoined
	 * OnPartyExited
	 * OnPartyStateChanged
	 * OnPartyPromotionLockoutStateChanged
	 * OnPartyConfigChanged
	 * OnPartyDataReceived
	 * OnPartyMemberPromoted
	 * OnPartyMemberExited
	 * OnPartyMemberJoined
	 * OnPartyMemberDataReceived
	 * OnPartyInvitesChanged
	 * OnPartyInviteRequestReceived
	 * OnPartyInviteReceivedEx
	 * OnPartyInviteRemovedEx
	 * OnPartyInviteResponseReceived
	 * OnPartyJoinRequestReceived
	 * OnPartyQueryJoinabilityReceived
	 * OnFillPartyJoinRequestData
	 * OnPartyAnalyticsEvent
	 * OnPartySystemStateChange
	 * OnPartyRequestToJoinReceived
	 * OnPartyRequestToJoinRemoved
	 */

	/**
	 * notification of when a party is joined
	 * @param LocalUserId - id associated with this notification
	 * @param PartyId - id associated with the party
	 */
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnPartyJoined, const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/);

	/**
	 * notification of when a party is exited
	 * @param LocalUserId - id associated with this notification
	 * @param PartyId - id associated with the party
	 */
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnPartyExited, const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/);

	/**
	 * Notification when a party's state has changed
	 * @param LocalUserId - id associated with this notification
	 * @param PartyId - id associated with the party
	 * @param State - state of the party
	 * @param PreviousState - previous state of the party
	*/
	DEFINE_ONLINE_DELEGATE_FOUR_PARAM(OnPartyStateChanged, const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, EPartyState /*State*/, EPartyState /*PreviousState*/);

	/**
	 * Notification when player promotion is locked out.
	 *
	 * @param PartyId - id associated with the party
	 * @param bLockoutState - if promotion is currently locked out
	 */
	DEFINE_ONLINE_DELEGATE_THREE_PARAM(OnPartyPromotionLockoutChanged, const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const bool /*bLockoutState*/);

	/**
	 * Notification when party data is updated
	 * @param LocalUserId - id associated with this notification
	 * @param PartyId - id associated with the party
	 * @param PartyConfig - party whose config was updated
	 */
	DEFINE_ONLINE_DELEGATE_THREE_PARAM(OnPartyConfigChanged, const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const FPartyConfiguration& /*PartyConfig*/);

	/**
	 * Notification when party data is updated
	 * @param LocalUserId - id associated with this notification
	 * @param PartyId - id associated with the party
	 * @param Namespace - namespace the party data is associated with
	 * @param PartyData - party data that was updated
	 */
	DEFINE_ONLINE_DELEGATE_FOUR_PARAM(OnPartyDataReceived, const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const FName& /*Namespace*/, const FOnlinePartyData& /*PartyData*/);

	/**
	* Notification when a member is promoted in a party
	* @param LocalUserId - id associated with this notification
	* @param PartyId - id associated with the party
	* @param NewLeaderId - id of member that was promoted
	*/
	DEFINE_ONLINE_DELEGATE_THREE_PARAM(OnPartyMemberPromoted, const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const FUniqueNetId& /*NewLeaderId*/);

	/**
	* Notification when a member exits a party
	* @param LocalUserId - id associated with this notification
	* @param PartyId - id associated with the party
	* @param MemberId - id of member that joined
	* @param Reason - why the member was removed
	*/
	DEFINE_ONLINE_DELEGATE_FOUR_PARAM(OnPartyMemberExited, const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const FUniqueNetId& /*MemberId*/, const EMemberExitedReason /*Reason*/);

	/**
	 * Notification when a member joins the party
	 * @param LocalUserId - id associated with this notification
	 * @param PartyId - id associated with the party
	 * @param MemberId - id of member that joined
	 */
	DEFINE_ONLINE_DELEGATE_THREE_PARAM(OnPartyMemberJoined, const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const FUniqueNetId& /*MemberId*/);

	/**
	 * Notification when party member data is updated
	 * @param LocalUserId - id associated with this notification
	 * @param PartyId - id associated with the party
	 * @param MemberId - id of member that had updated data
	 * @param Namespace - namespace the party data is associated with
	 * @param PartyMemberData - party member data that was updated
	 */
	DEFINE_ONLINE_DELEGATE_FIVE_PARAM(OnPartyMemberDataReceived, const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const FUniqueNetId& /*MemberId*/, const FName& /*Namespace*/, const FOnlinePartyData& /*PartyData*/);

	/**
	 * Notification when an invite list has changed for a party
	 * @param LocalUserId - user that is associated with this notification
	 */
	DEFINE_ONLINE_DELEGATE_ONE_PARAM(OnPartyInvitesChanged, const FUniqueNetId& /*LocalUserId*/);

	/**
	* Notification when a request for an invite has been received
	* @param LocalUserId - id associated with this notification
	* @param PartyId - id associated with the party
	* @param SenderId - id of user that sent the invite
	* @param RequestForId - id of user that sender is requesting the invite for - invalid if the sender is requesting the invite
	*/
	DEFINE_ONLINE_DELEGATE_FOUR_PARAM(OnPartyInviteRequestReceived, const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const FUniqueNetId& /*SenderId*/, const FUniqueNetId& /*RequestForId*/);

	/**
	 * Notification when a new invite is received
	 * @param LocalUserId - id associated with this notification
	 * @param Invitation - the invitation that was received
	 */
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnPartyInviteReceivedEx, const FUniqueNetId& /*LocalUserId*/, const IOnlinePartyJoinInfo& /*Invitation*/);

	/**
	 * Notification when an invite has been removed
	 * @param LocalUserId id associated with this notification
	 * @param Invitation the invitation that was removed
	 * @param Reason reason the invitation was removed
	 */
	DEFINE_ONLINE_DELEGATE_THREE_PARAM(OnPartyInviteRemovedEx, const FUniqueNetId& /*LocalUserId*/, const IOnlinePartyJoinInfo& /*Invitation*/, EPartyInvitationRemovedReason /*Reason*/);

	/**
	 * Notification when an invitation response is received
	 * @param LocalUserId - id associated with this notification
	 * @param PartyId - id associated with the party
	 * @param SenderId - id of user that responded to an invite
	 * @param Response - the response
	 */
	DEFINE_ONLINE_DELEGATE_FOUR_PARAM(OnPartyInviteResponseReceived, const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const FUniqueNetId& /*SenderId*/, const EInvitationResponse /*Response*/);

	/**
	 * Notification when a new reservation request is received
	 * Subscriber is expected to call ApproveJoinRequest
	 * @param LocalUserId - id associated with this notification
	 * @param PartyId - id associated with the party
	 * @param JoinRequestInfo - data about users that are joining
	 */
	DEFINE_ONLINE_DELEGATE_THREE_PARAM(OnPartyJoinRequestReceived, const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const IOnlinePartyPendingJoinRequestInfo& /*JoinRequestInfo*/);

	/**
	 * Notification when a player wants to know if the party is in a joinable state
	 * Subscriber is expected to call RespondToQueryJoinability
	 * @param LocalUserId - id associated with this notification
	 * @param PartyId - id associated with the party
	 * @param JoinRequestInfo - data about users that are joining
	 */
	DEFINE_ONLINE_DELEGATE_THREE_PARAM(OnQueryPartyJoinabilityReceived, const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const IOnlinePartyPendingJoinRequestInfo& /*JoinRequestInfo*/);

	/**
	 * Notification when a player wants to know if the party is in a joinable state
	 * Subscriber is expected to call RespondToQueryJoinability
	 * @param LocalUserId - id associated with this notification
	 * @param PartyId - id associated with the party
	 * @param PartyData - data provided by the sender for the leader to use to determine joinability
	 */
	DEFINE_ONLINE_DELEGATE_THREE_PARAM(OnFillPartyJoinRequestData, const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, FOnlinePartyData& /*PartyData*/);

	/**
	 * Notification when an analytics event needs to be recorded
	 * Subscriber is expected to record the event
	 * @param LocalUserId - id associated with this event
	 * @param EventName - name of the event
	 * @param Attributes - attributes for the event
	 */
	DEFINE_ONLINE_DELEGATE_THREE_PARAM(OnPartyAnalyticsEvent, const FUniqueNetId& /*LocalUserId*/, const FString& /*EventName*/, const TArray<FAnalyticsEventAttribute>& /*Attributes*/);

	/**
	* Notification of party system state change
	* @param NewState - new state this partysystem is in
	*/
	DEFINE_ONLINE_DELEGATE_ONE_PARAM(OnPartySystemStateChange, EPartySystemState /*NewState*/);

	/**
	 * Notification when a new party join request is received
	 * @param LocalUserId - id associated with this notification
	 * @param PartyId - id associated with the party
	 * @param RequesterId - id of player who requested to join
	 * @param Request - request to join information
	 */
	DEFINE_ONLINE_DELEGATE_FOUR_PARAM(OnPartyRequestToJoinReceived, const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const FUniqueNetId& /*RequesterId*/, const IOnlinePartyRequestToJoinInfo& /*Request*/);

	/**
	 * Notification when a new party join request is removed
	 * @param LocalUserId - id associated with this notification
	 * @param PartyId - id associated with the party
	 * @param RequesterId - id of player who requested to join
	 * @param Request - request to join information
	 * @param Reason - the reason why the request was removed
	 */
	DEFINE_ONLINE_DELEGATE_FIVE_PARAM(OnPartyRequestToJoinRemoved, const FUniqueNetId& /*LocalUserId*/, const FOnlinePartyId& /*PartyId*/, const FUniqueNetId& /*RequesterId*/, const IOnlinePartyRequestToJoinInfo& /*Request*/, EPartyRequestToJoinRemovedReason /*Reason*/);

	/**
	 * Dump out party state for all known parties
	 */
	virtual void DumpPartyState() = 0;

};

enum class ECreatePartyCompletionResult : int8
{
	UnknownClientFailure = -100,
	AlreadyInPartyOfSpecifiedType,
	AlreadyCreatingParty,
	AlreadyInParty,
	FailedToCreateMucRoom,
	NoResponse,
	LoggedOut,
	NotPrimaryUser,
	UnknownInternalFailure = 0,
	Succeeded = 1
};

enum class EJoinPartyCompletionResult : int8
{
	/** Unspecified error.  No message sent to party leader. */
	UnknownClientFailure = -100,
	/** Your build id does not match the build id of the party */
	BadBuild,
	/** Your provided access key does not match the party's access key */
	InvalidAccessKey,
	/** The party leader already has you in the joining players list */
	AlreadyInLeadersJoiningList,
	/** The party leader already has you in the party members list */
	AlreadyInLeadersPartyRoster,
	/** The party leader rejected your join request because the party is full*/
	NoSpace,
	/** The party leader rejected your join request for a game specific reason, indicated by the NotApprovedReason parameter */
	NotApproved,
	/** The player you send the join request to is not a member of the specified party */
	RequesteeNotMember,
	/** The player you send the join request to is not the leader of the specified party */
	RequesteeNotLeader,
	/** A response was not received from the party leader in a timely manner, the join attempt is considered failed */
	NoResponse,
	/** You were logged out while attempting to join the party */
	LoggedOut,
	/** Your platform is not compatible with the party */
	IncompatiblePlatform,

	/** We are currently waiting for a response for a previous join request for the specified party.  No message sent to party leader. */
	AlreadyJoiningParty,
	/** We are already in the party that you are attempting to join.  No message sent to the party leader. */
	AlreadyInParty,
	/** The party join info is invalid.  No message sent to the party leader. */
	JoinInfoInvalid,
	/** We are already in a party of the specified type.  No message sent to the party leader. */
	AlreadyInPartyOfSpecifiedType,
	/** Failed to send a message to the party leader.  No message sent to the party leader. */
	MessagingFailure,

	/** Game specific reason, indicated by the NotApprovedReason parameter.  Message might or might not have been sent to party leader. */
	GameSpecificReason,
	
	/** Your app id does not match the app id of the party */
	MismatchedApp,

	/** DEPRECATED */
	UnknownInternalFailure = 0,

	/** Successully joined the party */
	Succeeded = 1
};

enum class ELeavePartyCompletionResult : int8
{
	/** Unspecified error.  No message sent. */
	UnknownClientFailure = -100,
	/** Timed out waiting for a response to the message.  Party has been left. */
	NoResponse,
	/** You were logged out while attempting to leave the party.  Party has been left. */
	LoggedOut,

	/** You are not in the specified party.  No message sent. */
	UnknownParty,
	/** You are already leaving the party.  No message sent. */
	LeavePending,

	/** DEPRECATED! */
	UnknownLocalUser,
	/** DEPRECATED! */
	NotMember,
	/** DEPRECATED! */
	MessagingFailure,
	/** DEPRECATED! */
	UnknownTransportFailure,
	/** DEPRECATED! */
	UnknownInternalFailure = 0,

	/** Successfully left the party */
	Succeeded = 1
};

enum class EUpdateConfigCompletionResult : int8
{
	UnknownClientFailure = -100,
	UnknownParty,
	LocalMemberNotMember,
	LocalMemberNotLeader,
	RemoteMemberNotMember,
	MessagingFailure,
	NoResponse,
	UnknownInternalFailure = 0,
	Succeeded = 1
};

enum class ERequestPartyInvitationCompletionResult : int8
{
	NotLoggedIn = -100,
	InvitePending,
	AlreadyInParty,
	PartyFull,
	NoPermission,
	UnknownInternalFailure = 0,
	Succeeded = 1
};

enum class ESendPartyInvitationCompletionResult : int8
{
	NotLoggedIn = -100,
	InvitePending,
	AlreadyInParty,
	PartyFull,
	NoPermission,
	RateLimited,
	UnknownInternalFailure = 0,
	Succeeded = 1
};

enum class EAcceptPartyInvitationCompletionResult : int8
{
	NotLoggedIn = -100,
	InvitePending,
	AlreadyInParty,
	PartyFull,
	NoPermission,
	UnknownInternalFailure = 0,
	Succeeded = 1
};

enum class ERejectPartyInvitationCompletionResult : int8
{
	NotLoggedIn = -100,
	InvitePending,
	AlreadyInParty,
	PartyFull,
	NoPermission,
	UnknownInternalFailure = 0,
	Succeeded = 1
};

enum class EKickMemberCompletionResult : int8
{
	UnknownClientFailure = -100,
	UnknownParty,
	LocalMemberNotMember,
	LocalMemberNotLeader,
	RemoteMemberNotMember,
	MessagingFailure,
	NoResponse,
	LoggedOut,
	UnknownInternalFailure = 0,
	Succeeded = 1
};

enum class EPromoteMemberCompletionResult : int8
{
	UnknownClientFailure = -100,
	UnknownServiceFailure,
	UnknownParty,
	LocalMemberNotMember,
	LocalMemberNotLeader,
	PromotionAlreadyPending,
	TargetIsSelf,
	TargetNotMember,
	MessagingFailure,
	NoResponse,
	LoggedOut,
	UnknownInternalFailure = 0,
	Succeeded = 1
};

enum class EInvitationResponse : uint8
{
	UnknownFailure,
	BadBuild,
	Rejected,
	Accepted
};

enum class ERequestToJoinPartyCompletionResult : int8
{
	ValidationFailure = -100,
	NotAuthorized,
	Forbidden,
	UserNotFound,
	AlreadyExists,
	RateLimited,
	UnknownInternalFailure,
	Succeeded = 1
};

/**
 * QueryPartyJoinability result
 */
struct FQueryPartyJoinabilityResult
{
	/** Online error representing the result of the operation */
	FOnlineError Result;
	/** Enum result */
	EJoinPartyCompletionResult EnumResult = EJoinPartyCompletionResult::UnknownClientFailure;
	/** Subcode for the EnumResult */
	int32 SubCode = 0;
	/** If successful, the party's data */
	FOnlinePartyDataConstPtr PartyData;
};

/** @return the stringified version of the enum passed in */
ONLINESUBSYSTEM_API const TCHAR* ToString(const EPartyState Value);
/** @return the enum version of the string passed in */
ONLINESUBSYSTEM_API EPartyState EPartyStateFromString(const TCHAR* Value);

/** @return the stringified version of the enum passed in */
ONLINESUBSYSTEM_API const TCHAR* ToString(const EMemberConnectionStatus Value);
/** @return the stringified version of the enum passed in */
ONLINESUBSYSTEM_API const TCHAR* ToString(const EMemberExitedReason Value);
/** @return the stringified version of the enum passed in */
ONLINESUBSYSTEM_API const TCHAR* ToString(const EPartyInvitationRemovedReason Value);
/** @return the stringified version of the enum passed in */
ONLINESUBSYSTEM_API const TCHAR* ToString(const EPartyRequestToJoinRemovedReason Value);
/** @return the stringified version of the enum passed in */
ONLINESUBSYSTEM_API const TCHAR* ToString(const ECreatePartyCompletionResult Value);
/** @return the stringified version of the enum passed in */
ONLINESUBSYSTEM_API const TCHAR* ToString(const ESendPartyInvitationCompletionResult Value);
/** @return the stringified version of the enum passed in */
ONLINESUBSYSTEM_API const TCHAR* ToString(const EJoinPartyCompletionResult Value);
/** @return the stringified version of the enum passed in */
ONLINESUBSYSTEM_API const TCHAR* ToString(const ELeavePartyCompletionResult Value);
/** @return the stringified version of the enum passed in */
ONLINESUBSYSTEM_API const TCHAR* ToString(const EUpdateConfigCompletionResult Value);
/** @return the stringified version of the enum passed in */
ONLINESUBSYSTEM_API const TCHAR* ToString(const EKickMemberCompletionResult Value);
/** @return the stringified version of the enum passed in */
ONLINESUBSYSTEM_API const TCHAR* ToString(const EPromoteMemberCompletionResult Value);
/** @return the stringified version of the enum passed in */
ONLINESUBSYSTEM_API const TCHAR* ToString(const EInvitationResponse Value);
/** @return the stringified version of the enum passed in */
ONLINESUBSYSTEM_API const TCHAR* ToString(const ERequestToJoinPartyCompletionResult Value);

/** @return the stringified version of the enum passed in */
ONLINESUBSYSTEM_API const TCHAR* ToString(const PartySystemPermissions::EPermissionType Value);
/** @return the enum version of the string passed in */
ONLINESUBSYSTEM_API PartySystemPermissions::EPermissionType PartySystemPermissionTypeFromString(const TCHAR* Value);

/** @return the stringified version of the enum passed in */
ONLINESUBSYSTEM_API const TCHAR* ToString(const EJoinRequestAction Value);
/** @return the enum version of the string passed in */
ONLINESUBSYSTEM_API EJoinRequestAction JoinRequestActionFromString(const TCHAR* Value);

/** Dump party configuration for debugging */
ONLINESUBSYSTEM_API FString ToDebugString(const FPartyConfiguration& PartyConfiguration);
/** Dump join info for debugging */
ONLINESUBSYSTEM_API FString ToDebugString(const IOnlinePartyJoinInfo& JoinInfo);
/** Dump key/value pairs for debugging */
ONLINESUBSYSTEM_API FString ToDebugString(const FOnlineKeyValuePairs<FString, FVariantData>& KeyValAttrs);
/** Dump state about the party data for debugging */
ONLINESUBSYSTEM_API FString ToDebugString(const FOnlinePartyData& PartyData);

#undef UE_API
