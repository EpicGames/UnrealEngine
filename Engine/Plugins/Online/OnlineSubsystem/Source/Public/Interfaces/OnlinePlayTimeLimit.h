// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Features/IModularFeature.h"

class FUniqueNetId;

/**
 * Delegate called to warn a user of their play time amount
 * This is called when RewardRate changes, and periodically to remind the user of their play time
 *
 * @param LocalUserNum the controller number of the associated user
 * @param MinutesPlayed number of minutes the user has played
 * @param RewardRate current rate of rewards for the user
 */
DECLARE_MULTICAST_DELEGATE_SixParams(FWarnUserPlayTime, const FUniqueNetId& /*UserId*/, int32 /*MinutesPlayed*/, float /*RewardRate*/, const FString& /*DialogTitle*/, const FString& /*DialogText*/, const FString& /*ButtonText*/);
typedef FWarnUserPlayTime::FDelegate FWarnUserPlayTimeDelegate;

/**
 * Interface to provide play time limits
 */
class IOnlinePlayTimeLimit : public IModularFeature
{
public:
	/** Destructor */
	virtual ~IOnlinePlayTimeLimit() = default;

	/**
	 * Get the name of the modular feature, to be used to get the implementations
	 * @return name of the modular feature
	 */
	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName(TEXT("OnlinePlayTimeLimit"));
		return FeatureName;
	}

	/**
	 * Does the user have a play time limit?
	 * @param UserId the id of the user to check
	 * @return true if the user has a play time limit
	 */
	virtual bool HasTimeLimit(const FUniqueNetId& UserId) = 0;

	/**
	 * Get the play time in minutes for the user
	 * @param UserId the id of the user to check
	 * @return the play time in minutes for the user
	 */
	virtual int32 GetPlayTimeMinutes(const FUniqueNetId& UserId) = 0;

	/**
	 * Get the reward amount multiplier for the user
	 * Expected to start at 1.0 (full rewards), and is reduced based on their play time
	 * @param UserId the id of the user to check
	 * @return the reward amount multiplier
	 */
	virtual float GetRewardRate(const FUniqueNetId& UserId) = 0;

	/**
	 * Delegate called when a warning should be displayed to the user. See FWarnUserPlayTimeDelegate
	 */
	virtual FWarnUserPlayTime& GetWarnUserPlayTimeDelegate() = 0;
};
