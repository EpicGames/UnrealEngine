// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Internationalization.h"
#include "OnlineSubsystemTypes.h"
#include "Misc/DateTime.h"
#include "OnlineDelegateMacros.h"
#include "OnlineKeyValuePair.h"

#define UE_API ONLINESUBSYSTEM_API

ONLINESUBSYSTEM_API DECLARE_LOG_CATEGORY_EXTERN(LogOnlinePresence, Log, All);

#define UE_LOG_ONLINE_PRESENCE(Verbosity, Format, ...) \
{ \
	UE_LOGF(LogOnlinePresence, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

#define UE_CLOG_ONLINE_PRESENCE(Conditional, Verbosity, Format, ...) \
{ \
	UE_CLOGF(Conditional, LogOnlinePresence, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

/** Type of presence keys */
typedef FString FPresenceKey;

/** Type of presence properties - a key/value map */
typedef FOnlineKeyValuePairs<FPresenceKey, FVariantData> FPresenceProperties;

/** The default key that will update presence text in the platform's UI */
extern ONLINESUBSYSTEM_API const FString DefaultPresenceKey;

/** Custom presence data that is not seen by users but can be polled */
extern ONLINESUBSYSTEM_API const FString CustomPresenceDataKey;

/** Name of the client that sent the presence update */
extern ONLINESUBSYSTEM_API const FString DefaultAppIdKey;

/** User friendly name of the client that sent the presence update */
extern ONLINESUBSYSTEM_API const FString DefaultProductNameKey;

/** Platform of the client that sent the presence update */
extern ONLINESUBSYSTEM_API const FString DefaultPlatformKey;

/** Override Id of the client to set the presence state to */
extern ONLINESUBSYSTEM_API const FString OverrideAppIdKey;

/** Id of the session for the presence update. @todo samz - SessionId on presence data should be FUniqueNetId not uint64 */
extern ONLINESUBSYSTEM_API const FString DefaultSessionIdKey;

/** Resource the client is logged in with */
extern ONLINESUBSYSTEM_API const FString PresenceResourceKey;

namespace EOnlinePresenceState
{
	enum Type : uint8
	{
		Online,
		Offline,
		Away,
		ExtendedAway,
		DoNotDisturb,
		Chat
	};

	/** 
	 * @return the stringified version of the enum passed in 
	 */
	ONLINESUBSYSTEM_API const TCHAR* ToString(EOnlinePresenceState::Type EnumVal);

	/**
	 * @return EOnlinePresenceState from the string passed in
	 */
	ONLINESUBSYSTEM_API EOnlinePresenceState::Type FromString(const TCHAR* StringVal);

	UE_DEPRECATED(5.8, "Use ToLocText(EOnlinePresenceState::Online)")
	extern ONLINESUBSYSTEM_API const FText OnlineText;
	UE_DEPRECATED(5.8, "Use ToLocText(EOnlinePresenceState::Offline)")
	extern ONLINESUBSYSTEM_API const FText OfflineText;
	UE_DEPRECATED(5.8, "Use ToLocText(EOnlinePresenceState::Away)")
	extern ONLINESUBSYSTEM_API const FText AwayText;
	UE_DEPRECATED(5.8, "Use ToLocText(EOnlinePresenceState::DoNotDisturb)")
	extern ONLINESUBSYSTEM_API const FText DoNotDisturbText;
	UE_DEPRECATED(5.8, "Use ToLocText(EOnlinePresenceState::Chat)")
	extern ONLINESUBSYSTEM_API const FText ChatText;

	/** 
	 * @return the loc text version of the enum passed in 
	 */
	ONLINESUBSYSTEM_API const FText ToLocText(EOnlinePresenceState::Type EnumVal);
}

class FOnlineUserPresenceStatus
{
public:
	FString StatusStr;
	EOnlinePresenceState::Type State;
	FPresenceProperties Properties;

	FOnlineUserPresenceStatus()
		: State(EOnlinePresenceState::Offline)
	{
	}

	FString ToDebugString() const
	{
		FString PropertiesStr;
		for (const TPair<FPresenceKey, FVariantData>& Pair : Properties)
		{
			PropertiesStr += FString::Printf(TEXT("\n%s : %s"), *Pair.Key, *Pair.Value.ToString());
		}
		return FString::Printf(TEXT("FOnlineUserPresenceStatus {State: %s, Status: %s, Properties: %s}"), EOnlinePresenceState::ToString(State), *StatusStr, *PropertiesStr);
	}
};

/**
 * Parameters to IOnlinePresence::SetPresence. Similar to FOnlineUserPresenceStatus, but fields requesting update must be explicitly set.
 */
struct FOnlinePresenceSetPresenceParameters
{
public:
	/** Updated status string */
	TOptional<FString> StatusStr;
	/** Updated state */
	TOptional<EOnlinePresenceState::Type> State;
	/** Updated properties. Note that this will update all properties, not just the subset specified in Properties. */
	TOptional<FPresenceProperties> Properties;

	FString ToDebugString() const
	{
		FString PropertiesStr;
		if (Properties.IsSet())
		{
			for (const TPair<FPresenceKey, FVariantData>& Pair : Properties.GetValue())
			{
				PropertiesStr += FString::Printf(TEXT("\n%s : %s"), *Pair.Key, *Pair.Value.ToString());
			}
		}
		else
		{
			PropertiesStr = TEXT("Unset");
		}
		return FString::Printf(TEXT("FOnlinePresenceSetPresenceParameters {State: %s, Status: %s, Properties: %s}"), 
			State.IsSet() ? EOnlinePresenceState::ToString(State.GetValue()) : TEXT("Unset"),
			StatusStr.IsSet() ? *StatusStr.GetValue() : TEXT("Unset"), 
			*PropertiesStr);
	}
};

/**
 * Presence info for an online user returned via IOnlinePresence interface
 */
class FOnlineUserPresence
{
public:
	FUniqueNetIdPtr SessionId;
	uint32 bIsOnline:1;
	uint32 bIsPlaying:1;
	uint32 bIsPlayingThisGame:1;
	uint32 bIsJoinable:1;
	uint32 bHasVoiceSupport:1;
	FDateTime LastOnline;
	FOnlineUserPresenceStatus Status;

	/** Constructor */
	FOnlineUserPresence()
	{
		Reset();
	}

	void Reset()
	{
		SessionId = nullptr;
		bIsOnline = 0;
		bIsPlaying = 0;
		bIsPlayingThisGame = 0;
		bIsJoinable = 0;
		bHasVoiceSupport = 0;
		Status = FOnlineUserPresenceStatus();
		LastOnline = FDateTime::MaxValue();
	}

	const FString GetPlatform() const
	{
		FString ParsedOnlinePlatform = TEXT("");
		const FVariantData* VariantOnlinePlatform = Status.Properties.Find(DefaultPlatformKey);
		if (VariantOnlinePlatform != nullptr && VariantOnlinePlatform->GetType() == EOnlineKeyValuePairDataType::String)
		{
			VariantOnlinePlatform->GetValue(ParsedOnlinePlatform);
		}
		return ParsedOnlinePlatform;
	}

	const FString GetAppId() const
	{
		FString Result = TEXT("");
		const FVariantData* AppId = Status.Properties.Find(DefaultAppIdKey);
		const FVariantData* OverrideAppId = Status.Properties.Find(OverrideAppIdKey);
		if (OverrideAppId != nullptr && OverrideAppId->GetType() == EOnlineKeyValuePairDataType::String)
		{
			OverrideAppId->GetValue(Result);
		}
		else if (AppId != nullptr && AppId->GetType() == EOnlineKeyValuePairDataType::String)
		{
			AppId->GetValue(Result);
		}
		return Result;
	}

	FString ToDebugString() const
	{
		return FString::Printf(TEXT("FOnlineUserPresence {Online: %d Playing: %d ThisGame: %d Joinable: %d VoiceSupport: %d SessionId: %s Status: %s"), bIsOnline, bIsPlaying, bIsPlayingThisGame, bIsJoinable, bHasVoiceSupport, SessionId.IsValid() ? *SessionId->ToDebugString() : TEXT("NULL"), *Status.ToDebugString());
	}
};

/**
 * Delegate executed when new presence data is available for a user.
 *
 * @param UserId The unique id of the user whose presence was received.
 * @param Presence The received presence
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPresenceReceived, const class FUniqueNetId& /*UserId*/, const TSharedRef<FOnlineUserPresence>& /*Presence*/);
typedef FOnPresenceReceived::FDelegate FOnPresenceReceivedDelegate;

/**
 * Delegate executed when the array of presence data for a user changes.
 *
 * @param UserId The unique id of the user whose presence was received.
 * @param PresenceArray The updated presence array
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPresenceArrayUpdated, const class FUniqueNetId& /*UserId*/, const TArray<TSharedRef<FOnlineUserPresence> >& /*PresenceArray*/);
typedef FOnPresenceArrayUpdated::FDelegate FOnPresenceArrayUpdatedDelegate;

/**
 *	Interface class for getting and setting rich presence information.
 */
class IOnlinePresence
{
public:
	/** Virtual destructor to force proper child cleanup */
	virtual ~IOnlinePresence() = default;

	/**
	 * Delegate executed when setting or querying presence for a user has completed.
	 *
	 * @param UserId The unique id of the user whose presence was set.
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error.
	 */
	DECLARE_DELEGATE_TwoParams(FOnPresenceTaskCompleteDelegate, const class FUniqueNetId& /*UserId*/, const bool /*bWasSuccessful*/);

	/**
	 * Starts an async task that sets presence information for the user.
	 *
	 * @param User The unique id of the user whose presence is being set.
	 * @param Status The collection of state and key/value pairs to set as the user's presence data.
	 * @param Delegate The delegate to be executed when the potentially asynchronous set operation completes.
	 */
	virtual void SetPresence(const FUniqueNetId& User, const FOnlineUserPresenceStatus& Status, const FOnPresenceTaskCompleteDelegate& Delegate = FOnPresenceTaskCompleteDelegate()) = 0;

	/**
	 * Starts an async task that sets presence information for the user.
	 *
	 * @param User The unique id of the user whose presence is being set.
	 * @param Status The collection of state and key/value pairs to set as the user's presence data.
	 * @param Delegate The delegate to be executed when the potentially asynchronous set operation completes.
	 */
	UE_API virtual void SetPresence(const FUniqueNetId& User, FOnlinePresenceSetPresenceParameters&& Parameters, const FOnPresenceTaskCompleteDelegate& Delegate = FOnPresenceTaskCompleteDelegate());

	/**
	 * Starts an async operation that will update the cache with presence data from all users in the Users array.
	 * On platforms that support multiple keys, this function will query all keys.
	 *
	 * @param Users The list of unique ids of the users to query for presence information.
	 * @param Delegate The delegate to be executed when the potentially asynchronous query operation completes.
	 */
	virtual void QueryPresence(const FUniqueNetId& User, const FOnPresenceTaskCompleteDelegate& Delegate = FOnPresenceTaskCompleteDelegate()) = 0;

	/**
	 * Starts an async operation that will update the cache with presence data from all users in the Users array.
	 * On platforms that support multiple keys, this function will query all keys.
	 *
	 * @param Users The unique id of the user initiating the query for presence information.
	 * @param UserIds The list of unique ids of the users to query for presence information.
	 * @param Delegate The delegate to be executed when the potentially asynchronous query operation completes.
	 */
	virtual void QueryPresence(const FUniqueNetId& LocalUserId, const TArray<FUniqueNetIdRef>& UserIds, const FOnPresenceTaskCompleteDelegate& Delegate) {};

	/**
	 * Delegate executed when new presence data is available for a user.
	 *
	 * @param UserId The unique id of the user whose presence was received.
	 * @param Presence The presence that was received.
	 */
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnPresenceReceived, const class FUniqueNetId& /*UserId*/, const TSharedRef<FOnlineUserPresence>& /*Presence*/);

	/**
	 * Delegate executed when the array of presence data for a user changes.
	 *
	 * @param UserId The unique id of the user whose presence was received.
	 * @param PresenceArray The updated presence array
	 */
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnPresenceArrayUpdated, const class FUniqueNetId& /*UserId*/, const TArray<TSharedRef<FOnlineUserPresence> >& /*NewPresenceArray*/);

	/**
	 * Gets the cached presence information for a user.
	 *
	 * @param User The unique id of the user from which to get presence information.
	 * @param OutPresence If found, a shared pointer to the cached presence data for User will be stored here.
	 * @return Whether the data was found or not.
	 */
	virtual EOnlineCachedResult::Type GetCachedPresence(const FUniqueNetId& User, TSharedPtr<FOnlineUserPresence>& OutPresence) = 0;

	/**
	* Gets the cached presence information for a user.
	*
	* @param LocalUserId The unique id of the local user
	* @param User The unique id of the user from which to get presence information.
	* @param AppId The id of the application you want to get the presence of
	* @param OutPresence If found, a shared pointer to the cached presence data for User will be stored here.
	* @return Whether the data was found or not.
	*/
	virtual EOnlineCachedResult::Type GetCachedPresenceForApp(const FUniqueNetId& LocalUserId, const FUniqueNetId& User, const FString& AppId, TSharedPtr<FOnlineUserPresence>& OutPresence) = 0;
};

#undef UE_API
