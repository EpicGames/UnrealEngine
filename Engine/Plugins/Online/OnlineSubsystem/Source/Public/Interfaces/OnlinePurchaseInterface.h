// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineEntitlementsInterface.h"
#include "Interfaces/OnlineStoreInterfaceV2.h"
#include "Serialization/JsonSerializerMacros.h"

struct FOnlineError;

#define PURCHASE_REDEMPTION_CODE_USED "errors.com.epicgames.coderedemption.code_used"
#define PURCHASE_REDEMPTION_CODE_NOT_FOUND "errors.com.epicgames.coderedemption.code_not_found"
#define PURCHASE_REDEMPTION_NOT_ELIGIBLE "errors.com.epicgames.ecommerce.fulfillment.code.criteria.reject"

ONLINESUBSYSTEM_API DECLARE_LOG_CATEGORY_EXTERN(LogOnlinePurchase, Log, All);

#define UE_LOG_ONLINE_PURCHASE(Verbosity, Format, ...) \
{ \
	UE_LOGF(LogOnlinePurchase, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

#define UE_CLOG_ONLINE_PURCHASE(Conditional, Verbosity, Format, ...) \
{ \
	UE_CLOGF(Conditional, LogOnlinePurchase, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

/**
 * Info needed for checkout
 */
class FPurchaseCheckoutRequest
{
public:
	/**
	 * Add a offer entry for purchase
	 *
	 * @param InNamespace namespace of offer to be purchased
	 * @param InOfferId id of offer to be purchased
	 * @param InQuantity number to purchase
	 * @param bInIsConsumable is the offer consumable or one time purchase
	 */
	void AddPurchaseOffer(const FOfferNamespace& InNamespace, const FUniqueOfferId& InOfferId, int32 InQuantity, bool bInIsConsumable = true)
	{ 
		PurchaseOffers.Add(FPurchaseOfferEntry(InNamespace, InOfferId, InQuantity, bInIsConsumable));
	}
	/**
	 * Single offer entry for purchase 
	 */
	struct FPurchaseOfferEntry
	{
		FPurchaseOfferEntry(const FOfferNamespace& InOfferNamespace, const FUniqueOfferId& InOfferId, int32 InQuantity, bool bInIsConsumable)
			: OfferNamespace(InOfferNamespace)
			, OfferId(InOfferId)
			, Quantity(InQuantity)
		{ }

		/** Namespace in which the offer resides */
		FOfferNamespace OfferNamespace;
		/** Platform specific offer id (defined on backend) */
		FUniqueOfferId OfferId;
		/** Number of offers of this type to purchase */
		int32 Quantity;
	};	
	/** List of offers being purchased */
	TArray<FPurchaseOfferEntry> PurchaseOffers;

	/** Optional AccountId to include with the purchase request (if supported) */
	TOptional<FString> AccountId;
	
	/** Key - value pair properties for passing to the checkout. */
	TMap<FString, FString> AdditionalCheckoutProperties;

};

/**
 * State of a purchase transaction
 */
enum class EPurchaseTransactionState : uint8
{
	/** processing has not started on the purchase */
	NotStarted,
	/** currently processing the purchase */
	Processing,
	/** purchase completed successfully */
	Purchased,
	/** purchase completed but failed */
	Failed,
	/** purchase has been deferred (neither failed nor completed) */
	Deferred,
	/** purchase canceled by user */
	Canceled,
	/** prior purchase that has been restored */
	Restored,
	/** purchase failed as not allowed */
	NotAllowed,
	/** purchase failed as invalid */
	Invalid
};

/**
 * Receipt result from checkout
 */
class FPurchaseReceipt
	: public FJsonSerializable
{
public:
	FPurchaseReceipt()
	: TransactionState(EPurchaseTransactionState::NotStarted)
	{
	}

	struct FLineItemInfo
		: public FJsonSerializable
	{
		/** The platform identifier of this purchase type */
		FString ItemName;

		/** unique identifier representing this purchased item (the specific instance owned by this account) */
		FUniqueEntitlementId UniqueId;

		/** platform-specific opaque validation info (required to verify UniqueId belongs to this account) */
		FString ValidationInfo;

		inline bool IsRedeemable() const { return !ValidationInfo.IsEmpty(); }

		BEGIN_JSON_SERIALIZER
			JSON_SERIALIZE("itemName", ItemName);
			JSON_SERIALIZE("uniqueItemId", UniqueId);
			JSON_SERIALIZE("validationInfo", ValidationInfo);
		END_JSON_SERIALIZER
	};

	/**
	 * Single purchased offer offer
	 */
	struct FReceiptOfferEntry
		: public FJsonSerializable
	{
		FReceiptOfferEntry()
			: Quantity(0)
		{
		}

		FReceiptOfferEntry(const FOfferNamespace& InNamespace, const FUniqueOfferId& InOfferId, int32 InQuantity)
			: Namespace(InNamespace)
			, OfferId(InOfferId)
			, Quantity(InQuantity)
		{
		}

		FOfferNamespace Namespace;
		FUniqueOfferId OfferId;
		int32 Quantity;

		/** Information about the individual items purchased */
		TArray<FLineItemInfo> LineItems;

		BEGIN_JSON_SERIALIZER
			JSON_SERIALIZE("namespace", Namespace);
			JSON_SERIALIZE("offerId", OfferId);
			JSON_SERIALIZE("quantity", Quantity);
			JSON_SERIALIZE_ARRAY_SERIALIZABLE("items", LineItems, FLineItemInfo);
		END_JSON_SERIALIZER
	};

	/**
	 * Add a offer entry that has been purchased
	 *
	 * @param InNamespace of the offer that has been purchased
	 * @param InOfferId id of offer that has been purchased
	 * @param InQuantity number purchased
	 */
	void AddReceiptOffer(const FOfferNamespace& InNamespace, const FUniqueOfferId& InOfferId, int32 InQuantity)
	{
		ReceiptOffers.Add(FReceiptOfferEntry(InNamespace, InOfferId, InQuantity));
	}

	void AddReceiptOffer(const FReceiptOfferEntry& ReceiptOffer)
	{
		ReceiptOffers.Add(ReceiptOffer);
	}

	void AddReceiptOffer(FReceiptOfferEntry&& ReceiptOffer)
	{
		ReceiptOffers.Add(MoveTemp(ReceiptOffer));
	}

	BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE("transactionId", TransactionId);
		JSON_SERIALIZE_ARRAY_SERIALIZABLE("receiptList", ReceiptOffers, FReceiptOfferEntry);
	END_JSON_SERIALIZER

public:
	/** Unique Id for this transaction/order */
	FString TransactionId;
	/** Current state of the purchase */
	EPurchaseTransactionState TransactionState;

	/** List of offers that were purchased */
	TArray<FReceiptOfferEntry> ReceiptOffers;
};

/**
 * Info needed for code redemption
 */
class FRedeemCodeRequest
{
public:
	/** Code to redeem */
	FString Code;

	/** Optional CodeUseId that was given if code was previously locked before redeeming - See IOnlineCodeRedemption::LockCode */
	FString CodeUseId;

	/** Where this code is being fulfilled from - e.g. Launcher, GameName*/
	FString FulfillmentSource;
};

/**
 * Delegate called when checkout process completes
 */
DECLARE_DELEGATE_TwoParams(FOnPurchaseCheckoutComplete, const FOnlineError& /*Result*/, const TSharedRef<FPurchaseReceipt>& /*Receipt*/);

/**
* Delegate called when checkout process completes, this delegate is used when the entitlement or receipt information of the purchase is not needed by the caller.
*/
DECLARE_DELEGATE_OneParam(FOnPurchaseReceiptlessCheckoutComplete, const FOnlineError& /*Result*/);

/**
 * Delegate called when code redemption process completes
 */
DECLARE_DELEGATE_TwoParams(FOnPurchaseRedeemCodeComplete, const FOnlineError& /*Result*/, const TSharedRef<FPurchaseReceipt>& /*Receipt*/);

/**
 * Delegate called when query receipt process completes
 */
DECLARE_DELEGATE_OneParam(FOnQueryReceiptsComplete, const FOnlineError& /*Result*/);

/**
 * Delegate called when the information dialog shows and completes
 */
DECLARE_DELEGATE_OneParam(FOnShowAlternativeBillingInformationDialogComplete, const FOnlineError& /*Result*/);

/**
* Delegate called when receipt validation completes
*/
DECLARE_DELEGATE_TwoParams(FOnFinalizeReceiptValidationInfoComplete, const FOnlineError& /*Result*/, const FString& /*ValidationInfo*/);

/**
 * Delegate called when we are informed of a new receipt we did not initiate in-game
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnUnexpectedPurchaseReceipt, const FUniqueNetId& /*UserId*/);
typedef FOnUnexpectedPurchaseReceipt::FDelegate FOnUnexpectedPurchaseReceiptDelegate;

/**
* If the underlying store supports it,
* delegate which is executed when external purchase token request completes
*/
DECLARE_DELEGATE_TwoParams(FOnRequestExternalPurchaseTokenComplete, bool /*bWasSuccessful*/, FString /*Output*/);


/**
 * IOnlinePurchase - Interface for IAP (In App Purchases) services
 */
class IOnlinePurchase
{
public:
	virtual ~IOnlinePurchase() = default;

	/**
	 * Determine if user is allowed to purchase on this platform from a policy or permission perspective
	 *
	 * @param UserId user initiating the request
	 *
	 * @return true if user is allowed to make a purchase
	 */
	virtual bool IsAllowedToPurchase(const FUniqueNetId& UserId) = 0;

	/**
	 * Determine if user is allowed to purchase from store on this platform from a policy or permission perspective
	 *
	 * @param UserId user initiating the request
	 * 
	 * @param Error an error result indicating why the the user isn't allowed when relevant and available.
	 *
	 * @return true if user is allowed to make a purchase
	 */
	virtual bool IsAllowedToPurchase(const FUniqueNetId& UserId, FOnlineError& Error) = 0;

	/**
	 * Initiate the checkout process for purchasing offers via payment
	 *
	 * @param UserId user initiating the request
	 * @param CheckoutRequest info needed for the checkout request
	 * @param Delegate completion callback (guaranteed to be called)
	 */
	virtual void Checkout(const FUniqueNetId& UserId, const FPurchaseCheckoutRequest& CheckoutRequest, const FOnPurchaseCheckoutComplete& Delegate) = 0;


	/**
	 * Initiate the checkout process for purchasing offers via payment without receipts returned
	 * Recommended for use when the entitlement or receipt information of the purchase is not needed by the caller
	 * 
	 * @param UserId user initiating the request
	 * @param CheckoutRequest info needed for the checkout request
	 * @param Delegate completion callback (guaranteed to be called)
	 */
	virtual void Checkout(const FUniqueNetId& UserId, const FPurchaseCheckoutRequest& CheckoutRequest, const FOnPurchaseReceiptlessCheckoutComplete& Delegate) = 0;
	
	/**
	 * Finalizes a purchase with the supporting platform
	 * Acknowledges that the purchase has been properly redeemed by the application
	 *
	 * @param UserId user where the purchase was made
	 * @param ReceiptId purchase id for this platform
	 */
	virtual void FinalizePurchase(const FUniqueNetId& UserId, const FString& ReceiptId) = 0;

	/**
	 * Finalizes a purchase with the supporting platform
	 * Acknowledges that the purchase has been properly redeemed by the application. Prefer this overload whenever the validation info is available.
	 *
	 * @param UserId user where the purchase was made
	 * @param ReceiptId purchase id for this platform
	 * @param ReceiptInfo the receipt validation info for this platform
	 */
	virtual void FinalizePurchase(const FUniqueNetId& UserId, const FString& ReceiptId, const FString& ReceiptInfo)
	{
		// Redirect to the classic method for platforms that don't support this overload yet.
		FinalizePurchase(UserId, ReceiptId);
	}

	/**
	 * Initiate the checkout process for obtaining offers via code redemption
	 *
	 * @param UserId user initiating the request
	 * @param RedeemCodeRequest info needed for the redeem request
	 * @param Delegate completion callback (guaranteed to be called)
	 */
	virtual void RedeemCode(const FUniqueNetId& UserId, const FRedeemCodeRequest& RedeemCodeRequest, const FOnPurchaseRedeemCodeComplete& Delegate) = 0;

	/**
	 * Query for all of the user's receipts from prior purchases
	 *
	 * @param UserId user initiating the request
	 * @param bRestoreReceipts initiate recovery of any receipts on the specific platform
	 * @param Delegate completion callback (guaranteed to be called)
	 */
	virtual void QueryReceipts(const FUniqueNetId& UserId, bool bRestoreReceipts, const FOnQueryReceiptsComplete& Delegate) = 0;

	/**
	 * Get list of cached receipts for user (includes transactions currently being processed)
	 *
	 * @param UserId user initiating the request
	 * @param OutReceipts [out] list of receipts for the user 
	 */
	virtual void GetReceipts(const FUniqueNetId& UserId, TArray<FPurchaseReceipt>& OutReceipts) const = 0;

	virtual void FinalizeReceiptValidationInfo(const FUniqueNetId& UserId, FString& InReceiptValidationInfo, const FOnFinalizeReceiptValidationInfoComplete& Delegate) = 0;

	/**
	 * If the underlying IAP service requires an informational dialog to be shown, show it and trigger 
	 * the delegate on completion with the status
	 * @param Delegate completion callback (guaranteed to be called)
	 */
	virtual void ShowAlternativeBillingInformationDialog(const FOnShowAlternativeBillingInformationDialogComplete& Delegate) {}

	
	/**
	* If the underlying IAP service supports external purchase tokens, request external purchase token
	*
	* @param Delegate completion callback
	*/
	virtual void RequestExternalPurchaseToken(const FOnRequestExternalPurchaseTokenComplete& Delegate) const { Delegate.ExecuteIfBound(false, ""); }
	
	/**
	 * Delegate fired when the local system tells us of a new completed purchase we may not have initiated in-game.
	 * Use this to know about new pending receipts in instances the local client did not start a purchase,
	 * such as when the application is in the background.
	 *
	 * @param UserId The beneficiary of this new receipt
	 *
	 */
	DEFINE_ONLINE_DELEGATE_ONE_PARAM(OnUnexpectedPurchaseReceipt, const FUniqueNetId& /*UserId*/);

private:
	void OnRedirectToCheckoutComplete(const FOnlineError& ErrorResult, const TSharedRef<FPurchaseReceipt>& Receipt, const FOnPurchaseReceiptlessCheckoutComplete Delegate);
};

inline const TCHAR* LexToString(EPurchaseTransactionState State)
{
	switch (State)
	{
	case EPurchaseTransactionState::NotStarted: return TEXT("NotStarted");
	case EPurchaseTransactionState::Processing: return TEXT("Processing");
	case EPurchaseTransactionState::Purchased: return TEXT("Purchased");
	case EPurchaseTransactionState::Failed: return TEXT("Failed");
	case EPurchaseTransactionState::Deferred: return TEXT("Deferred");
	case EPurchaseTransactionState::Canceled: return TEXT("Canceled");
	case EPurchaseTransactionState::Restored: return TEXT("Restored");
	case EPurchaseTransactionState::NotAllowed: return TEXT("NotAllowed");
	case EPurchaseTransactionState::Invalid: return TEXT("Invalid");
	default: return TEXT("Unknown");
	}
}
