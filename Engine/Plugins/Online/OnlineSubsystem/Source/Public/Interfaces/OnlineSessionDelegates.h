// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

/**
 * Delegate fired when a session create request has completed
 *
 * @param SessionName the name of the session this callback is for
 * @param bWasSuccessful true if the async action completed without error, false if there was an error
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnCreateSessionComplete, FName, bool);
typedef FOnCreateSessionComplete::FDelegate FOnCreateSessionCompleteDelegate;

/**
 * Delegate fired when the online session has transitioned to the started state
 *
 * @param SessionName the name of the session the that has transitioned to started
 * @param bWasSuccessful true if the async action completed without error, false if there was an error
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnStartSessionComplete, FName, bool);
typedef FOnStartSessionComplete::FDelegate FOnStartSessionCompleteDelegate;

/**
 * Delegate fired when a update session request has completed
 *
 * @param SessionName the name of the session this callback is for
 * @param bWasSuccessful true if the async action completed without error, false if there was an error
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnUpdateSessionComplete, FName, bool);
typedef FOnUpdateSessionComplete::FDelegate FOnUpdateSessionCompleteDelegate;

/**
 * Delegate fired when the online session has transitioned to the ending state
 *
 * @param SessionName the name of the session the that was ended
 * @param bWasSuccessful true if the async action completed without error, false if there was an error
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnEndSessionComplete, FName, bool);
typedef FOnEndSessionComplete::FDelegate FOnEndSessionCompleteDelegate;

/**
 * Delegate fired when a destroying an online session has completed
 *
 * @param SessionName the name of the session this callback is for
 * @param bWasSuccessful true if the async action completed without error, false if there was an error
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnDestroySessionComplete, FName, bool);
typedef FOnDestroySessionComplete::FDelegate FOnDestroySessionCompleteDelegate;

/**
 * Broadcast delegate fired when the Matchmaking for an online session has completed
 *
 * @param SessionName the name of the session the that can now be joined (on success)
 * @param bWasSuccessful true if the async action completed without error, false if there was an error
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnMatchmakingComplete, FName, bool);
typedef FOnMatchmakingComplete::FDelegate FOnMatchmakingCompleteDelegate;

/**
 * Delegate fired when StartMatchmaking completes
 * Related to FOnMatchmakingComplete, but this is not a broadcast delegate
 *
 * @param SessionName the name of the session that was passed to StartMatchmaking
 * @param ErrorDetails extended details of the failure (if failed)
 * @param Results results of matchmaking (if succeeded)
 */
DECLARE_DELEGATE_ThreeParams(FOnStartMatchmakingComplete, FName /*SessionName*/, const struct FOnlineError& /*ErrorDetails*/, const struct FSessionMatchmakingResults& /*Results*/);

/**
 * Delegate fired when the Matchmaking request has been canceled
 *
 * @param SessionName the name of the session that was passed to CancelMatchmaking
 * @param bWasSuccessful true if the async action completed without error, false if there was an error
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnCancelMatchmakingComplete, FName, bool);
typedef FOnCancelMatchmakingComplete::FDelegate FOnCancelMatchmakingCompleteDelegate;

/**
 * Delegate fired when the search for an online session has completed
 *
 * @param bWasSuccessful true if the async action completed without error, false if there was an error
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnFindSessionsComplete, bool);
typedef FOnFindSessionsComplete::FDelegate FOnFindSessionsCompleteDelegate;

/**
 * Delegate fired when the cancellation of a search for an online session has completed
 *
 * @param bWasSuccessful true if the async action completed without error, false if there was an error
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnCancelFindSessionsComplete, bool);
typedef FOnCancelFindSessionsComplete::FDelegate FOnCancelFindSessionsCompleteDelegate;
