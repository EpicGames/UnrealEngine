// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemTypes.h"
#include "OnlineDelegateMacros.h"
#include "OnlineSessionDelegates.h"
#include "OnlineKeyValuePair.h"

class FOnlineSession;
class FOnlineSessionSearch;
class FOnlineSessionSearchResult;
class FOnlineSessionSettings;

ONLINESUBSYSTEM_API DECLARE_LOG_CATEGORY_EXTERN(LogOnlineSession, Log, All);

#define UE_LOG_ONLINE_SESSION(Verbosity, Format, ...) \
{ \
	UE_LOGF(LogOnlineSession, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

#define UE_CLOG_ONLINE_SESSION(Conditional, Verbosity, Format, ...) \
{ \
	UE_CLOGF(Conditional, LogOnlineSession, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

/** Possible results of a JoinSession attempt */
namespace EOnJoinSessionCompleteResult
{
	enum Type
	{
		/** The join worked as expected */
		Success,
		/** There are no open slots to join */
		SessionIsFull,
		/** The session couldn't be found on the service */
		SessionDoesNotExist,
		/** There was an error getting the session server's address */
		CouldNotRetrieveAddress,
		/** The user attempting to join is already a member of the session */
		AlreadyInSession,
		/** An error not covered above occurred */
		UnknownError
	};
}

/** Convert a EOnJoinSessionCompleteResult into a string */
inline const TCHAR* LexToString(const EOnJoinSessionCompleteResult::Type Value)
{
	switch (Value)
	{
	case EOnJoinSessionCompleteResult::Success:
		return TEXT("Success");
	case EOnJoinSessionCompleteResult::SessionIsFull:
		return TEXT("SessionIsFull");
	case EOnJoinSessionCompleteResult::SessionDoesNotExist:
		return TEXT("SessionDoesNotExist");
	case EOnJoinSessionCompleteResult::CouldNotRetrieveAddress:
		return TEXT("CouldNotRetrieveAddress");
	case EOnJoinSessionCompleteResult::AlreadyInSession:
		return TEXT("AlreadyInSession");
	case EOnJoinSessionCompleteResult::UnknownError:
		; // Intentional fall-through
	}
		
	return TEXT("UnknownError");
}

/**
 * Delegate fired when the joining process for an online session has completed
 *
 * @param SessionName the name of the session this callback is for
 * @param bWasSuccessful true if the async action completed without error, false if there was an error
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnJoinSessionComplete, FName, EOnJoinSessionCompleteResult::Type);
typedef FOnJoinSessionComplete::FDelegate FOnJoinSessionCompleteDelegate;

/**
 * Delegate fired when a player has joined a session
 *
 * @param SessionName The name of the session that changed
 * @param UniqueId The ID of the user who joined
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnSessionParticipantJoined, FName, const FUniqueNetId&);
typedef FOnSessionParticipantJoined::FDelegate FOnSessionParticipantJoinedDelegate;

/** Possible motives for a participant leaving an online session */
enum class EOnSessionParticipantLeftReason : uint8
{
	/** The participant left the session of their own accord */
	Left,
	/** The participant got disconnected from a session that is still active */
	Disconnected,
	/** The participant was forcefully removed from the session */
	Kicked,
	/** The session ended and the participant got removed from it */
	Closed
};

ONLINESUBSYSTEM_API const TCHAR* ToLogString(EOnSessionParticipantLeftReason LeaveReason);

/**
 * Delegate fired when a player has left a session
 *
 * @param SessionName The name of the session that changed
 * @param UniqueId The ID of the user who left
 * @param LeaveReason An enum defining the motive for the participant's leave
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnSessionParticipantLeft, FName, const FUniqueNetId&, EOnSessionParticipantLeftReason);
typedef FOnSessionParticipantLeft::FDelegate FOnSessionParticipantLeftDelegate;

/**
 * Delegate fired when session is requesting QOS measurements
 * @param The name of the session for which the measurements need to be made
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnQosDataRequested, FName);
typedef FOnQosDataRequested::FDelegate FOnQosDataRequestedDelegate;

/**
 * Delegate fired when a session's settings have changed
 * @param The name of the session that had its settings changed
 * @param The updated session settings
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnSessionSettingsUpdated, FName, const FOnlineSessionSettings&);
typedef FOnSessionSettingsUpdated::FDelegate FOnSessionSettingsUpdatedDelegate;

/**
 * Delegate fired when a session's member settings have changed
 * @param The name of the session that had a member's settings changed
 * @param The UniqueNetId of the member whose settings changed
 * @param The updated member settings
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnSessionParticipantSettingsUpdated, FName, const FUniqueNetId&, const FOnlineSessionSettings&);
typedef FOnSessionParticipantSettingsUpdated::FDelegate FOnSessionParticipantSettingsUpdatedDelegate;

/**
 * Delegate fired once a single search result is returned (ie friend invite / join)
 * Session has not been joined at this point, and requires a call to JoinSession()
 *
 * @param LocalUserNum the controller number of the accepting user
 * @param bWasSuccessful the session was found and is joinable, false otherwise
 * @param SearchResult the search/settings for the session result we've been given
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnSingleSessionResultComplete, int32, bool, const FOnlineSessionSearchResult&);
typedef FOnSingleSessionResultComplete::FDelegate FOnSingleSessionResultCompleteDelegate;

/**
* Delegate fired once a single search result is returned (ie friend invite / join)
* Session has not been joined at this point, and requires a call to JoinSession()
*
* @param LocalUserNum the controller number of the accepting user
* @param bWasSuccessful the session was found and is joinable, false otherwise
* @param SearchResult the search/settings for the session result we've been given
*/
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnFindFriendSessionComplete, int32, bool, const TArray<FOnlineSessionSearchResult>&);
typedef FOnFindFriendSessionComplete::FDelegate FOnFindFriendSessionCompleteDelegate;

/**
 * Delegate fired when an individual server's query has completed
 *
 * @param bWasSuccessful true if the async action completed without error, false if there was an error
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnPingSearchResultsComplete, bool);
typedef FOnPingSearchResultsComplete::FDelegate FOnPingSearchResultsCompleteDelegate;

/**
 * Called when a user accepts a session invitation. Allows the game code a chance
 * to clean up any existing state before accepting the invite. The invite must be
 * accepted by calling JoinSession() after clean up has completed
 *
 * @param bWasSuccessful true if the async action completed without error, false if there was an error
 * @param ControllerId the controller number of the accepting user
 * @param UserId the user being invited
 * @param InviteResult the search/settings for the session we're joining via invite
 */
DECLARE_MULTICAST_DELEGATE_FourParams(FOnSessionUserInviteAccepted, const bool, const int32, FUniqueNetIdPtr, const FOnlineSessionSearchResult&);
typedef FOnSessionUserInviteAccepted::FDelegate FOnSessionUserInviteAcceptedDelegate;

/**
 * Called when a user receives a session invitation. Allows the game code to decide
 * on accepting the invite. The invite can be accepted by calling JoinSession()
 *
 * @param UserId the user being invited
 * @param FromId the user that sent the invite
 * @param AppId the id of the client/app user was in when sending hte game invite
 * @param InviteResult the search/settings for the session we're joining via invite
 */
DECLARE_MULTICAST_DELEGATE_FourParams(FOnSessionInviteReceived, const FUniqueNetId& /*UserId*/, const FUniqueNetId& /*FromId*/, const FString& /*AppId*/, const FOnlineSessionSearchResult& /*InviteResult*/);
typedef FOnSessionInviteReceived::FDelegate FOnSessionInviteReceivedDelegate;

/**
 * Delegate fired when the session registration process has completed
 *
 * @param SessionName the name of the session the player joined or not
 * @param Players the players that were registered from the online service
 * @param bWasSuccessful true if the async action completed without error, false if there was an error
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnRegisterPlayersComplete, FName, const TArray< FUniqueNetIdRef >&, bool);
typedef FOnRegisterPlayersComplete::FDelegate FOnRegisterPlayersCompleteDelegate;

/**
 * Delegate fired when the un-registration process has completed
 *
 * @param SessionName the name of the session the player left
 * @param PlayerId the players that were unregistered from the online service
 * @param bWasSuccessful true if the async action completed without error, false if there was an error
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnUnregisterPlayersComplete, FName, const TArray< FUniqueNetIdRef >&, bool);
typedef FOnUnregisterPlayersComplete::FDelegate FOnUnregisterPlayersCompleteDelegate;

/**
 * Delegate fired when local player registration has completed
 */
DECLARE_DELEGATE_TwoParams(FOnRegisterLocalPlayerCompleteDelegate, const FUniqueNetId&, EOnJoinSessionCompleteResult::Type);

/**
 * Delegate fired when local player unregistration has completed
 */
DECLARE_DELEGATE_TwoParams(FOnUnregisterLocalPlayerCompleteDelegate, const FUniqueNetId&, const bool);	

/** Possible reasons for the service to cause a session failure */
namespace ESessionFailure
{
	enum Type
	{
		/** General loss of connection */
		ServiceConnectionLost
	};
}

/** Convert a ESessionFailure into a string */
inline const TCHAR* LexToString(const ESessionFailure::Type Value)
{
	switch (Value)
	{
	case ESessionFailure::ServiceConnectionLost:
		return TEXT("ServiceConnectionLost");
	}

	return TEXT("UnknownError");
}

/**
 * Delegate fired when an unexpected error occurs that impacts session connectivity or use
 *
 * @param PlayerId The player impacted by the failure (may be empty if unknown, or if all players are affected)
 * @param FailureType What kind of failure occurred 
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnSessionFailure, const FUniqueNetId&, ESessionFailure::Type);
typedef FOnSessionFailure::FDelegate FOnSessionFailureDelegate;

/**
 * Delegate called when the service requests that we destroy the session.
 * The game should react by cleaning up any state associated with the session (leaving the match, etc), and calling DestroySession.
 *
 * @param LocalUserNum The index for the player that made the request
 * @param SessionName The identifier for the session to be destroyed
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnDestroySessionRequested, int32 /*LocalUserNum*/, FName /*SessionName*/);
typedef FOnDestroySessionRequested::FDelegate FOnDestroySessionRequestedDelegate;

/** Attributes for a matchmaking user */
struct FSessionMatchmakingUser
{
	/** Id of the user */
	FUniqueNetIdRef UserId;
	/** Attributes for the user */
	FOnlineKeyValuePairs<FString, FVariantData> Attributes;
};

/** Matchmaking results */
struct FSessionMatchmakingResults
{
	// Stub struct that can be easily added to without requiring delegate signature changes
};

/**
 * Interface definition for the online services session services 
 * Session services are defined as anything related managing a session 
 * and its state within a platform service
 */
class IOnlineSession
{
protected:

	/** Hidden on purpose */
	IOnlineSession() {};

	/**
	 * Adds a new named session to the list (new session)
	 *
	 * @param SessionName the name to search for
	 * @param GameSettings the game settings to add
	 *
	 * @return a pointer to the struct that was added
	 */
	virtual class FNamedOnlineSession* AddNamedSession(FName SessionName, const FOnlineSessionSettings& SessionSettings) = 0;

	/**
	 * Adds a new named session to the list (from existing session data)
	 *
	 * @param SessionName the name to search for
	 * @param GameSettings the game settings to add
	 *
	 * @return a pointer to the struct that was added
	 */
	virtual class FNamedOnlineSession* AddNamedSession(FName SessionName, const FOnlineSession& Session) = 0;

public:
	virtual ~IOnlineSession() = default;

	/**
	 * Create a session id from a string
	 * @param SessionIdStr the string representation of the session id
	 * @return a session id, or nullptr if SessionIdStr is invalid
	 */
	virtual FUniqueNetIdPtr CreateSessionIdFromString(const FString& SessionIdStr) = 0;

	/**
	 * Searches the named session array for the specified session
	 *
	 * @param SessionName the name to search for
	 *
	 * @return pointer to the struct if found, NULL otherwise
	 */
	virtual class FNamedOnlineSession* GetNamedSession(FName SessionName) = 0;

	/**
	 * Searches the named session array for the specified session and removes it
	 *
	 * @param SessionName the name to search for
	 */
	virtual void RemoveNamedSession(FName SessionName) = 0;

	/**
	 * Searches the named session array for any presence enabled session
	 */
	virtual bool HasPresenceSession() = 0;

	/**
	 * Get the current state of a named session
	 *
	 * @param SessionName name of session to query
	 *
	 * @return State of specified session
	 */
	virtual EOnlineSessionState::Type GetSessionState(FName SessionName) const = 0;

	/**
	 * Creates an online session based upon the settings object specified.
	 * NOTE: online session registration is an async process and does not complete
	 * until the OnCreateSessionComplete delegate is called.
	 *
	 * @param HostingPlayerNum the index of the player hosting the session
	 * @param SessionName the name to use for this session so that multiple sessions can exist at the same time
	 * @param NewSessionSettings the settings to use for the new session
	 *
	 * @return true if successful creating the session, false otherwise
	 */
	virtual bool CreateSession(int32 HostingPlayerNum, FName SessionName, const FOnlineSessionSettings& NewSessionSettings) = 0;

	/**
	 * Creates an online session based upon the settings object specified.
	 * NOTE: online session registration is an async process and does not complete
	 * until the OnCreateSessionComplete delegate is called.
	 *
	 * @param HostingPlayerId the index of the player hosting the session
	 * @param SessionName the name to use for this session so that multiple sessions can exist at the same time
	 * @param NewSessionSettings the settings to use for the new session
	 *
	 * @return true if successful creating the session, false otherwise
	 */
	virtual bool CreateSession(const FUniqueNetId& HostingPlayerId, FName SessionName, const FOnlineSessionSettings& NewSessionSettings) = 0;

	/**
	* Delegate fired when a session create request has completed
	*
	* @param SessionName the name of the session this callback is for
	* @param bWasSuccessful true if the async action completed without error, false if there was an error
	*/
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnCreateSessionComplete, FName, bool);

	/**
	 * Marks an online session as in progress (as opposed to being in lobby or pending)
	 *
	 * @param SessionName the name of the session that is being started
	 *
	 * @return true if the call succeeds, false otherwise
	 */
	virtual bool StartSession(FName SessionName) = 0;

	/**
	 * Delegate fired when the online session has transitioned to the started state
	 *
	 * @param SessionName the name of the session the that has transitioned to started
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 */
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnStartSessionComplete, FName, bool);

	/**
	 * Updates the localized settings/properties for the session in question
	 *
	 * @param SessionName the name of the session to update
	 * @param UpdatedSessionSettings the object to update the session settings with
	 * @param bShouldRefreshOnlineData whether to submit the data to the backend or not
	 *
	 * @return true if successful creating the session, false otherwise
	 */
	virtual bool UpdateSession(FName SessionName, FOnlineSessionSettings& UpdatedSessionSettings, bool bShouldRefreshOnlineData = true) = 0;

	/**
	 * Delegate fired when a update request has completed
	 *
	 * @param SessionName the name of the session this callback is for
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 */
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnUpdateSessionComplete, FName, bool);

	/**
	 * Marks an online session as having been ended
	 *
	 * @param SessionName the name of the session the to end
	 *
	 * @return true if the call succeeds, false otherwise
	 */
	virtual bool EndSession(FName SessionName) = 0;

	/**
	 * Delegate fired when the online session has transitioned to the ending state
	 *
	 * @param SessionName the name of the session the that was ended
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 */
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnEndSessionComplete, FName, bool);

	/**
	 * Destroys the specified online session
	 * NOTE: online session de-registration is an async process and does not complete
	 * until the OnDestroySessionComplete delegate is called.
	 *
	 * @param SessionName the name of the session to delete
	 *
	 * @return true if successful destroying the session, false otherwise
	 */
	virtual bool DestroySession(FName SessionName, const FOnDestroySessionCompleteDelegate& CompletionDelegate = FOnDestroySessionCompleteDelegate()) = 0;

	/**
	 * Delegate fired when a destroying an online session has completed
	 *
	 * @param SessionName the name of the session this callback is for
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 */
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnDestroySessionComplete, FName, bool);

	/**
	 * Determine if the player is registered in the specified session
	 *
	 * @param UniqueId the player to check if in session or not
	 * @return true if the player is registered in the session
	 */
	virtual bool IsPlayerInSession(FName SessionName, const FUniqueNetId& UniqueId) = 0;

	/**
	 * Begins cloud based matchmaking for a session
	 *
	 * @param LocalPlayers the ids of all local players that will participate in the match
	 * @param SessionName the name of the session to use, usually will be GAME_SESSION_NAME
	 * @param NewSessionSettings the desired settings to match against or create with when forming new sessions
	 * @param SearchSettings the desired settings that the matched session will have
	 *
	 * @return true if successful searching for sessions, false otherwise
	 */
	virtual bool StartMatchmaking(const TArray< FUniqueNetIdRef >& LocalPlayers, FName SessionName, const FOnlineSessionSettings& NewSessionSettings, TSharedRef<FOnlineSessionSearch>& SearchSettings) = 0;

	/**
	 * Begins cloud based matchmaking for a session
	 *
	 * @param LocalPlayers the ids of all local players that will participate in the match
	 * @param SessionName the name of the session to use, usually will be GAME_SESSION_NAME
	 * @param NewSessionSettings the desired settings to match against or create with when forming new sessions
	 * @param SearchSettings the desired settings that the matched session will have
	 *
	 * @return true if successful searching for sessions, false otherwise
	 */
	virtual bool ONLINESUBSYSTEM_API StartMatchmaking(const TArray<FSessionMatchmakingUser>& LocalPlayers, FName SessionName, const FOnlineSessionSettings& NewSessionSettings, TSharedRef<FOnlineSessionSearch>& SearchSettings, const FOnStartMatchmakingComplete& CompletionDelegate);
	
	/**
	 * Delegate fired when the cloud matchmaking has completed
	 *
	 * @param SessionName The name of the session that was found via matchmaking
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 */
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnMatchmakingComplete, FName, bool);

	/**
	 * Cancel a Matchmaking request for a given session name
	 *
	 * @param SearchingPlayerNum the index of the player canceling the search
	 * @param SessionName the name of the session that was passed to StartMatchmaking (or CreateSession)
	 */ 
	virtual bool CancelMatchmaking(int32 SearchingPlayerNum, FName SessionName) = 0;

	/**
	 * Cancel a Matchmaking request for a given session name
	 *
	 * @param SearchingPlayerId the id of the player canceling the search
	 * @param SessionName the name of the session that was passed to StartMatchmaking (or CreateSession)
	 */ 
	virtual bool CancelMatchmaking(const FUniqueNetId& SearchingPlayerId, FName SessionName) = 0;

	/**
	 * Delegate fired when the cloud matchmaking has been canceled
	 *
	 * @param SessionName the name of the session that was canceled
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 */
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnCancelMatchmakingComplete, FName, bool);

	/**
	 * Searches for sessions matching the settings specified
	 *
	 * @param SearchingPlayerNum the index of the player searching for a match
	 * @param SearchSettings the desired settings that the returned sessions will have
	 *
	 * @return true if successful searching for sessions, false otherwise
	 */
	virtual bool FindSessions(int32 SearchingPlayerNum, const TSharedRef<FOnlineSessionSearch>& SearchSettings) = 0;

	/**
	 * Searches for sessions matching the settings specified
	 *
	 * @param SearchingPlayerId the id of the player searching for a match
	 * @param SearchSettings the desired settings that the returned sessions will have
	 *
	 * @return true if successful searching for sessions, false otherwise
	 */
	virtual bool FindSessions(const FUniqueNetId& SearchingPlayerId, const TSharedRef<FOnlineSessionSearch>& SearchSettings) = 0;

	/**
	 * Delegate fired when the search for an online session has completed
	 *
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 */
	DEFINE_ONLINE_DELEGATE_ONE_PARAM(OnFindSessionsComplete, bool);

	/**
	 * Find a single advertised session by session id
	 *
	 * @param SearchingUserId user initiating the request
	 * @param Platform platform the session is on
	 * @param SessionId session id to search for
	 * @param FriendId optional id of user to verify in session
	 * @param CompletionDelegate delegate to call on completion
	 *
	 * @return true on success, false otherwise
	 */
	virtual bool FindSessionById(const FUniqueNetId& SearchingUserId, const FUniqueNetId& SessionId, const FUniqueNetId& FriendId, const FOnSingleSessionResultCompleteDelegate& CompletionDelegate) = 0;

	/**
	* Find a single advertised session by session id (with userdata)
	*
	* @param SearchingUserId user initiating the request
	* @param Platform platform the session is on
	* @param SessionId session id to search for
	* @param FriendId optional id of user to verify in session
	* @param UserData optional data that may be required by search
	* @param CompletionDelegate delegate to call on completion
	*
	* @return true on success, false otherwise
	*/
	virtual bool FindSessionById(const FUniqueNetId& SearchingUserId, const FUniqueNetId& SessionId, const FUniqueNetId& FriendId, const FString& UserData, const FOnSingleSessionResultCompleteDelegate& CompletionDelegate)
	{
		return FindSessionById(SearchingUserId, SessionId, FriendId, CompletionDelegate);
	}

	/**
	 * Cancels the current search in progress if possible for that search type
	 *
	 * @return true if successful searching for sessions, false otherwise
	 */
	virtual bool CancelFindSessions() = 0;

	/**
	 * Delegate fired when the cancellation of a search for an online session has completed
	 *
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 */
	DEFINE_ONLINE_DELEGATE_ONE_PARAM(OnCancelFindSessionsComplete, bool);

	/**
	 * Fetches the additional data a session exposes outside of the online service.
	 * NOTE: notifications will come from the OnPingSearchResultsComplete delegate
	 *
	 * @param SearchResult the specific search result to query
	 *
	 * @return true if the query was started, false otherwise
	 */
	virtual bool PingSearchResults(const FOnlineSessionSearchResult& SearchResult) = 0;

	/**
	 * Delegate fired when an individual server's query has completed
	 *
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 */
	DEFINE_ONLINE_DELEGATE_ONE_PARAM(OnPingSearchResultsComplete, bool);

	/**
	 * Joins the session specified
	 *
	 * @param LocalUserNum the index of the player searching for a match
	 * @param SessionName the name of the session to join
	 * @param DesiredSession the desired session to join
	 *
	 * @return true if the call completed successfully, false otherwise
	 */
	virtual bool JoinSession(int32 LocalUserNum, FName SessionName, const FOnlineSessionSearchResult& DesiredSession) = 0;

	/**
	 * Joins the session specified
	 *
	 * @param LocalUserId the id of the player searching for a match
	 * @param SessionName the name of the session to join
	 * @param DesiredSession the desired session to join
	 *
	 * @return true if the call completed successfully, false otherwise
	 */
	virtual bool JoinSession(const FUniqueNetId& LocalUserId, FName SessionName, const FOnlineSessionSearchResult& DesiredSession) = 0;


	/**
	 * Delegate fired when the process for a local user joining an online session has completed
	 *
	 * @param SessionName the name of the session this callback is for
	 * @param Result EOnJoinSessionCompleteResult describing the outcome of the call 
	 */
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnJoinSessionComplete, FName, EOnJoinSessionCompleteResult::Type);

	/**
	* Delegate fired when a player joins a session
	* @param SessionName The name of the session
	* @param UniqueId The ID of the user who joined
	*/
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnSessionParticipantJoined, FName, const FUniqueNetId&);

	/**
	* Delegate fired when a player leaves a session
	* @param SessionName The name of the session
	* @param UniqueId The ID of the user who left
	* @param LeaveReason An enum defining the motive for the participant's leave
	*/
	DEFINE_ONLINE_DELEGATE_THREE_PARAM(OnSessionParticipantLeft, FName, const FUniqueNetId&, EOnSessionParticipantLeftReason);

	/**
	 * Delegate fired when session is requesting QOS measurements
	 * @param The name of the session for which the measurements need to be made
	 */
	DEFINE_ONLINE_DELEGATE_ONE_PARAM(OnQosDataRequested, FName);

	/**
	 * Delegate fired when a session's settings have changed
	 *
	 * @param SessionName The name of the session that had its settings changed
	 * @param SessionSettings The updated session settings
	 */
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnSessionSettingsUpdated, FName, const FOnlineSessionSettings&);

	/**
	 * Delegate fired when a session's member settings have changed
	 * 
	 * @param SessionName The name of the session that had a member's settings changed
	 * @param TargetUniqueNetId The UniqueNetId of the member whose settings changed
	 * @param SessionSettings The updated member settings
	 */
	DEFINE_ONLINE_DELEGATE_THREE_PARAM(OnSessionParticipantSettingsUpdated, FName, const FUniqueNetId&, const FOnlineSessionSettings&);

	/**
	 * Allows the local player to follow a friend into a session
	 *
	 * @param LocalUserNum the local player wanting to join
	 * @param Friend the player that is being followed
	 *
	 * @return true if the async call worked, false otherwise
	 */
	virtual bool FindFriendSession(int32 LocalUserNum, const FUniqueNetId& Friend) = 0;

	/**
	 * Allows the local player to follow a friend into a session
	 *
	 * @param LocalUserId the local player wanting to join
	 * @param Friend the player that is being followed
	 *
	 * @return true if the async call worked, false otherwise
	 */
	virtual bool FindFriendSession(const FUniqueNetId& LocalUserId, const FUniqueNetId& Friend) = 0;

	/**
	* Allows the local player to retrieve the session of multiple friends
	*
	* @param LocalUserId the local player wanting to join
	* @param FriendList the potential players to follow
	*
	* @return true if the async call worked, false otherwise
	*/
	virtual bool FindFriendSession(const FUniqueNetId& LocalUserId, const TArray<FUniqueNetIdRef>& FriendList) = 0;

	/**
	 * Delegate fired once the find friend task has completed
	 * Session has not been joined at this point, and still requires a call to JoinSession()
	 *
	 * @param LocalUserNum the controller number of the accepting user
	 * @param bWasSuccessful the session was found and is joinable, false otherwise
	 * @param FriendSearchResult the search/settings for the session we're attempting to join
	 */
	DEFINE_ONLINE_PLAYER_DELEGATE_TWO_PARAM(MAX_LOCAL_PLAYERS, OnFindFriendSessionComplete, bool, const TArray<FOnlineSessionSearchResult>&);

	/**
	 * Sends an invitation to play in the player's current session
	 *
	 * @param LocalUserNum the user that is sending the invite
	 * @param SessionName session to invite them to
	 * @param Friend the player to send the invite to
	 *
	 * @return true if successful, false otherwise
	 */
	virtual bool SendSessionInviteToFriend(int32 LocalUserNum, FName SessionName, const FUniqueNetId& Friend) = 0;

	/**
	 * Sends an invitation to play in the player's current session
	 *
	 * @param LocalUserId the user that is sending the invite
	 * @param SessionName session to invite them to
	 * @param Friend the player to send the invite to
	 *
	 * @return true if successful, false otherwise
	 */
	virtual bool SendSessionInviteToFriend(const FUniqueNetId& LocalUserId, FName SessionName, const FUniqueNetId& Friend) = 0;

	/**
	 * Sends invitations to play in the player's current session
	 *
	 * @param LocalUserNum the user that is sending the invite
	 * @param SessionName session to invite them to
	 * @param Friends the player to send the invite to
	 *
	 * @return true if successful, false otherwise
	 */
	virtual bool SendSessionInviteToFriends(int32 LocalUserNum, FName SessionName, const TArray< FUniqueNetIdRef >& Friends) = 0;

	/**
	 * Sends invitations to play in the player's current session
	 *
	 * @param LocalUserId the user that is sending the invite
	 * @param SessionName session to invite them to
	 * @param Friends the player to send the invite to
	 *
	 * @return true if successful, false otherwise
	 */
	virtual bool SendSessionInviteToFriends(const FUniqueNetId& LocalUserId, FName SessionName, const TArray< FUniqueNetIdRef >& Friends) = 0;

	/**
	 * Called when a user accepts a session invitation. Allows the game code a chance
	 * to clean up any existing state before accepting the invite. The invite must be
	 * accepted by calling JoinSession() after clean up has completed
	 *
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 * @param ControllerId the controller number of the accepting user
	 * @param UserId the user being invited
	 * @param InviteResult the search/settings for the session we're joining via invite
	 */
	DEFINE_ONLINE_DELEGATE_FOUR_PARAM(OnSessionUserInviteAccepted, const bool /*bWasSuccessful*/, const int32 /*ControllerId*/, FUniqueNetIdPtr /*UserId*/, const FOnlineSessionSearchResult& /*InviteResult*/);

	/**
	 * Called when a user receives a session invitation. Allows the game code to decide
	 * on accepting the invite. The invite can be accepted by calling JoinSession()
	 *
	 * @param UserId the user being invited
	 * @param FromId the user that sent the invite
	 * @param AppId the id of the client/app user was in when sending hte game invite
	 * @param InviteResult the search/settings for the session we're joining via invite
	 */
	DEFINE_ONLINE_DELEGATE_FOUR_PARAM(OnSessionInviteReceived, const FUniqueNetId& /*UserId*/, const FUniqueNetId& /*FromId*/, const FString& /*AppId*/, const FOnlineSessionSearchResult& /*InviteResult*/);

	/**
	 * Returns the platform specific connection information for joining the match.
	 * Call this function from the delegate of join completion
	 *
	 * @param SessionName the name of the session to resolve
	 * @param ConnectInfo the string containing the platform specific connection information
	 * @param PortType type of port to append to result (Game, Beacon, etc)
	 *
	 * @return true if the call was successful, false otherwise
	 */
	virtual bool GetResolvedConnectString(FName SessionName, FString& ConnectInfo, FName PortType = NAME_GamePort) = 0;

	/**
	 * Returns the platform specific connection information for joining a search result.
	 *
	 * @param SearchResult the search result to get connection info from
	 * @param PortType type of port to append to result (Game, Beacon, etc)
	 * @param ConnectInfo the string containing the platform specific connection information
	 *
	 * @return true if the call was successful, false otherwise
	 */
	virtual bool GetResolvedConnectString(const class FOnlineSessionSearchResult& SearchResult, FName PortType, FString& ConnectInfo) = 0;

	/**
	 * Returns the session settings object for the session with a matching name
	 *
	 * @param SessionName the name of the session to return
	 *
	 * @return the settings for this session name
	 */
	virtual FOnlineSessionSettings* GetSessionSettings(FName SessionName) = 0;

	/**
	 * Returns the name for the Voice Chat room, if any were created by setting bUseLobbiesVoiceChatIfAvailable to true during session creation
	 * 
	 * @param LocalUserNum the user asking for the room name
	 * @param SessionName the name of the related session
	 * 
	 * @return the name for the corresponding Voice Chat room, or an empty FString if there is none
	 */
	virtual FString ONLINESUBSYSTEM_API GetVoiceChatRoomName(int32 LocalUserNum, const FName& SessionName);

	/**
	 * Registers a player with the online service as being part of the online session
	 *
	 * @param SessionName the name of the session the player is joining
	 * @param UniquePlayerId the player to register with the online service
	 * @param bWasInvited whether the player was invited to the session or searched for it
	 *
	 * @return true if the call succeeds, false otherwise
	 */
	virtual bool RegisterPlayer(FName SessionName, const FUniqueNetId& PlayerId, bool bWasInvited) = 0;

	/**
	 * Registers a group of players with the online service as being part of the online session
	 *
	 * @param SessionName the name of the session the player is joining
	 * @param Players the list of players to register with the online service
	 * @param bWasInvited was this list of players invited
	 *
	 * @return true if the call succeeds, false otherwise
	 */
	virtual bool RegisterPlayers(FName SessionName, const TArray< FUniqueNetIdRef >& Players, bool bWasInvited = false) = 0;

	/**
	 * Delegate fired when the session registration process has completed
	 *
	 * @param SessionName the name of the session the player joined or not
	 * @param PlayerId the player that was registered in the online service
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 */
	DEFINE_ONLINE_DELEGATE_THREE_PARAM(OnRegisterPlayersComplete, FName, const TArray< FUniqueNetIdRef >&, bool);

	/**
	 * Unregisters a player with the online service as being part of the online session
	 *
	 * @param SessionName the name of the session the player is leaving
	 * @param PlayerId the player to unregister with the online service
	 *
	 * @return true if the call succeeds, false otherwise
	 */
	virtual bool UnregisterPlayer(FName SessionName, const FUniqueNetId& PlayerId) = 0;

	/**
	 * Unregisters a group of players with the online service as being part of the online session
	 *
	 * @param SessionName the name of the session the player is joining
	 * @param Players the list of players to unregister with the online service
	 *
	 * @return true if the call succeeds, false otherwise
	 */
	virtual bool UnregisterPlayers(FName SessionName, const TArray< FUniqueNetIdRef >& Players) = 0;

	/**
	 * Delegate fired when the un-registration process has completed
	 *
	 * @param SessionName the name of the session the player left
	 * @param PlayerId the player that was unregistered from the online service
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 */
	DEFINE_ONLINE_DELEGATE_THREE_PARAM(OnUnregisterPlayersComplete, FName, const TArray< FUniqueNetIdRef >&, bool);

	/**
	 * Registers a local player with a session.
	 *
	 * @param PlayerId the player to register
	 * @param SessionName the session in which to register the player
	 * @param Delegate the delegate executed when the asynchronous operation completes
	 */
	virtual void RegisterLocalPlayer(const FUniqueNetId& PlayerId, FName SessionName, const FOnRegisterLocalPlayerCompleteDelegate& Delegate) = 0;

	/**
	 * Unregisters a local player with a session.
	 *
	 * @param PlayerId the player to unregister
	 * @param SessionName the session in which to unregister the player
	 * @param Delegate the delegate executed when the asynchronous operation completes
	 */
	virtual void UnregisterLocalPlayer(const FUniqueNetId& PlayerId, FName SessionName, const FOnUnregisterLocalPlayerCompleteDelegate& Delegate) = 0;

	/**
	* Removes a player from the session.
	*
	* @param LocalUserNum the user that is starting the removal
	* @param SessionName the session in which to unregister the player
	* @param TargetPlayerId the player to remove
	*/
	virtual void ONLINESUBSYSTEM_API RemovePlayerFromSession(int32 LocalUserNum, FName SessionName, const FUniqueNetId& TargetPlayerId);

	/**
	 * Delegate fired when an unexpected error occurs that impacts session connectivity or use
	 *
	 * @param PlayerId The player impacted by the failure (may be empty if unknown, or if all players are affected)
	 * @param FailureType What kind of failure occured 
	 */
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnSessionFailure, const FUniqueNetId&, ESessionFailure::Type);

	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnDestroySessionRequested, int32 /*LocalUserNum*/, FName /*SessionName*/);

	/**
	 * Gets the number of known sessions registered with the interface
	 */
	virtual int32 GetNumSessions() = 0;

	/**
	 *	Dumps out the session state for all known sessions
	 */
	virtual void DumpSessionState() = 0;
};

typedef TSharedPtr<IOnlineSession, ESPMode::ThreadSafe> IOnlineSessionPtr;
