// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once

#include "Delegates/Delegate.h"
#include "OnlineDelegateMacros.h"

class FSharedContentHandle;
class FUniqueNetId;

/**
 * Delegate fired when a shared file read from the network platform's storage is complete
 *
 * @param bWasSuccessful whether the file read was successful or not
 * @param SharedHandle the handle of the read content
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnReadSharedFileComplete, bool, const FSharedContentHandle&);
typedef FOnReadSharedFileComplete::FDelegate FOnReadSharedFileCompleteDelegate;

/**
 * Delegate fired when a shared file write to the network platform's storage is complete
 *
 * @param bWasSuccessful whether the file Write was successful or not
 * @param UserId User owning the storage
 * @param Filename the name of the file this was for
 * @param SharedHandle the handle to the shared file, may be platform dependent
 */
DECLARE_MULTICAST_DELEGATE_FourParams(FOnWriteSharedFileComplete, bool, const FUniqueNetId&, const FString&, const TSharedRef<FSharedContentHandle>&);
typedef FOnWriteSharedFileComplete::FDelegate FOnWriteSharedFileCompleteDelegate;


/**
 * Provides the interface for sharing files already on the cloud with other users
 */
class IOnlineSharedCloud
{
protected:
	IOnlineSharedCloud() {};

public:
	virtual ~IOnlineSharedCloud() = default;

	/**
	 * Copies the shared data into the specified buffer for the specified file
	 *
	 * @param SharedHandle the name of the file to read
	 * @param FileContents the out buffer to copy the data into
	 *
	 * @return true if the data was copied, false otherwise
	 */
	virtual bool GetSharedFileContents(const FSharedContentHandle& SharedHandle, TArray<uint8>& FileContents) = 0;

	/**
	 * Empties the set of all downloaded files if possible (no async tasks outstanding)
	 *
	 * @return true if they could be deleted, false if they could not
	 */
	virtual bool ClearSharedFiles() = 0;

	/**
	 * Empties the cached data for this file if it is not being downloaded currently
	 *
	 * @param SharedHandle the name of the file to read
	 *
	 * @return true if it could be deleted, false if it could not
	 */
	virtual bool ClearSharedFile(const FSharedContentHandle& SharedHandle) = 0;

	/**
	 * Starts an asynchronous read of the specified shared file from the network platform's file store
	 *
	 * @param SharedHandle the name of the file to read
	 *
	 * @return true if the calls starts successfully, false otherwise
	 */
	virtual bool ReadSharedFile(const FSharedContentHandle& SharedHandle) = 0;

	/**
	 * Delegate fired when a shared file read from the network platform's storage is complete
	 *
	 * @param bWasSuccessful whether the content read was successful or not
	 * @param SharedHandle the handle of the read content
	 */
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnReadSharedFileComplete, bool, const FSharedContentHandle&);

	/**
	 * Starts an asynchronous write of the specified shared file to the network platform's file store
	 *
	 * @param UserId User owning the storage
	 * @param Filename the name of the file to write
	 * @param Contents data to write to the file
	 *
	 * @return true if the calls starts successfully, false otherwise
	 */
	virtual bool WriteSharedFile(const FUniqueNetId& UserId, const FString& Filename, TArray<uint8>& Contents) = 0;

	/**
	 * Delegate fired when a shared file write to the network platform's storage is complete
	 *
	 * @param bWasSuccessful whether the file Write was successful or not
	 * @param UserId User owning the storage
	 * @param Filename the name of the file this was for
	 * @param SharedHandle the handle to the shared file, may be platform dependent
	 */
	DEFINE_ONLINE_DELEGATE_FOUR_PARAM(OnWriteSharedFileComplete, bool, const FUniqueNetId&, const FString&, const TSharedRef<FSharedContentHandle>&);

	/** Interface to get some test content handles */
	virtual void GetDummySharedHandlesForTest(TArray< TSharedRef<FSharedContentHandle> > & OutHandles) = 0;
};

typedef TSharedPtr<IOnlineSharedCloud, ESPMode::ThreadSafe> IOnlineSharedCloudPtr;
