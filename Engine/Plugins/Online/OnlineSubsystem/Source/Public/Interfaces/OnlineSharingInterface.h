// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemTypes.h"
#include "OnlineDelegateMacros.h"

ONLINESUBSYSTEM_API DECLARE_LOG_CATEGORY_EXTERN(LogOnlineSharing, Log, All);

#define UE_LOG_ONLINE_SHARING(Verbosity, Format, ...) \
{ \
	UE_LOGF(LogOnlineSharing, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

#define UE_CLOG_ONLINE_SHARING(Conditional, Verbosity, Format, ...) \
{ \
	UE_CLOGF(Conditional, LogOnlineSharing, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

/**
 * Called to notify of a post being shared on the server
 *
 * @param LocalUserNum		- The controller number of the associated user
 * @param bWasSuccessful	- True if a post was successfully uploaded
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnSharePostComplete, int32, bool);
typedef FOnSharePostComplete::FDelegate FOnSharePostCompleteDelegate;


/**
 * Called to notify that a read request for a news feed has completed.
 *
 * @param LocalUserNum		- The controller number of the associated user
 * @param bWasSuccessful	- True if the read request completed successfully
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnReadNewsFeedComplete, int32, bool);
typedef FOnReadNewsFeedComplete::FDelegate FOnReadNewsFeedCompleteDelegate;

/**
 * Called to notify that read permissions have been updated on the server
 *
 * @param LocalUserNum		- The controller number of the associated user
 * @param bWasSuccessful	- True if the read permissions updated successfully
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnRequestNewReadPermissionsComplete, int32, bool);
typedef FOnRequestNewReadPermissionsComplete::FDelegate FOnRequestNewReadPermissionsCompleteDelegate;



/**
 * Called to notify that publish permissions have been updated on the server
 *
 * @param LocalUserNum		- The controller number of the associated user
 * @param bWasSuccessful	- True if the publish permissions updated successfully
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnRequestNewPublishPermissionsComplete, int32, bool);
typedef FOnRequestNewPublishPermissionsComplete::FDelegate FOnRequestNewPublishPermissionsCompleteDelegate;


/**
 *	FOnlineStatusUpdate - Object which hosts all the content we will post to a news feed
 */
struct FOnlineStatusUpdate
{
	/** Which type of status update for this post. May be ignored for some platforms */
	FString Type;

	/** The text which forms this post */
	FString Message;
	
	/** An accompanying image for this post, if desired */
	struct FImage* Image;

	/** A list of friends which are included in the post */
	TArray<FUniqueNetIdRef > TaggedFriends;

	/** The privacy of this post */
	EOnlineStatusUpdatePrivacy PostPrivacy;

	/**
	 * Default Constructor
	 */
	FOnlineStatusUpdate() 
		: Type(TEXT("Default"))
		, Image(NULL)
		, PostPrivacy(EOnlineStatusUpdatePrivacy::OnlyMe)
	{}
};

/**
 * Contains information about a single permission granted by a backend service
 */
struct FSharingPermission
{

public:
	
	/** Name of the platform specific permission */
	FString Name;
	/** Type of permission */
	EOnlineSharingCategory Type;
	/** Status of the permission (granted/declined) */
	EOnlineSharingPermissionState Status;

	FSharingPermission()
		: Type(EOnlineSharingCategory::None)
		, Status(EOnlineSharingPermissionState::Unknown)
	{
	}

	explicit FSharingPermission(const FString& InName, EOnlineSharingCategory InType)
		: Name(InName)
		, Type(InType)
		, Status(EOnlineSharingPermissionState::Unknown)
	{
	}

	inline bool operator==(const FSharingPermission& Other) const
	{
		return (Status == Other.Status) && (Type == Other.Type) && (Name == Other.Name);
	}
};

/**
 * Called when a current permissions query has completed
 *
 * @param LocalUserNum		- The controller number of the associated user
 * @param bWasSuccessful	- True if the query completed successfully
 * @param Permissions		- Current state of permissions
 */
DECLARE_DELEGATE_ThreeParams(FOnRequestCurrentPermissionsComplete, int32 /*LocalUserNum*/, bool /*bWasSuccessful*/, const TArray<FSharingPermission>& /*Permissions*/);

/**
 *	IOnlineSharing - Interface class for sharing.
 */
class IOnlineSharing
{
public:
	virtual ~IOnlineSharing() = default;

	///////////////////////////////////////////////////////////
	// PERMISSIONS

	/**
	 * Request the current set of permissions across all sharing
	 *
	 * @param LocalUserNum the controller number of the associated user
	 * @param CompletionDelegate the delegate to fire when the operation is completed
	 */
	virtual void RequestCurrentPermissions(int32 LocalUserNum, FOnRequestCurrentPermissionsComplete& CompletionDelegate) = 0;

	/**
	 * @return the currently cached permissions for a given user
	 */
	virtual void GetCurrentPermissions(int32 LocalUserNum, TArray<FSharingPermission>& OutPermissions) = 0;
	
	/**
	 * Called when we have successfully/failed to read a news feed from the server
	 *
	 * @param LocalUserNum the controller number of the associated user
	 * @param bWasSuccessful true if server was contacted and the feed was read
	 */
	DEFINE_ONLINE_PLAYER_DELEGATE_ONE_PARAM(MAX_LOCAL_PLAYERS, OnRequestNewReadPermissionsComplete, bool);


	/**
	 * Request a new set of read permissions from our OSS
	 *
	 * @param LocalUserNum		- The controller number of the associated user
	 * @param NewPermissions	- The read permissions we are requesting authorization for.
	 */
	virtual bool RequestNewReadPermissions(int32 LocalUserNum, EOnlineSharingCategory NewPermissions) = 0;

	/**
	 * Called when we have successfully/failed to read a news feed from the server
	 *
	 * @param LocalUserNum the controller number of the associated user
	 * @param bWasSuccessful true if server was contacted and the feed was read
	 */
	DEFINE_ONLINE_PLAYER_DELEGATE_ONE_PARAM(MAX_LOCAL_PLAYERS, OnRequestNewPublishPermissionsComplete, bool);
	

	/**
	 * Request a new set of publish permissions from our OSS
	 *
	 * @param LocalUserNum		- The controller number of the associated user
	 * @param NewPermissions	- The publish permissions we are requesting authorization for.
	 */
	virtual bool RequestNewPublishPermissions(int32 LocalUserNum, EOnlineSharingCategory NewPermissions, EOnlineStatusUpdatePrivacy Privacy) = 0;

public:

	///////////////////////////////////////////////////////////
	// READING FROM SERVER


	/**
	 * Called when we have successfully/failed to read a news feed from the server
	 *
	 * @param LocalUserNum the controller number of the associated user
	 * @param bWasSuccessful true if server was contacted and the feed was read
	 */
	DEFINE_ONLINE_PLAYER_DELEGATE_ONE_PARAM(MAX_LOCAL_PLAYERS, OnReadNewsFeedComplete, bool);


	/**
	 * Read the status feed of the user specified.
	 *
	 * @param LocalUserNum the controller number of the associated user
	 *
	 * @return - Whether a server request was executed
	 */
	virtual bool ReadNewsFeed(int32 LocalUserNum, int32 NumPostsToRead = 50) = 0;


	/**
	 * Get a news feed object which was previously synced from the server
	 *
	 * @param LocalUserId - The id of the user we are reading news feeds for
	 * @param NewsFeedIdx - The index of the news feed we are looking up
	 * @param OutNewsFeed - The news feed object we are searching for
	 *
	 * @return Whether the news feed was obtained
	 */
	virtual EOnlineCachedResult::Type GetCachedNewsFeed(int32 LocalUserNum, int32 NewsFeedIdx, FOnlineStatusUpdate& OutNewsFeed) = 0;


	/**
	 * Get all the status update objects for the specified local user
	 *
	 * @param LocalUserId - The local user Id of the user we are obtaining status updates for
	 * @param OutNewsFeeds - The collection of news feeds obtained from the server for the given player
	 *
	 * @return Whether news feeds were obtained
	 */
	virtual EOnlineCachedResult::Type GetCachedNewsFeeds(int32 LocalUserNum, TArray<FOnlineStatusUpdate>& OutNewsFeeds) = 0;

public:

	///////////////////////////////////////////////////////////
	// PUBLISHING TO SERVER


	/**
	 * Called when we have successfully/failed to post a status update to the server
	 *
	 * @param LocalUserNum		- The controller number of the associated user
	 * @param bWasSuccessful	- True if a post was successfully added to the server
	 */
	DEFINE_ONLINE_PLAYER_DELEGATE_ONE_PARAM(MAX_LOCAL_PLAYERS, OnSharePostComplete, bool);
	

	/**
	 * Set the online permissions for reading a users data.
	 *
	 * @param LocalUserNum	- The controller number of the associated user
	 * @param StatusUpdate	- The message we are posting to the feed.
	 *
	 * @return - Whether a server request was executed
	 */
	virtual bool ShareStatusUpdate(int32 LocalUserNum, const FOnlineStatusUpdate& StatusUpdate) = 0;
};
