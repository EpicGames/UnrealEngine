// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Online/CoreOnline.h"
#include "OnlineKeyValuePair.h"

struct FOnlineError;

using FOnlineStatValue = FVariantData;

ONLINESUBSYSTEM_API DECLARE_LOG_CATEGORY_EXTERN(LogOnlineStats, Log, All);

#define UE_LOG_ONLINE_STATS(Verbosity, Format, ...) \
{ \
	UE_LOGF(LogOnlineStats, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

/** Object to represent a new stat value and how to use it in relation to previous values */
struct FOnlineStatUpdate
{
public:
	/** How should we modify this stat in relation to previous values? */
	enum class EOnlineStatModificationType : uint8
	{
		/** Let the backend decide how to update this value (or set to new value if backend does not decide) */
		Unknown,
		/** Add the new value to the previous value */
		Sum,
		/** Overwrite previous value with the new value */
		Set,
		/** Only replace previous value if new value is larger */
		Largest,
		/** Only replace previous value if new value is smaller */
		Smallest
	};

	/** Construct an empty FOnlineStatUpdate */
	FOnlineStatUpdate()
		: NewValue(0)
		, ModificationType(EOnlineStatModificationType::Unknown)
	{
	}

	/** Construct a new FOnlineStatUpdate from a value */
	FOnlineStatUpdate(const FOnlineStatValue& InNewValue, const EOnlineStatModificationType InModificationType)
		: NewValue(InNewValue)
		, ModificationType(InModificationType)
	{
	}

	FOnlineStatUpdate(FOnlineStatValue&& InNewValue, const EOnlineStatModificationType InModificationType)
		: NewValue(MoveTemp(InNewValue))
		, ModificationType(InModificationType)
	{
	}

	/** Copy/Assignment construction */
	FOnlineStatUpdate(FOnlineStatUpdate&& Other) = default;
	FOnlineStatUpdate(const FOnlineStatUpdate& Other) = default;
	FOnlineStatUpdate& operator=(FOnlineStatUpdate&& Other) = default;
	FOnlineStatUpdate& operator=(const FOnlineStatUpdate& Other) = default;

	/** Set this stat update to a new value/modification type */
	void Set(const FOnlineStatValue& InNewValue, const EOnlineStatModificationType InModificationType)
	{
		NewValue = InNewValue;
		ModificationType = InModificationType;
	}

	void Set(FOnlineStatValue&& InNewValue, const EOnlineStatModificationType InModificationType)
	{
		NewValue = MoveTemp(InNewValue);
		ModificationType = InModificationType;
	}

	/** Get the current value */
	const FOnlineStatValue& GetValue() const
	{
		return NewValue;
	}

	/** Get the current modification type */
	EOnlineStatModificationType GetModificationType() const
	{
		return ModificationType;
	}

	/** Get the current type of stat (int32, float, etc) */
	EOnlineKeyValuePairDataType::Type GetType() const
	{
		return NewValue.GetType();
	}

	/** Get the value of this stat as a string */
	FString ToString() const
	{
		return NewValue.ToString();
	}

	/** Returns true if this stat is numeric */
	bool IsNumeric() const
	{
		return NewValue.IsNumeric();
	}

private:
	template<typename T>
	T SwitchOnNumeric(const T& Left, const T& Right, EOnlineStatModificationType ModType) const
	{
		switch (ModificationType)
		{
		case EOnlineStatModificationType::Sum:
			return Left + Right;
		case EOnlineStatModificationType::Set:
			return Right;
		case EOnlineStatModificationType::Largest:
			return (Left > Right) ? Left : Right;
		case EOnlineStatModificationType::Smallest:
			return (Left > Right) ? Right : Left;
		case EOnlineStatModificationType::Unknown:
		default:
			return Right; //default- new
		}
	}

public:
	/** Takes the value saved here and its corresponding ModificationType and outputs the new final value. New final value should be directly set onto the preexisting value */
	FOnlineStatValue GetResult(const FOnlineStatValue& Other) const
	{
		if(Other.GetType() != NewValue.GetType())
		{
			// mismatched- just return the new value
			return NewValue;
		}

		switch(Other.GetType())
		{
		case EOnlineKeyValuePairDataType::Int32:
		{
			int32 Old, New;
			Other.GetValue(Old);
			NewValue.GetValue(New);

			FVariantData ReturnData;
			ReturnData.SetValue(SwitchOnNumeric(Old, New, ModificationType));
			return ReturnData;
		}
		case EOnlineKeyValuePairDataType::Int64:
		{
			int64 Old, New;
			Other.GetValue(Old);
			NewValue.GetValue(New);

			FVariantData ReturnData;
			ReturnData.SetValue(SwitchOnNumeric(Old,New, ModificationType));
			return ReturnData;
		}
		case EOnlineKeyValuePairDataType::UInt32:
		{
			uint32 Old, New;
			Other.GetValue(Old);
			NewValue.GetValue(New);

			FVariantData ReturnData;
			ReturnData.SetValue(SwitchOnNumeric(Old, New, ModificationType));
			return ReturnData;
		}
		case EOnlineKeyValuePairDataType::UInt64:
		{
			uint64 Old, New;
			Other.GetValue(Old);
			NewValue.GetValue(New);
			FVariantData ReturnData;
			ReturnData.SetValue(SwitchOnNumeric(Old, New, ModificationType));
			return ReturnData;
		}
		case EOnlineKeyValuePairDataType::Float:
		{
			float Old, New;
			Other.GetValue(Old);
			NewValue.GetValue(New);

			FVariantData ReturnData;
			ReturnData.SetValue(SwitchOnNumeric(Old, New, ModificationType));
			return ReturnData;
		}
		case EOnlineKeyValuePairDataType::Double:
		{
			double Old, New;
			Other.GetValue(Old);
			NewValue.GetValue(New);

			FVariantData ReturnData;
			ReturnData.SetValue(SwitchOnNumeric(Old, New, ModificationType));
			return ReturnData;
		}
		case EOnlineKeyValuePairDataType::String:
		default:
		{
			return NewValue; // only valid operation here is Set
		}
		}
	}

private:
	FOnlineStatValue NewValue;
	EOnlineStatModificationType ModificationType;
};

template <class TStatType>
struct FOnlineUserStatsPair
{
public:
	FOnlineUserStatsPair(const FUniqueNetIdRef InAccount)
		: Account(InAccount)
	{
		check(Account->IsValid());
	}

	FOnlineUserStatsPair(const FUniqueNetIdRef InAccount, const TMap<FString, TStatType>& InStats)
		: Account(InAccount)
		, Stats(InStats)
	{
		check(Account->IsValid());
	}

	FOnlineUserStatsPair(const FUniqueNetIdRef InAccount, TMap<FString, TStatType>&& InStats)
		: Account(InAccount)
		, Stats(MoveTemp(InStats))
	{
		check(Account->IsValid());
	}

public:
	FUniqueNetIdRef Account;
	TMap<FString, TStatType> Stats;

	using StatType = TStatType;
};

/** Delegate called when a stat update has completed, with a ResultState parameter to represent success or failure */
DECLARE_DELEGATE_OneParam(FOnlineStatsUpdateStatsComplete, const FOnlineError& /*ResultState*/);

/** A pair of a user and an array of their stats */
using FOnlineStatsUserStats = FOnlineUserStatsPair<FOnlineStatValue>;

/** Delegate called when a user's stats have finished being queried, with a ResultState parameter to represent success or failure */
DECLARE_DELEGATE_TwoParams(FOnlineStatsQueryUserStatsComplete, const FOnlineError& /*ResultState*/, const TSharedPtr<const FOnlineStatsUserStats>& /*QueriedStats*/);

/** Delegate called when multiple users' stats have finished being queried, with a ResultState parameter to represent success or failure */
DECLARE_DELEGATE_TwoParams(FOnlineStatsQueryUsersStatsComplete, const FOnlineError& /*ResultState*/, const TArray<TSharedRef<const FOnlineStatsUserStats>>& /*UsersStatsResult*/);

/** A pair of a user and an array of their stats to be updated */
using FOnlineStatsUserUpdatedStats = FOnlineUserStatsPair<FOnlineStatUpdate>;

/** An interface to update stat backends with */
class IOnlineStats
{

public:
	virtual ~IOnlineStats() = default;

public:
	/**
	 * Query a specific user's stats
	 *
	 * @param LocalUserId User to query as (if applicable)
	 * @param StatsUser User to get stats for
	 * @param Delegate Called when the user's stats have finished being requested and are now available, or when we fail to retrieve the user's stats
	 */
	virtual void QueryStats(const FUniqueNetIdRef LocalUserId, const FUniqueNetIdRef StatsUser, const FOnlineStatsQueryUserStatsComplete& Delegate) = 0;

	/**
	 * Query a one or more user's stats
	 *
	 * @param LocalUserId User to query as (if applicable)
	 * @param StatsUser Users to get stats for
	 * @param StatNames Stats to get stats for all specified users
	 * @param Delegate Called when the user's stats have finished being requested and are now available, or when we fail to retrieve the user's stats
	 */
	virtual void QueryStats(const FUniqueNetIdRef LocalUserId, const TArray<FUniqueNetIdRef>& StatUsers, const TArray<FString>& StatNames, const FOnlineStatsQueryUsersStatsComplete& Delegate) = 0;

	/**
	 * Get a user's cached stats object
	 *
	 * @param StatsUserId The user to get stats for
	 * @return The results if cached, else a null pointer
	 */
	virtual TSharedPtr<const FOnlineStatsUserStats> GetStats(const FUniqueNetIdRef StatsUserId) const = 0;

	/**
	 * Asynchronous update one or more user's stats
	 *
	 * @param LocalUserId The user to update the stats as (if applicable)
	 * @param UpdatedStats The array of user to stats pairs to update the backend with
	 * @param Delegate Called when update has completed
	 */
	virtual void UpdateStats(const FUniqueNetIdRef LocalUserId, const TArray<FOnlineStatsUserUpdatedStats>& UpdatedUserStats, const FOnlineStatsUpdateStatsComplete& Delegate) = 0;

#if !UE_BUILD_SHIPPING
	/**
	 * Request the stats reset, for debugging purposes
	 *
	 * @param StatsUserId The user who's stats are to be deleted
	 */
	virtual void ResetStats( const FUniqueNetIdRef StatsUserId ) = 0;
#endif // !UE_BUILD_SHIPPING

};
