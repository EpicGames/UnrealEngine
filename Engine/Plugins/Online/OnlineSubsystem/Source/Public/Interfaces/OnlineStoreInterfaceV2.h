// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Internationalization/Text.h"
#include "Misc/DateTime.h"
#include "OnlineDelegateMacros.h"
#include "OnlineStoreInterfaceV2.generated.h"

ONLINESUBSYSTEM_API DECLARE_LOG_CATEGORY_EXTERN(LogOnlineStoreV2, Log, All);

#define UE_LOG_ONLINE_STOREV2(Verbosity, Format, ...) \
{ \
	UE_LOGF(LogOnlineStoreV2, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

#define UE_CLOG_ONLINE_STOREV2(Conditional, Verbosity, Format, ...) \
{ \
	UE_CLOGF(Conditional, LogOnlineStoreV2, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

class FUniqueNetId;

typedef FString FUniqueOfferId;
typedef FString FOfferNamespace;
typedef FString FUniqueCategoryId;

/**
 * Possible result states of an in-app purchase transaction
 */
UENUM(BlueprintType)
namespace EInAppPurchaseState
{
	enum Type : int
	{
		Unknown = 0 UMETA(DisplayName = "Unknown"),
		Success UMETA(DisplayName = "Success"),
		Failed UMETA(DisplayName = "Failed"),
		Cancelled UMETA(DisplayName = "Cancelled"),
		Invalid UMETA(DisplayName = "Invalid"),
		NotAllowed UMETA(DisplayName = "NotAllowed"),
		Restored UMETA(DisplayName = "Restored"),
		AlreadyOwned UMETA(DisplayName = "AlreadyOwned"),
	};
}

enum class EOnlineStoreOfferDiscountType : uint8
{
	/** Offer isn't on sale*/
	NotOnSale = 0u,
	/** Offer price should be displayed as a percentage of regular price */
	Percentage,
	/** Offer price should be displayed as an amount off regular price */
	DiscountAmount,
	/** Offer price should be displayed as a new price */
	PayAmount
};

namespace EOnlineStoreOfferDiscount
{
	inline EOnlineStoreOfferDiscountType FromString(const TCHAR* const String)
	{
		if (FCString::Stricmp(String, TEXT("Percentage")) == 0)
		{
			return EOnlineStoreOfferDiscountType::Percentage;
		}
		else if (FCString::Stricmp(String, TEXT("DiscountAmount")) == 0)
		{
			return EOnlineStoreOfferDiscountType::DiscountAmount;
		}
		else if (FCString::Stricmp(String, TEXT("PayAmount")) == 0)
		{
			return EOnlineStoreOfferDiscountType::PayAmount;
		}
		else
		{
			return EOnlineStoreOfferDiscountType::NotOnSale;
		}
	}
}

/**
 * Offer entry for display from online store
 */
class FOnlineStoreOffer
{
public:
	FOnlineStoreOffer()
	: RegularPrice(-1)
	, NumericPrice(-1)
	, ReleaseDate(0)
	, ExpirationDate(FDateTime::MaxValue())
	, DiscountType(EOnlineStoreOfferDiscountType::NotOnSale)
	{
	}

	virtual ~FOnlineStoreOffer() = default;

	/** Unique offer identifier */
	FUniqueOfferId OfferId;

	/** Title for display */
	FText Title;
	/** Short description for display */
	FText Description;
	/** Full description for display */
	FText LongDescription;

	/** Regular non-sale price as text for display */
	FText RegularPriceText;
	/** Regular non-sale price in numeric form for comparison/sorting */
	int64 RegularPrice;

	/** Final-Pricing (Post-Sales/Discounts) as text for display */
	FText PriceText;
	/** Final-Price (Post-Sales/Discounts) in numeric form for comparison/sorting */
	int64 NumericPrice;

	/** Price currency code */
	FString CurrencyCode;

	/** Date the offer was released */
	FDateTime ReleaseDate;
	/** Date this information is no longer valid (maybe due to sale ending, etc) */
	FDateTime ExpirationDate;
	/** Type of discount currently running on this offer (if any) */
	EOnlineStoreOfferDiscountType DiscountType;

	TMap<FString, FString> DynamicFields;

	/** @return FText suitable for localized display */
	virtual FText GetDisplayRegularPrice() const
	{
		if (!RegularPriceText.IsEmpty())
		{
			return RegularPriceText;
		}
		else
		{
			return FText::AsCurrencyBase(RegularPrice, CurrencyCode);
		}
	}

	/** @return FText suitable for localized display */
	virtual FText GetDisplayPrice() const
	{
		if (!PriceText.IsEmpty())
		{
			return PriceText;
		}
		else
		{
			return FText::AsCurrencyBase(NumericPrice, CurrencyCode);
		}
	}

	/** @return True if offer can be purchased */
	virtual bool IsPurchaseable() const
	{
		return true;
	}
};
typedef TSharedRef<FOnlineStoreOffer> FOnlineStoreOfferRef;

/**
 * Category used to organize offers in the online store
 */
class FOnlineStoreCategory
{
public:
	/** Unique identifier for this category */
	FUniqueCategoryId Id;
	/** Description for display */
	FText Description;
	/** List of optional sub categories */
	TArray<FOnlineStoreCategory> SubCategories;
};

/**
 * Filter for querying a subset of offers from the online store
 */
class FOnlineStoreFilter
{
public:
	/** Keyword strings to match when filtering items/offers */
	TArray<FString> Keywords;
	/** Category paths to match when filtering offers */
	TArray<FOnlineStoreCategory> IncludeCategories;
	/** Category paths to exclude when filtering offers */
	TArray<FOnlineStoreCategory> ExcludeCategories;
};

/**
 * Delegate called when available online categories have been queried
 */
DECLARE_DELEGATE_TwoParams(FOnQueryOnlineStoreCategoriesComplete, bool /*bWasSuccessful*/, const FString& /*Error*/);

/**
 * Delegate called when online store query completes
 */
DECLARE_DELEGATE_ThreeParams(FOnQueryOnlineStoreOffersComplete, bool /*bWasSuccessful*/, const TArray<FUniqueOfferId>& /*OfferIds*/, const FString& /*Error*/);

/**
 * Delegate fired when a session create request has completed
 *
 * @param SessionName the name of the session this callback is for
 * @param bWasSuccessful true if the async action completed without error, false if there was an error
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnQueryForAvailablePurchasesComplete, bool);
typedef FOnQueryForAvailablePurchasesComplete::FDelegate FOnQueryForAvailablePurchasesCompleteDelegate;

/**
 *	Access to available offers for purchase
 */
class IOnlineStoreV2
{
public:

	virtual ~IOnlineStoreV2()= default;

	/**
	* Delegate which is executed when QueryForAvailablePurchases completes
	*/
	DEFINE_ONLINE_DELEGATE_ONE_PARAM(OnQueryForAvailablePurchasesComplete, bool);

	/**
	 * Query for available store categories. Delegate callback is guaranteed.
	 *
	 * @param UserId user initiating the request
	 * @param Delegate completion callback
	 */
	virtual void QueryCategories(const FUniqueNetId& UserId, const FOnQueryOnlineStoreCategoriesComplete& Delegate = FOnQueryOnlineStoreCategoriesComplete()) = 0;

	/**
	 * Get currently cached store categories
	 *
	 * @param OutCategories [out] list of categories previously queried
	 */
	virtual void GetCategories(TArray<FOnlineStoreCategory>& OutCategories) const = 0;

	/**
	 * Query for available store offers using a filter. Delegate callback is guaranteed.
	 *
	 * @param UserId user initiating the request
	 * @param Filter only return offers matching the filter
	 * @param Delegate completion callback
	 */
	virtual void QueryOffersByFilter(const FUniqueNetId& UserId, const FOnlineStoreFilter& Filter, const FOnQueryOnlineStoreOffersComplete& Delegate = FOnQueryOnlineStoreOffersComplete()) = 0;

	/**
	 * Query for available store offers matching the given ids. Delegate callback is guaranteed.
	 *
	 * @param UserId user initiating the request
	 * @param OfferIds only return offers matching these ids
	 * @param Delegate completion callback
	 *
	 * @return true if async operation started
	 */
	virtual void QueryOffersById(const FUniqueNetId& UserId, const TArray<FUniqueOfferId>& OfferIds, const FOnQueryOnlineStoreOffersComplete& Delegate = FOnQueryOnlineStoreOffersComplete()) = 0;

	/**
	 * Get currently cached store offers
	 *
	 * @param OutOffers [out] list of offers previously queried
	 */
	virtual void GetOffers(TArray<FOnlineStoreOfferRef>& OutOffers) const = 0;

	/**
	 * Get currently cached store offer entry
	 *
	 * @param OfferId id of offer to find
	 *
	 * @return offer if found or null
	 */
	virtual TSharedPtr<FOnlineStoreOffer> GetOffer(const FUniqueOfferId& OfferId) const = 0;

	/**
	* Return store country if it has been fetched.
	*/
	virtual FString GetStoreCountry() const { return ""; }
};
