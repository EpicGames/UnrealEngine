// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "OnlineDelegateMacros.h"

/**
 * Called when the time request from the server is complete
 *
 * @param bWasSuccessful true if server was contacted and a valid result received
 * @param DateTimeStr string representing UTC server time (yyyy.MM.dd-HH.mm.ss)
 * @param Error string representing the error condition
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnQueryServerUtcTimeComplete, bool, const FString&, const FString&);
typedef FOnQueryServerUtcTimeComplete::FDelegate FOnQueryServerUtcTimeCompleteDelegate;

/**
 * Interface for querying server time from an online service
 */
class IOnlineTime
{

protected:
	IOnlineTime() {};

public:
	virtual ~IOnlineTime() = default;

	/**
	 * Send request for current UTC time from the server
	 *
	 * @return true if the query was started
	 */
	virtual bool QueryServerUtcTime() = 0;

	/**
	 * Called when the time request from the server is complete
	 *
	 * @param bWasSuccessful true if server was contacted and a valid result received
	 * @param DateTimeStr string representing UTC server time (yyyy.MM.dd-HH.mm.ss)
	 * @param Error string representing the error condition
	 */
	DEFINE_ONLINE_DELEGATE_THREE_PARAM(OnQueryServerUtcTimeComplete, bool, const FString&, const FString&);

	/**
	 * Retrieve cached timestamp from last server time query 
	 *
	 * @return string representation of time (yyyy.MM.dd-HH.mm.ss)
	 */
	virtual FString GetLastServerUtcTime() = 0;
};

typedef TSharedPtr<IOnlineTime, ESPMode::ThreadSafe> IOnlineTimePtr;
