// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemTypes.h"
#include "OnlineDelegateMacros.h"

ONLINESUBSYSTEM_API DECLARE_LOG_CATEGORY_EXTERN(LogOnlineTitleFile, Log, All);

#define UE_LOG_ONLINE_TITLEFILE(Verbosity, Format, ...) \
{ \
	UE_LOGF(LogOnlineTitleFile, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

#define UE_CLOG_ONLINE_TITLEFILE(Conditional, Verbosity, Format, ...) \
{ \
	UE_CLOGF(Conditional, LogOnlineTitleFile, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

struct FAnalyticsEventAttribute;

/**
 * Delegate fired when the list of files has been returned from the network store
 *
 * @param bWasSuccessful whether the file list was successful or not
 * @param Error string representing the error condition
 *
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnEnumerateFilesComplete, bool, const FString&);
typedef FOnEnumerateFilesComplete::FDelegate FOnEnumerateFilesCompleteDelegate;

/**
 * Delegate fired when as file read from the network platform's storage progresses
 *
 * @param FileName the name of the file this was for
 * @param NumBytes the number of bytes read so far
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnReadFileProgress, const FString&, uint64);
typedef FOnReadFileProgress::FDelegate FOnReadFileProgressDelegate;

/**
 * Delegate fired when a file read from the network platform's storage is complete
 *
 * @param bWasSuccessful whether the file read was successful or not
 * @param FileName the name of the file this was for
 *
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnReadFileComplete, bool, const FString&);
typedef FOnReadFileComplete::FDelegate FOnReadFileCompleteDelegate;

/**
 * Delegate fired when we would like to report an event to an analytics provider.
 * 
 * @param EventName the name of the event
 * @param Attributes the key/value pairs of analytics event attributes to include with the event
 */
 DECLARE_MULTICAST_DELEGATE_TwoParams(FOnTitleFileAnalyticsEvent, const FString& /* EventName */, const TArray<FAnalyticsEventAttribute>& /* Attributes */)
 typedef FOnTitleFileAnalyticsEvent::FDelegate FOnTitleFileAnalyticsEventDelegate;

class IOnlineTitleFile
{
protected:
	IOnlineTitleFile() {};

public:
	virtual ~IOnlineTitleFile() = default;

	/**
	 * Copies the file data into the specified buffer for the specified file
	 *
	 * @param FileName the name of the file to read
	 * @param FileContents the out buffer to copy the data into
	 *
 	 * @return true if the data was copied, false otherwise
	 */
	virtual bool GetFileContents(const FString& FileName, TArray<uint8>& FileContents) = 0;

	/**
	 * Empties the set of downloaded files if possible (no async tasks outstanding)
	 *
	 * @return true if they could be deleted, false if they could not
	 */
	virtual bool ClearFiles() = 0;

	/**
	 * Empties the cached data for this file if it is not being downloaded currently
	 *
	 * @param FileName the name of the file to remove from the cache
	 *
	 * @return true if it could be deleted, false if it could not
	 */
	virtual bool ClearFile(const FString& FileName) = 0;

	/**
	 * Delete cached files on disk
	 *
	 * @param bSkipEnumerated if true then only non-enumerated files are deleted
	 */
	virtual void DeleteCachedFiles(bool bSkipEnumerated) = 0;

	/**
	* Requests a list of available files from the network store
	*
	* @param Page paging info to use for query
	*
	* @return true if the request has started, false if not
	*/
	virtual bool EnumerateFiles(const FPagedQuery& Page = FPagedQuery()) = 0;

	/**
	 * Delegate fired when the list of files has been returned from the network store
	 *
	 * @param bWasSuccessful whether the file list was successful or not
	 * @param Error string representing the error condition
	 *
	 */
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnEnumerateFilesComplete, bool, const FString&);

	/**
	 * Returns the list of files that was returned by the network store
	 * 
	 * @param Files out array of file metadata
	 *
	 */
	virtual void GetFileList(TArray<FCloudFileHeader>& Files) = 0;

	/**
	 * Starts an asynchronous read of the specified file from the network platform's file store
	 *
	 * @param FileToRead the name of the file to read
	 *
	 * @return true if the calls starts successfully, false otherwise
	 */
	virtual bool ReadFile(const FString& FileName) = 0;

	/**
	 * Delegate fired when a file read from the network platform's storage is complete
	 *
	 * @param bWasSuccessful whether the file read was successful or not
	 * @param FileName the name of the file this was for
	 *
	 */
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnReadFileComplete, bool, const FString&);

	/**
	 * Delegate fired when as file read from the network platform's storage progresses
	 *
	 * @param FileName the name of the file this was for
	 * @param NumBytes the number of bytes read so far
	 */
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnReadFileProgress, const FString&, uint64);

	/**
	 * Delegate fired when we would like to report an event to an analytics provider.
	 *
	 * @param EventName the name of the event
	 * @param Attributes the key/value pairs of analytics event attributes to include with the event
	 */
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnTitleFileAnalyticsEvent, const FString& /* EventName */, const TArray<FAnalyticsEventAttribute>& /* Attributes */)
};

typedef TSharedPtr<IOnlineTitleFile, ESPMode::ThreadSafe> IOnlineTitleFilePtr;
typedef TSharedRef<IOnlineTitleFile, ESPMode::ThreadSafe> IOnlineTitleFileRef;
