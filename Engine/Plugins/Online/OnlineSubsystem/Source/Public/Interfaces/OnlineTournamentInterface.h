// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Online/CoreOnlineFwd.h"
#include "OnlineKeyValuePair.h"

struct FDateTime;
struct FTimespan;

ONLINESUBSYSTEM_API DECLARE_LOG_CATEGORY_EXTERN(LogOnlineTournament, Display, All);

#define UE_LOG_ONLINE_TOURNAMENT(Verbosity, Format, ...) \
{ \
	UE_LOGF(LogOnlineTournament, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

#define UE_CLOG_ONLINE_TOURNAMENT(Conditional, Verbosity, Format, ...) \
{ \
	UE_CLOGF(Conditional, LogOnlineTournament, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

struct FOnlineError;

/** UniqueNetId of a tournament */
using FOnlineTournamentId = FUniqueNetId;
using FOnlineTournamentIdRef = FUniqueNetIdRef;
using FOnlineTournamentIdPtr = FUniqueNetIdPtr;

/** UniqueNetId of a match in a tournament */
using FOnlineTournamentMatchId = FUniqueNetId;
using FOnlineTournamentMatchIdRef = FUniqueNetIdRef;
using FOnlineTournamentMatchIdPtr = FUniqueNetIdPtr;

/** UniqueNetId of a participant (Player or team) in a tournament */
using FOnlineTournamentParticipantId = FUniqueNetId;
using FOnlineTournamentParticipantIdRef = FUniqueNetIdRef;
using FOnlineTournamentParticipantIdPtr = FUniqueNetIdPtr;

/** UniqueNetId of a team in a tournament */
using FOnlineTournamentTeamId = FUniqueNetId;
using FOnlineTournamentTeamIdRef = FUniqueNetIdRef;
using FOnlineTournamentTeamIdPtr = FUniqueNetIdPtr;

/**
 * What format the tournament is being run as
 */
enum class EOnlineTournamentFormat : uint8
{
	/** The tournament is being run in the single elimination format */
	SingleElimination,
	/** The tournament is being run in the double elimination format */
	DoubleElimination,
	/** The tournament is being run in the swiss format */
	Swiss,
	/** The tournament is being run in the round-robin format */
	RoundRobin,
	/** The tournament is being run in a custom format */
	Custom
};

inline void LexFromString(TOptional<EOnlineTournamentFormat>& Format, const TCHAR* const String)
{
	if (FCString::Stricmp(String, TEXT("SingleElimination")) == 0)
	{
		Format = EOnlineTournamentFormat::SingleElimination;
	}
	else if (FCString::Stricmp(String, TEXT("DoubleElimination")) == 0)
	{
		Format = EOnlineTournamentFormat::DoubleElimination;
	}
	else if (FCString::Stricmp(String, TEXT("Swiss")) == 0)
	{
		Format = EOnlineTournamentFormat::Swiss;
	}
	else if (FCString::Stricmp(String, TEXT("RoundRobin")) == 0)
	{
		Format = EOnlineTournamentFormat::RoundRobin;
	}
	else if (FCString::Stricmp(String, TEXT("Custom")) == 0)
	{
		Format = EOnlineTournamentFormat::Custom;
	}
	else
	{
		Format = TOptional<EOnlineTournamentFormat>();
	}
}

inline FString LexToString(const EOnlineTournamentFormat Format)
{
	switch (Format)
	{
	case EOnlineTournamentFormat::SingleElimination:
		return TEXT("SingleElimination");
	case EOnlineTournamentFormat::DoubleElimination:
		return TEXT("DoubleElimination");
	case EOnlineTournamentFormat::Swiss:
		return TEXT("Swiss");
	case EOnlineTournamentFormat::RoundRobin:
		return TEXT("RoundRobin");
	case EOnlineTournamentFormat::Custom:
		return TEXT("Custom");
	}

	checkNoEntry();
	return FString();
}

/**
 * What state the tournament is currently in
 */
enum class EOnlineTournamentState : uint8
{
	/** The tournament has been created, but participants may not be registered yet */
	Created,
	/** The tournament is now open for registration */
	OpenRegistration,
	/** The tournament registration has now closed, but the tournament has not started yet */
	ClosedRegistration,
	/** The tournament is now in progress */
	InProgress,
	/** The tournament has now finished and all results are finalized */
	Finished,
	/** The tournament was cancelled */
	Cancelled
};

inline void LexFromString(TOptional<EOnlineTournamentState>& State, const TCHAR* const String)
{
	if (FCString::Stricmp(String, TEXT("Created")) == 0)
	{
		State = EOnlineTournamentState::Created;
	}
	else if (FCString::Stricmp(String, TEXT("OpenRegistration")) == 0)
	{
		State = EOnlineTournamentState::OpenRegistration;
	}
	else if (FCString::Stricmp(String, TEXT("ClosedRegistration")) == 0)
	{
		State = EOnlineTournamentState::ClosedRegistration;
	}
	else if (FCString::Stricmp(String, TEXT("InProgress")) == 0)
	{
		State = EOnlineTournamentState::InProgress;
	}
	else if (FCString::Stricmp(String, TEXT("Finished")) == 0)
	{
		State = EOnlineTournamentState::Finished;
	}
	else if (FCString::Stricmp(String, TEXT("Cancelled")) == 0)
	{
		State = EOnlineTournamentState::Cancelled;
	}
	else
	{
		State = TOptional<EOnlineTournamentState>();
	}
}

inline FString LexToString(const EOnlineTournamentState State)
{
	switch (State)
	{
	case EOnlineTournamentState::Created:
		return TEXT("Created");
	case EOnlineTournamentState::OpenRegistration:
		return TEXT("OpenRegistration");
	case EOnlineTournamentState::ClosedRegistration:
		return TEXT("ClosedRegistration");
	case EOnlineTournamentState::InProgress:
		return TEXT("InProgress");
	case EOnlineTournamentState::Finished:
		return TEXT("Finished");
	case EOnlineTournamentState::Cancelled:
		return TEXT("Cancelled");
	}

	checkNoEntry();
	return FString();
}

/**
 * What participant format does this tournament support?
 */
enum class EOnlineTournamentParticipantType : uint8
{
	/** The tournament has individual players facing each other */
	Individual,
	/** The tournament has teams facing other teams */
	Team
};

inline void LexFromString(TOptional<EOnlineTournamentParticipantType>& State, const TCHAR* const String)
{
	if (FCString::Stricmp(String, TEXT("Individual")) == 0)
	{
		State = EOnlineTournamentParticipantType::Individual;
	}
	else if (FCString::Stricmp(String, TEXT("Team")) == 0)
	{
		State = EOnlineTournamentParticipantType::Team;
	}
	else
	{
		State = TOptional<EOnlineTournamentParticipantType>();
	}
}

inline FString LexToString(const EOnlineTournamentParticipantType ParticipantType)
{
	switch (ParticipantType)
	{
	case EOnlineTournamentParticipantType::Individual:
		return TEXT("Individual");
	case EOnlineTournamentParticipantType::Team:
		return TEXT("Team");
	}

	checkNoEntry();
	return FString();
}

/**
 * Filters to use when querying for tournament information.
 *
 * Some of these fields may be required, depending on the backing online system.  Some fields
 * may not be specified if other fields are specified, depending on the backing online system.
 */
struct FOnlineTournamentQueryFilter
{
	/** What direction to sort these results by (useful when specifying limits and offsetts)*/
	enum class EOnlineTournamentSortDirection : uint8
	{
		/** Results will be sorted in Ascending order */
		Ascending,
		/** Results will be sorted in Descending order */
		Descending
	};

	/** Filter tournament information that does not match this participant type */
	TOptional<EOnlineTournamentParticipantType> ParticipantType;
	/** Filter tournament information that does not match this tournament format */
	TOptional<EOnlineTournamentFormat> Format;
	/** Only include tournament information that includes this team (on team tournaments) */
	TOptional<FOnlineTournamentTeamIdRef> TeamId;
	/** Only include tournament information that includes this player */
	TOptional<FUniqueNetIdRef> PlayerId;
	/** Limit the results to this many entries */
	TOptional<uint32> Limit;
	/** Start the results this many entries in */
	TOptional<uint32> Offset;
	/** Sort the results in this direction */
	TOptional<EOnlineTournamentSortDirection> SortDirection;
};

/**
 * Details about a participant and their current score
 */
struct FOnlineTournamentScore
{
	virtual ~FOnlineTournamentScore() = default;

	FOnlineTournamentScore(const FOnlineTournamentParticipantIdRef& InParticipantId, const EOnlineTournamentParticipantType InParticipantType, const FVariantData& InScore)
		: ParticipantId(InParticipantId)
		, ParticipantType(InParticipantType)
		, Score(InScore)
	{
	}

	/** The ParticipantId who achieved Score */
	FOnlineTournamentParticipantIdRef ParticipantId;
	/** The type of participant this is */
	EOnlineTournamentParticipantType ParticipantType;
	/** The score for this participant */
	FVariantData Score;
};

/**
 * The results of a match
 */
struct FOnlineTournamentMatchResults
{
	struct FOnlineTournamentScreenshotData
	{
		FOnlineTournamentScreenshotData(const FString& InScreenshotFormat, const TArray<uint8>& InScreenshotData)
			: ScreenshotFormat(InScreenshotFormat)
			, ScreenshotData(InScreenshotData)
		{
		}

		FOnlineTournamentScreenshotData(FString&& InScreenshotFormat, TArray<uint8>&& InScreenshotData)
			: ScreenshotFormat(MoveTemp(InScreenshotFormat))
			, ScreenshotData(MoveTemp(InScreenshotData))
		{
		}

		/** The format of the screenshot stored in ScreenshotData */
		FString ScreenshotFormat;
		/** Raw bytes of a screenshot in the ScreenshotFormat format */
		TArray<uint8> ScreenshotData;
	};

	/** Score data to submit */
	TArray<FOnlineTournamentScore> ScoresToSubmit;

	/** Optiona notes about a score */
	TOptional<FString> Notes;
	/** Optional screenshot data for proof of a score */
	TOptional<FOnlineTournamentScreenshotData> Screenshot;
};

/**
 * The details of a team in a tournament
 */
struct IOnlineTournamentTeamDetails
	: public TSharedFromThis<IOnlineTournamentTeamDetails>
{
	virtual ~IOnlineTournamentTeamDetails() = default;

	/** Get the TeamId of this team */
	virtual FOnlineTournamentTeamIdRef GetTeamId() const = 0;
	/** Get the player ids of this team (if they are known) */
	virtual TOptional<TArray<FUniqueNetIdRef>> GetPlayerIds() const = 0;
	/** Get the display name of this Team */
	virtual FString GetDisplayName() const = 0;
	/** Get an attribute for this team (varies by online platform) */
	virtual TOptional<FVariantData> GetAttribute(const FName AttributeName) const = 0;
};

/**
 * Filters to use when querying for participant information.
 */
struct FOnlineTournamentParticipantQueryFilter
{
	FOnlineTournamentParticipantQueryFilter(const EOnlineTournamentParticipantType InParticipantType)
		: ParticipantType(InParticipantType)
	{
	}

	EOnlineTournamentParticipantType ParticipantType;
	TOptional<uint32> Limit;
	TOptional<uint32> Offset;
};

enum class EOnlineTournamentParticipantState : uint8
{
	/** The participant has registered for the upcoming event */
	Registered,
	/** The participant has checked into the event and are ready to play */
	CheckedIn,
	/** The participant was present for the past event */
	Present,
	/** The participant was not present for the past event */
	Absent
};

inline void LexFromString(TOptional<EOnlineTournamentParticipantState>& State, const TCHAR* const String)
{
	if (FCString::Stricmp(String, TEXT("Registered")) == 0)
	{
		State = EOnlineTournamentParticipantState::Registered;
	}
	else if (FCString::Stricmp(String, TEXT("CheckedIn")) == 0)
	{
		State = EOnlineTournamentParticipantState::CheckedIn;
	}
	else if (FCString::Stricmp(String, TEXT("Present")) == 0)
	{
		State = EOnlineTournamentParticipantState::Present;
	}
	else if (FCString::Stricmp(String, TEXT("Absent")) == 0)
	{
		State = EOnlineTournamentParticipantState::Absent;
	}
	else
	{
		State = TOptional<EOnlineTournamentParticipantState>();
	}
}

inline FString LexToString(const EOnlineTournamentParticipantState ParticipantType)
{
	switch (ParticipantType)
	{
	case EOnlineTournamentParticipantState::Registered:
		return TEXT("Registered");
	case EOnlineTournamentParticipantState::CheckedIn:
		return TEXT("CheckedIn");
	case EOnlineTournamentParticipantState::Present:
		return TEXT("Present");
	case EOnlineTournamentParticipantState::Absent:
		return TEXT("Absent");
	}

	checkNoEntry();
	return FString();
}

/**
 * The tournament-specific details of a participant in a tournament
 */
struct IOnlineTournamentParticipantDetails
	: public TSharedFromThis<IOnlineTournamentParticipantDetails>
{
	virtual ~IOnlineTournamentParticipantDetails() = default;

	/** Get the Tournament ID this participant is from */
	virtual FOnlineTournamentIdRef GetTournamentId() const = 0;
	/** Get the Player ID of this tournament participant (if applicable) */
	virtual FUniqueNetIdPtr GetPlayerId() const = 0;
	/** Get the Team ID of this tournament participant (if applicable) */
	virtual FOnlineTournamentTeamIdPtr GetTeamId() const = 0;
	/** Get the display name of this participant */
	virtual const FString& GetDisplayName() const = 0;
	/** Get the current state of the tournament participant */
	virtual EOnlineTournamentParticipantState GetState() const = 0;
	/** Get the current position of this tournament participant (if applicable) */
	virtual TOptional<int32> GetPosition() const = 0;
	/** Get the current score of this tournament participant (if applicable) */
	virtual TOptional<FVariantData> GetScore() const = 0;
	/** Get meta-data for this participant (varies based on online platform) */
	virtual TOptional<FVariantData> GetAttribute(const FName AttributeName) const = 0;
};

/**
	* States this match can be in
	*/
enum class EOnlineTournamentMatchState : uint8
{
	Created,
	InProgress,
	Finished
};

inline void LexFromString(TOptional<EOnlineTournamentMatchState>& State, const TCHAR* const String)
{
	if (FCString::Stricmp(String, TEXT("Created")) == 0)
	{
		State = EOnlineTournamentMatchState::Created;
	}
	else if (FCString::Stricmp(String, TEXT("InProgress")) == 0)
	{
		State = EOnlineTournamentMatchState::InProgress;
	}
	else if (FCString::Stricmp(String, TEXT("Finished")) == 0)
	{
		State = EOnlineTournamentMatchState::Finished;
	}
	else
	{
		State = TOptional<EOnlineTournamentMatchState>();
	}
}

inline FString LexToString(const EOnlineTournamentMatchState ParticipantType)
{
	switch (ParticipantType)
	{
	case EOnlineTournamentMatchState::Created:
		return TEXT("Created");
	case EOnlineTournamentMatchState::InProgress:
		return TEXT("InProgress");
	case EOnlineTournamentMatchState::Finished:
		return TEXT("Finished");
	}

	checkNoEntry();
	return FString();
}

/**
 * The details of a match
 */
struct IOnlineTournamentMatchDetails
	: public TSharedFromThis<IOnlineTournamentMatchDetails>
{
	virtual ~IOnlineTournamentMatchDetails() = default;

	/** Get the MatchId for this match*/
	virtual FOnlineTournamentMatchIdRef GetMatchId() const = 0;
	/** Get the type of participants for this match */
	virtual EOnlineTournamentParticipantType GetParticipantType() const = 0;
	/** Get the current state of this match */
	virtual EOnlineTournamentMatchState GetMatchState() const = 0;
	/** Get the bracket of this match */
	virtual TOptional<FString> GetBracket() const = 0;
	/** Get the round of this match */
	virtual TOptional<int32> GetRound() const = 0;
	/** Get the start time of this match in UTC */
	virtual TOptional<FDateTime> GetStartDateUTC() const = 0;
	/** Get the array of participants for this match */
	virtual TArray<TSharedRef<const IOnlineTournamentParticipantDetails>> GetParticipants() const = 0;
	/** Get meta-data for this match (varies based on online platform) */
	virtual TOptional<FVariantData> GetAttribute(const FName AttributeName) const = 0;
};

/**
 * The details of a tournament
 */
struct IOnlineTournamentDetails
	: public TSharedFromThis<IOnlineTournamentDetails>
{
	virtual ~IOnlineTournamentDetails() = default;

	/** Get the Tournament ID for this tournament */
	virtual FOnlineTournamentIdRef GetTournamentId() const = 0;
	/** Get the Title for this tournament */
	virtual const FString& GetTitle() const = 0;
	/** Get the Description for this tournament */
	virtual const FString& GetDescription() const = 0;
	/** Get the current state of this tournament */
	virtual EOnlineTournamentState GetState() const = 0;
	/** Get the the format of this tournament */
	virtual EOnlineTournamentFormat GetFormat() const = 0;
	/** Get the type of participants that are involved in this tournament */
	virtual EOnlineTournamentParticipantType GetParticipantType() const = 0;
	/** Get the list of Participant IDs for this tournament if known */
	virtual const TArray<TSharedRef<const IOnlineTournamentParticipantDetails>>& GetParticipants() const = 0;
	/** Get the registration start time of this tournament in UTC */
	virtual TOptional<FDateTime> GetRegistrationStartDateUTC() const = 0;
	/** Get the registration end time of this tournament in UTC */
	virtual TOptional<FDateTime> GetRegistrationEndDateUTC() const = 0;
	/** Get the start time of this tournament in UTC */
	virtual TOptional<FDateTime> GetStartDateUTC() const = 0;
	/** Get the check-in time of this tournament in UTC */
	virtual TOptional<FTimespan> GetCheckInTimespan() const = 0;
	/** Get the end time of this tournament in UTC */
	virtual TOptional<FDateTime> GetEndDateUTC() const = 0;
	/** Get the last time in UTC this tournament's details were updated */
	virtual TOptional<FDateTime> GetLastUpdatedDateUTC() const = 0;
	/** Does this tournament require a premium subscription to participate in? */
	virtual TOptional<bool> RequiresPremiumSubscription() const = 0;
	/** Get meta-data for this tournament (varies based on online platform) */
	virtual TOptional<FVariantData> GetAttribute(const FName AttributeName) const = 0;
};

/**
 * A delegate for when a tournament list has finished being queried.
 *
 * @param ResultStatus The result of the query
 * @param TournamentIds An optional array of Tournament IDs
 */
DECLARE_DELEGATE_TwoParams(FOnlineTournamentQueryTournamentListComplete, const FOnlineError& /*ResultStatus*/, const TOptional<TArray<FOnlineTournamentIdRef>>& /*TournamentIds*/);

/**
 * A delegate for when tournament details have finished being queried.
 *
 * @param ResultStatus The result of the query
 * @param TournamentDetails An optional array of Tournament Details
 */
DECLARE_DELEGATE_TwoParams(FOnlineTournamentQueryTournamentDetailsComplete, const FOnlineError& /*ResultStatus*/, const TOptional<TArray<TSharedRef<const IOnlineTournamentDetails>>>& /*TournamentDetails*/);

/**
 * A delegate for when a match list has finished being queried.
 *
 * @param ResultStatus The result of the query
 * @param MatchIds An optional array of Match Ids
 */
DECLARE_DELEGATE_TwoParams(FOnlineTournamentQueryMatchListComplete, const FOnlineError& /*ResultStatus*/, const TOptional<TArray<FOnlineTournamentMatchIdRef>>& /*MatchIds*/);

/**
 * A delegate for when match details have finished being queried.
 *
 * @param ResultStatus The result of the query
 * @param MatchDetails An optional array of Match Details
 */
DECLARE_DELEGATE_TwoParams(FOnlineTournamentQueryMatchDetailsComplete, const FOnlineError& /*ResultStatus*/, const TOptional<TArray<TSharedRef<const IOnlineTournamentMatchDetails>>>& /*MatchDetails*/);

/**
 * A delegate for when participant lists have finished being queried.
 *
 * @param ResultStatus The result of the query
 * @param TotalResults Total possible results there are to be queried
 * @param ParticipantList An optional array of Participant Details
 */
DECLARE_DELEGATE_ThreeParams(FOnlineTournamentQueryParticipantListComplete, const FOnlineError& /*ResultStatus*/, const TOptional<uint32> /*TotalResults*/, const TOptional<TArray<TSharedRef<const IOnlineTournamentParticipantDetails>>>& /*ParticipantList*/);

/**
 * A delegate for when team details have finished being queried.
 *
 * @param ResultStatus The result of the query
 * @param TeamDetails An optional array of Team Details
 */
DECLARE_DELEGATE_TwoParams(FOnlineTournamentQueryTeamDetailsComplete, const FOnlineError& /*ResultStatus*/, const TOptional<TArray<TSharedRef<const IOnlineTournamentTeamDetails>>>& /*TeamDetails*/);

/**
 * A delegate for when match results have finished being submitted.
 *
 * @param ResultStatus The result of the query
 */
DECLARE_DELEGATE_OneParam(FOnlineTournamentSubmitMatchResultsComplete, const FOnlineError& /*ResultStatus*/);


using FAdditionalMetaDataMap = TMap<FName, FString>;

/**
 * A delegate for when a tournament has been joined
 *
 * @param UserId The user who joined the tournament
 * @param TournamentId The ID of the tournament joined
 * @param AdditonalData A map of additional platform-specific data that was provided when this event was triggered
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnOnlineTournamentTournamentJoined, const FUniqueNetIdRef /*UserId*/, const FOnlineTournamentIdRef /*TournamentId*/, const FAdditionalMetaDataMap& /*AdditionalData*/);
using FOnOnlineTournamentTournamentJoinedDelegate = FOnOnlineTournamentTournamentJoined::FDelegate;

/**
 * A delegate for when a tournament has been joined
 *
 * @param UserId The user who joined the tournament
 * @param MatchId The ID of the match joined
 * @param AdditonalData A map of additional platform-specific data that was provided when this event was triggered
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnOnlineTournamentMatchJoined, const FUniqueNetIdRef /*UserId*/, const FOnlineTournamentMatchIdRef /*MatchId*/, const FAdditionalMetaDataMap& /*AdditionalData*/);
using FOnOnlineTournamentMatchJoinedDelegate = FOnOnlineTournamentMatchJoined::FDelegate;

/**
 * Interface to handle requesting and submitting information related to tournaments
 */
class IOnlineTournament
	: public TSharedFromThis<IOnlineTournament, ESPMode::ThreadSafe>
{
public:
	virtual ~IOnlineTournament() = default;

	/**
	 * Query a list of tournaments available for a user using specified filters.
	 *
	 * @param UserId The User to query tournaments as
	 * @param QueryFilter The filter to user to include/exclude tournaments from the list.  One or more options MAY be required, depending on the online platform.
	 * @param Delegate A delegate that is called when our tournament list query is complete
	 */
	virtual void QueryTournamentList(const FUniqueNetIdRef UserId, const FOnlineTournamentQueryFilter& QueryFilter, const FOnlineTournamentQueryTournamentListComplete& Delegate) = 0;

	/**
	 * Get a list of all Tournament IDs that have been queried by the specified user.
	 *
	 * @param UserId A User who previously queried tournaments
	 * @return If previously queried by this user, an array of results from the query.
	 */
	virtual TArray<FOnlineTournamentIdRef> GetTournamentList(const FUniqueNetIdRef UserId) const = 0;

	/**
	 * Query tournament details from the perspective of the specified user.
	 *
	 * @param UserId A User to query tournament details as
	 * @param TournamentIds An array of Tournament IDs to get more information about
	 * @param Delegate A delegate that is called when our tournament details query is complete
	 */
	virtual void QueryTournamentDetails(const FUniqueNetIdRef UserId, const TArray<FOnlineTournamentIdRef>& TournamentIds, const FOnlineTournamentQueryTournamentDetailsComplete& Delegate) = 0;

	/**
	 * Get a tournament details result for a tournament that had been previously queried by the specified user.
	 *
	 * @param UserId A User who previously queried the tournament
	 * @param TournamentId The Tournament ID that previously had details queried by UserId
	 * @return If this tournament was previously queried by this user, the result from the query.
	 */
	virtual TSharedPtr<const IOnlineTournamentDetails> GetTournamentDetails(const FUniqueNetIdRef UserId, const FOnlineTournamentIdRef TournamentId) const = 0;

	/**
	 * Get tournament detail results for specified tournaments that have been previously queried by the specified user.
	 *
	 * @param UserId A User who previously queried tournaments
	 * @param TournamentId The Tournament ID that previously had details queried by UserId
	 * @return If tournaments have been previously queried by this user, any matching tournaments from the results are of those queries.
	 */
	virtual TArray<TSharedPtr<const IOnlineTournamentDetails>> GetTournamentDetails(const FUniqueNetIdRef UserId, const TArray<FOnlineTournamentIdRef>& TournamentIds) const = 0;

	/**
	 * Query a list of matches for a tournament from the perspective of the specified user.
	 *
	 * @param UserId A User to query match details as
	 * @param TournamentId The Tournament ID to get Match IDs from
	 * @param Delegate A delegate that is called when our match list query is complete
	 */
	virtual void QueryMatchList(const FUniqueNetIdRef UserId, const FOnlineTournamentIdRef TournamentId, const FOnlineTournamentQueryMatchListComplete& Delegate) = 0;

	/**
	 * Get match detail results for that have been previously queried by the specified user.
	 *
	 * @param UserId A User who previously queried match results
	 * @param TournamentId The Tournament ID that previously had match results queried by UserId
	 * @return If tournaments have been previously queried by this user, all results of those queries.
	 */
	virtual TArray<FOnlineTournamentMatchIdRef> GetMatchList(const FUniqueNetIdRef UserId, const FOnlineTournamentIdRef TournamentId) const = 0;

	/**
	 * Query match details for a tournament from the perspective of the specified user.
	 *
	 * @param UserId The User to query match details
	 * @param MatchId The Match IDs to get more information about
	 * @param Delegate A delegate that is called when our match details query is complete
	 */
	virtual void QueryMatchDetails(const FUniqueNetIdRef UserId, const TArray<FOnlineTournamentMatchIdRef>& MatchIds, const FOnlineTournamentQueryMatchDetailsComplete& Delegate) = 0;

	/**
	 * Get a match's details that have been previously queried by the specified user.
	 *
	 * @param UserId A User who previously queried the match
	 * @param MatchId The Match ID that previously had details queried by UserId
	 * @return If this match has been previously queried by this user, the result is of that query.
	 */
	virtual TSharedPtr<const IOnlineTournamentMatchDetails> GetMatchDetails(const FUniqueNetIdRef UserId, const FOnlineTournamentMatchIdRef MatchId) const = 0;

	/**
	 * Get match details that have been previously queried by the specified user.
	 *
	 * @param UserId A User who previously queried matches
	 * @param MatchIds Match ids that previously had details queried by UserId
	 * @return If matches have been previously queried by this user, any matching matches from the results are of those queries.
	 */
	virtual TArray<TSharedPtr<const IOnlineTournamentMatchDetails>> GetMatchDetails(const FUniqueNetIdRef UserId, const TArray<FOnlineTournamentMatchIdRef>& MatchIds) const = 0;

	/**
	 * Query a list of participants for a tournament from the perspective of the specified user.
	 *
	 * It is valid to request Team IDs or Player IDs from a Team tournament, and it is only valid to request Player IDs from an Individual tournament.
	 *
	 * @param UserId A User to query Participant IDs as
	 * @param TournamentId The Tournament ID to get Participant IDs from
	 * @param QueryFilter Filter to use to query participants.
	 * @param Delegate A delegate that is called when our match list query is complete
	 */
	virtual void QueryParticipantList(const FUniqueNetIdRef UserId, const FOnlineTournamentIdRef TournamentId, const FOnlineTournamentParticipantQueryFilter& QueryFilter, const FOnlineTournamentQueryParticipantListComplete& Delegate) = 0;

	/**
	 * Get Participant details that have been previously queried by the specified user.
	 *
	 * @param UserId A User who previously queried participants
	 * @param TournamentId The Tournament ID that previously had participants queried by UserId
	 * @param ParticipantType The type of IDs to return from this tournament.  This type must have been previously queried.
	 * @return If results have previously previously been queried by this user, the results of that query.
	 */
	virtual TArray<TSharedRef<const IOnlineTournamentParticipantDetails>> GetParticipantList(const FUniqueNetIdRef UserId, const FOnlineTournamentIdRef TournamentId, const EOnlineTournamentParticipantType ParticipantType) const = 0;

	/**
	 * Query team details from the perspective of the specified user.
	 *
	 * @param UserId A User to query team details as
	 * @param TeamId The Team IDs to get more information about
	 * @param Delegate A delegate that is called when our team details query is complete
	 */
	virtual void QueryTeamDetails(const FUniqueNetIdRef UserId, const TArray<FOnlineTournamentTeamIdRef>& TeamIds, const FOnlineTournamentQueryTeamDetailsComplete& Delegate) = 0;

	/**
	 * Get a teams's details that have been previously queried by the specified user.
	 *
	 * @param UserId A User who previously queried the team
	 * @param TeamId The Team ID that previously had details queried by UserId
	 * @return If this team has been previously queried by this user, the result is of that query.
	 */
	virtual TSharedPtr<const IOnlineTournamentTeamDetails> GetTeamDetails(const FUniqueNetIdRef UserId, const FOnlineTournamentTeamIdRef TeamId) const = 0;

	/**
	 * Get team details that have been previously queried by the specified user.
	 *
	 * @param UserId A User who previously queried teams
	 * @param TeamIds The Team ID that previously had details queried by UserId
	 * @return If teams have been previously queried by this user, any matching teams from the results are of those queries.
	 */
	virtual TArray<TSharedPtr<const IOnlineTournamentTeamDetails>> GetTeamDetails(const FUniqueNetIdRef UserId, const TArray<FOnlineTournamentTeamIdRef>& TeamIds) const = 0;

	/**
	 * Submit match results for a tournament match.
	 *
	 * @param UserId The User to report match results for
	 * @param MatchId The Match Id to report match results for
	 * @param MatchResults The results of the match
	 * @param Delegate A delegate that is called when our match stats submissions is complete
	 */
	virtual void SubmitMatchResults(const FUniqueNetIdRef UserId, const FOnlineTournamentMatchIdRef MatchId, const FOnlineTournamentMatchResults& MatchResults, const FOnlineTournamentSubmitMatchResultsComplete& Delegate) = 0;

	/**
	 * Register for updates when a tournament has been joined
	 *
	 * @param Delegate Delegate to be called when a tournament has been joined
	 */
	virtual FDelegateHandle AddOnOnlineTournamentTournamentJoined(const FOnOnlineTournamentTournamentJoinedDelegate& Delegate) = 0;

	/**
	 * Unregister for tournament join updates using a previously-registered delegate handle
	 *
	 * @param DelegateHandle Delegate handle that was previously registered
	 */
	virtual void RemoveOnOnlineTournamentTournamentJoined(const FDelegateHandle& DelegateHandle) = 0;

	/**
	 * Register for updates when a tournament match has been joined
	 *
	 * @param Delegate Delegate to be called when a tournament match has been joined
	 */
	virtual FDelegateHandle AddOnOnlineTournamentMatchJoinedDelegate(const FOnOnlineTournamentMatchJoinedDelegate& Delegate) = 0;

	/**
	 * Unregister for tournament match join updates using a previously-registered delegate handle
	 *
	 * @param DelegateHandle Delegate handle that was previously registered
	 */
	virtual void RemoveOnOnlineTournamentMatchJoinedDelegate(const FDelegateHandle& DelegateHandle) = 0;

#if !UE_BUILD_SHIPPING
	/** Print all cached tournament information into the logs */
	virtual void DumpCachedTournamentInfo(const FUniqueNetIdRef UserId) const = 0;
	/** Print all cached match information into the logs */
	virtual void DumpCachedMatchInfo(const FUniqueNetIdRef UserId) const = 0;
	/** Print all cached participant information into the logs */
	virtual void DumpCachedParticipantInfo(const FUniqueNetIdRef UserId) const = 0;
	/** Print all cached team information into the logs */
	virtual void DumpCachedTeamInfo(const FUniqueNetIdRef UserId) const = 0;
#endif // !UE_BUILD_SHIPPING
};
