// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h" // IWYU pragma: keep
#include "Features/IModularFeature.h"

#define UE_API ONLINESUBSYSTEM_API

class FName;

class FOutputDevice;

/**
 * Online Tracing Interface
 */
class IOnlineTracing
	: public IModularFeature
{
public:
	/**
	 * Starts a context
	 *
	 * @param ContextName user-specified context name.
	 */

	static UE_API void StartContext(FName ContextName);
	
	/**
	 * Ends a context
	 *
	 * @param ContextName user-specified context name.
	 */
	static UE_API void EndContext(FName ContextName);

	virtual bool GetUncompressedTracingLog(TArray<uint8>& OutLog) = 0;

	virtual FString GetFilename() = 0;

	static UE_API IOnlineTracing* GetTracingHelper();

protected:
	virtual void StartContextImpl(FName ContextName) = 0;

	virtual void EndContextImpl(FName ContextName) = 0;
};

#undef UE_API
