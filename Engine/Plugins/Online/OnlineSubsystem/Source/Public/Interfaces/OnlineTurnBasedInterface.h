// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineTurnBasedInterface.generated.h"

/** 
 * FTurnBasedMatchRequest contains all of the information required for a matchmaker to create an FTurnBasedMatch
 */
class FTurnBasedMatchRequest
{
public:
    FTurnBasedMatchRequest()
    : MinNumberOfPlayers(0)
	, MaxNumberOfPlayers(0)
    , PlayerGroup(0)
	, ShowExistingMatches(false)
    {}

	FTurnBasedMatchRequest(int32 InMinNumberOfPlayers, int32 InMaxNumberOfPlayers, uint32 InPlayerGroup, bool InShowExistingMatches)
	: MinNumberOfPlayers(InMinNumberOfPlayers)
	, MaxNumberOfPlayers(InMaxNumberOfPlayers)
	, PlayerGroup(InPlayerGroup)
	, ShowExistingMatches(InShowExistingMatches)
	{}

    void SetMinNumberOfPlayers(int32 InMinNumberOfPlayers)	{ MinNumberOfPlayers = InMinNumberOfPlayers; }
    int32 GetMinNumberOfPlayers() const						{ return MinNumberOfPlayers; }

	void SetMaxNumberOfPlayers(int32 InMaxNumberOfPlayers)	{ MaxNumberOfPlayers = InMaxNumberOfPlayers; }
	int32 GetMaxNumberOfPlayers() const						{ return MaxNumberOfPlayers; }

    void SetPlayerGroup(uint32 InPlayerGroup)				{ PlayerGroup = InPlayerGroup;  }
    uint32 GetPlayerGroup() const							{ return PlayerGroup; }

	void SetShowExistingMatches(bool InShowExistingMatches) { ShowExistingMatches = InShowExistingMatches; }
	bool GetShowExistingMatches() const						{ return ShowExistingMatches; }

private:

	// The minimum number of players needed for this match
    int32 MinNumberOfPlayers;

	// The maximum number of players needed for this match
	int32 MaxNumberOfPlayers;

	// The player group represents another parameter to pass in to matchmaking. For example, 1 could be Deathmatch, 2 could be Capture the Flag, etc.
	// Only players with the same player group will be matched together.
    uint32 PlayerGroup;

	// If true, the native matchmaking interface will show matches the player is in already
	bool ShowExistingMatches;
};

DECLARE_DELEGATE_TwoParams(FQuitMatchSignature, FString, bool);   // bool Success
DECLARE_DELEGATE_TwoParams(FRemoveMatchSignature, FString, bool);   // bool Success
DECLARE_DELEGATE_TwoParams(FUploadMatchDataSignature, FString, bool);   // bool Success
DECLARE_DELEGATE_TwoParams(FDownloadMatchDataSignature, FString, bool);  // bool Success
DECLARE_DELEGATE_TwoParams(FEndMatchSignature, FString, bool);   // bool Success

/**
* EMPMatchOutcome represents all the possible outcomes for this player in a match
*/
UENUM(BlueprintType)
namespace EMPMatchOutcome
{
    enum Outcome : int
    {
        None,
        Quit,
        Won,
        Lost,
        Tied,
        TimeExpired,
        First,
        Second,
        Third,
        Fourth,
    };
}

/**
* FTurnBasedMatch contains all of the information about an in-progress turn based match
*/
class FTurnBasedMatch
{
public:
    virtual ~FTurnBasedMatch() = default;

	/* Get the number of players in the match */
    virtual int32 GetNumberOfPlayers() const { return 2; }

    /* Get the user-friendly display name for the given player.  Can return false if the name of the associated player has not been loaded */
    virtual bool GetPlayerDisplayName(int32 PlayerIndex, FString& Name) const { return false; }

    /* Request a reload of the match's data.  Once the callback has been called, use GetMatchData to get the data */
    virtual void ReloadMatchData(FDownloadMatchDataSignature DownloadCallback) {}

    /* HasMatchData is for when you want to know if the match has data, without getting at the data */
    virtual bool HasMatchData() const { return false; }

    /* Put the match data into OutMatchData, returning whether or not the operation was successful */
    virtual bool GetMatchData(TArray<uint8>& OutMatchData) const { return false; }
    
    /* Update the data for the match with the data provided */
    virtual void SetMatchData(const TArray<uint8>& NewMatchData, FUploadMatchDataSignature EndTurnCallback) {}

	/* Returns the Match ID for this match */
    virtual FString GetMatchID() const { return FString(); }

    /* Get the index of the local player in the match's list of participants */
    virtual int32 GetLocalPlayerIndex() const { return 0; }

    /* Get the index of the current player in the match's list of participants */
    virtual int32 GetCurrentPlayerIndex() const { return 0; }

	/* Get the outcome of the match (won/lost/quit/etc.) for the player with PlayerIndex */
    virtual EMPMatchOutcome::Outcome GetMatchOutcomeForPlayer(int32 PlayerIndex) const { return EMPMatchOutcome::None; }

    /* End the turn and upload MatchData. EndTurnCallback is called once the data is uploaded (or failed), with a (bool) Success parameter */
    virtual void EndTurnWithMatchData(const TArray<uint8>& MatchData, int32 TurnTimeoutInSeconds, FUploadMatchDataSignature EndTurnCallback) {}

    /* Leave the match, providing an outcome for the player */
    virtual void QuitMatch(EMPMatchOutcome::Outcome Outcome, int32 TurnTimeoutInSeconds, FQuitMatchSignature QuitMatchCallback) {}

	/* Ends the match while setting the match outcome (win/loss/tie) for all players */
	virtual void EndMatch(FEndMatchSignature QuitMatchCallback, EMPMatchOutcome::Outcome LocalPlayerOutcome, EMPMatchOutcome::Outcome OtherPlayersOutcome) {}
};

typedef TSharedRef<FTurnBasedMatch, ESPMode::ThreadSafe> FTurnBasedMatchRef;
typedef TSharedPtr<FTurnBasedMatch, ESPMode::ThreadSafe> FTurnBasedMatchPtr;

/**
* FTurnBasedMatchmakerDelegate provides the interface for all turn based matchmaking callbacks
*/
class FTurnBasedMatchmakerDelegate
{
public:
    virtual ~FTurnBasedMatchmakerDelegate() = default;

	// This is triggered if the player cancelled the matchmaking process
    virtual void OnMatchmakerCancelled() {}

	// This is triggered if matchmaking failed for any reason
    virtual void OnMatchmakerFailed() {}

	// This is triggered once a match has been successfully found
    virtual void OnMatchFound(FTurnBasedMatchRef Match) {}
};

typedef TSharedRef<FTurnBasedMatchmakerDelegate, ESPMode::ThreadSafe> FTurnBasedMatchmakerDelegateRef;
typedef TSharedPtr<FTurnBasedMatchmakerDelegate, ESPMode::ThreadSafe> FTurnBasedMatchmakerDelegatePtr;
typedef TWeakPtr<FTurnBasedMatchmakerDelegate, ESPMode::ThreadSafe> FTurnBasedMatchmakerDelegateWeakPtr;

/**
* FTurnBasedEventDelegate provides the interface for responding to events in turn based games
*/
class FTurnBasedEventDelegate
{
public:
    virtual ~FTurnBasedEventDelegate() = default;

	// This is triggered when the match has ended for any reason
    virtual void OnMatchEnded(FString MatchID) {}

	// This is triggered when it is the current player's turn
    virtual void OnMatchReceivedTurnEvent(FString MatchID, bool BecameActive, void* Match) {}
};

DECLARE_DELEGATE_TwoParams(FLoadTurnBasedMatchesSignature, const TArray<FString>&, bool);   // bool Success, TArray MatchArray
DECLARE_DELEGATE_TwoParams(FLoadTurnBasedMatchWithIDSignature, FString, bool);   // bool Success

typedef TSharedRef<FTurnBasedEventDelegate, ESPMode::ThreadSafe> FTurnBasedEventDelegateRef;
typedef TSharedPtr<FTurnBasedEventDelegate, ESPMode::ThreadSafe> FTurnBasedEventDelegatePtr;
typedef TWeakPtr<FTurnBasedEventDelegate, ESPMode::ThreadSafe> FTurnBasedEventDelegateWeakPtr;

/**
 *	IOnlineTurnBased - Interface class for turn based multiplayer matches
 */
class IOnlineTurnBased
{
public:
    virtual ~IOnlineTurnBased() = default;

	// Set a delegate to be called when matchmaking succeeds, fails, or is canceled
    virtual void SetMatchmakerDelegate(FTurnBasedMatchmakerDelegatePtr Delegate) {}

	// Show the platform specific matchmaker interface with the parameters given in FTurnBasedMatchRequest
    virtual void ShowMatchmaker(const FTurnBasedMatchRequest& MatchRequest) {}
   
	// Accessors for a delegate that gets called when match-specific events happen (such as the match ending or it is now the current players turn)
    virtual void SetEventDelegate(FTurnBasedEventDelegateWeakPtr Delegate) {}
	virtual FTurnBasedEventDelegateWeakPtr GetEventDelegate() const { return nullptr; }

	// Load all matches the current player is participating in
    virtual void LoadAllMatches(FLoadTurnBasedMatchesSignature MatchesLoadedCallback) {}

	// Reload a match with the given MatchID
    virtual void LoadMatchWithID(FString MatchID, FLoadTurnBasedMatchWithIDSignature MatchesLoadedCallback) {}

	// Returns the match object with the given MatchID
    virtual FTurnBasedMatchPtr GetMatchWithID(FString MatchID) const { return nullptr; }

	// Deletes a match entirely from the platform specific service
    virtual void RemoveMatch(FTurnBasedMatchRef Match, FRemoveMatchSignature RemoveMatchCallback) {}

	// Accessors referring to an object that inherits from UTurnBasedMatchInterface that acts like an FTurnBasedEventDelegate
	// (needed for blueprints)
	virtual void RegisterTurnBasedMatchInterfaceObject(UObject* Object) {}
	virtual UObject* GetTurnBasedMatchInterfaceObject() { return nullptr; }

	// The size of the game specific data stored per match
	virtual int32 GetMatchDataSize() { return 0; }
};

typedef TSharedPtr<class IOnlineTurnBased, ESPMode::ThreadSafe> IOnlineTurnBasedPtr;
