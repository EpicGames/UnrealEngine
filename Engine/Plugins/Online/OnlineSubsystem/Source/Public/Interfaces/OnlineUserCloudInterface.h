// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "OnlineDelegateMacros.h"

class FUniqueNetId;
struct FCloudFileHeader;
template <typename OptionalType> struct TOptional;

ONLINESUBSYSTEM_API DECLARE_LOG_CATEGORY_EXTERN(LogOnlineCloud, Log, All);

#define UE_LOG_ONLINE_CLOUD(Verbosity, Format, ...) \
{ \
	UE_LOGF(LogOnlineCloud, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

#define UE_CLOG_ONLINE_CLOUD(Conditional, Verbosity, Format, ...) \
{ \
	UE_CLOGF(Conditional, LogOnlineCloud, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

struct FAnalyticsEventAttribute;

/**
 * Delegate fired when the list of files has been returned from the network store
 *
 * @param bWasSuccessful whether the file list was successful or not
 * @param UserId User owning the storage
 *
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnEnumerateUserFilesComplete, bool, const FUniqueNetId&);
typedef FOnEnumerateUserFilesComplete::FDelegate FOnEnumerateUserFilesCompleteDelegate;

/**
 * Delegate fired at intervals during a user file write to the network platform's storage
 *
 * @param BytesWritten the number of bytes written so far
 * @param UserId User owning the storage
 * @param FileName the name of the file this was for
 *
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnWriteUserFileProgress64, uint64, const FUniqueNetId&, const FString&);
typedef FOnWriteUserFileProgress64::FDelegate FOnWriteUserFileProgress64Delegate;

/**
 * Delegate fired when a user file write to the network platform's storage is complete
 *
 * @param bWasSuccessful whether the file Write was successful or not
 * @param UserId User owning the storage
 * @param FileName the name of the file this was for
 *
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnWriteUserFileComplete, bool, const FUniqueNetId&, const FString&);
typedef FOnWriteUserFileComplete::FDelegate FOnWriteUserFileCompleteDelegate;

/**
 * Delegate fired when a user file write to the network platform's storage is canceled
 *
 * @param bWasSuccessful whether the cancellation was successful or not
 * @param UserId User owning the storage
 * @param FileName the name of the file this was for
 *
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnWriteUserFileCanceled, bool, const FUniqueNetId&, const FString&);
typedef FOnWriteUserFileCanceled::FDelegate FOnWriteUserFileCanceledDelegate;

/**
 * Delegate fired when a user file read from the network platform's storage is complete
 *
 * @param bWasSuccessful whether the file read was successful or not
 * @param UserId User owning the storage
 * @param FileName the name of the file this was for
 *
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnReadUserFileComplete, bool, const FUniqueNetId&, const FString&);
typedef FOnReadUserFileComplete::FDelegate FOnReadUserFileCompleteDelegate;

/**
 * Delegate fired when a user file delete from the network platform's storage is complete
 *
 * @param bWasSuccessful whether the file read was successful or not
 * @param UserId User owning the storage
 * @param FileName the name of the file this was for
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnDeleteUserFileComplete, bool, const FUniqueNetId&, const FString&);
typedef FOnDeleteUserFileComplete::FDelegate FOnDeleteUserFileCompleteDelegate;

/**
 * Delegate fired when getting usage statistics from the network platform's storage is complete
 *
 * @param bWasSuccessful  Whether the operation was successful or not
 * @param UserId          User owning the storage
 * @param BytesUsed       Will be set to the total number of bytes used by the user
 * @param TotalQuota      The total number of bytes the user is entitled to write
 */
DECLARE_MULTICAST_DELEGATE_FourParams(FOnRequestUsageInfoComplete, bool, const FUniqueNetId&, int64, const TOptional<int64>&);
typedef FOnRequestUsageInfoComplete::FDelegate FOnRequestUsageInfoCompleteDelegate;

/**
 * Delegate fired when we would like to report an event to an analytics provider.
 *
 * @param EventName the name of the event
 * @param Attributes the key/value pairs of analytics event attributes to include with the event
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnUserCloudAnalyticsEvent, const FString& /* EventName */, const TArray<FAnalyticsEventAttribute>& /* Attributes */)
typedef FOnUserCloudAnalyticsEvent::FDelegate FOnUserCloudAnalyticsEventDelegate;


/**
 * Provides access to per user cloud file storage
 */
class IOnlineUserCloud
{

protected:
	ONLINESUBSYSTEM_API IOnlineUserCloud();

public:
	ONLINESUBSYSTEM_API virtual ~IOnlineUserCloud();

	/**
	 * Copies the file data into the specified buffer for the specified file
	 *
	 * @param UserId User owning the storage
	 * @param FileName the name of the file to read
	 * @param FileContents the out buffer to copy the data into
	 *
	 * @return true if the data was copied, false otherwise
	 */
	virtual bool GetFileContents(const FUniqueNetId& UserId, const FString& FileName, TArray<uint8>& FileContents) = 0;

	/**
	 * Empties the set of downloaded files if possible (no async tasks outstanding)
	 *
	 * @param UserId User owning the storage
	 *
	 * @return true if they could be deleted, false if they could not
	 */
	virtual bool ClearFiles(const FUniqueNetId& UserId) = 0;

	/**
	 * Empties the cached data for this file if it is not being downloaded currently
	 *
	 * @param UserId User owning the storage
	 * @param FileName the name of the file to remove from the cache
	 *
	 * @return true if it could be deleted, false if it could not
	 */
	virtual bool ClearFile(const FUniqueNetId& UserId, const FString& FileName) = 0;

	/**
	 * Requests a list of available User files from the network store
	 *
	 * @param UserId User owning the storage
	 *
	 */
	virtual void EnumerateUserFiles(const FUniqueNetId& UserId) = 0;

	/**
	 * Delegate fired when the list of files has been returned from the network store
	 *
	 * @param bWasSuccessful whether the file list was successful or not
	 * @param UserId User owning the storage
	 *
	 */
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnEnumerateUserFilesComplete, bool, const FUniqueNetId&);

	/**
	 * Returns the list of User files that was returned by the network store
	 * 
	 * @param UserId User owning the storage
	 * @param UserFiles out array of file metadata
	 *
	 */
	virtual void GetUserFileList(const FUniqueNetId& UserId, TArray<FCloudFileHeader>& UserFiles) = 0;

	/**
	 * Starts an asynchronous read of the specified user file from the network platform's file store
	 *
	 * @param UserId User owning the storage
	 * @param FileToRead the name of the file to read
	 *
	 * @return true if the calls starts successfully, false otherwise
	 */
	virtual bool ReadUserFile(const FUniqueNetId& UserId, const FString& FileName) = 0;

	/**
	 * Delegate fired when a user file read from the network platform's storage is complete
	 *
	 * @param bWasSuccessful whether the file read was successful or not
	 * @param UserId User owning the storage
	 * @param FileName the name of the file this was for
	 *
	 */
	DEFINE_ONLINE_DELEGATE_THREE_PARAM(OnReadUserFileComplete, bool, const FUniqueNetId&, const FString&);

	/**
	 * Starts an asynchronous write of the specified user file to the network platform's file store
	 *
	 * @param UserId User owning the storage
	 * @param FileToWrite the name of the file to write
	 * @param FileContents the out buffer to copy the data into
	 * @param CompressBeforeUpload Whether the file will be compressed before upload. Defaulted to false. Use of this parameter isn't implemented in all platforms.
	 *
	 * @return true if the calls starts successfully, false otherwise
	 */
	virtual bool WriteUserFile(const FUniqueNetId& UserId, const FString& FileName, TArray<uint8>& FileContents, bool bCompressBeforeUpload = false) = 0;

	/**
	* Delegate fired at intervals during a user file write to the network platform's storage
	*
	* @param BytesWritten the number of bytes written so far
	* @param UserId User owning the storage
	* @param FileName the name of the file this was for
	*
	*/
	DEFINE_ONLINE_DELEGATE_THREE_PARAM(OnWriteUserFileProgress64, uint64, const FUniqueNetId&, const FString&);

	/**
	 * Delegate fired when a user file write to the network platform's storage is complete
	 *
	 * @param bWasSuccessful whether the file Write was successful or not
	 * @param UserId User owning the storage
	 * @param FileName the name of the file this was for
	 *
	 */
	DEFINE_ONLINE_DELEGATE_THREE_PARAM(OnWriteUserFileComplete, bool, const FUniqueNetId&, const FString&);

	/**
	 * Cancel the ongoing upload of the specified file, if it is in progress
	 * @param UserId User owning the storage
	 * @param FileNam the name of the file to cancel
	 *
	 */
	virtual void CancelWriteUserFile(const FUniqueNetId& UserId, const FString& FileName) = 0;

	/**
	 * Delegate fired when a user file write to the network platform's storage is canceled
	 *
	 * @param bWasSuccessful whether the file upload cancellations was successful or not
	 * @param UserId User owning the storage
	 * @param FileName the name of the file this was for
	 *
	 */
	DEFINE_ONLINE_DELEGATE_THREE_PARAM(OnWriteUserFileCanceled, bool, const FUniqueNetId&, const FString&);

	/**
	 * Starts an asynchronous delete of the specified user file from the network platform's file store
	 *
	 * @param UserId User owning the storage
	 * @param FileName the name of the file to delete
	 * @param bShouldCloudDelete whether to delete the file from the cloud
	 * @param bShouldLocallyDelete whether to delete the file locally
	 *
	 * @return true if the call starts successfully, false otherwise
	 */
	virtual bool DeleteUserFile(const FUniqueNetId& UserId, const FString& FileName, bool bShouldCloudDelete, bool bShouldLocallyDelete) = 0;

	/**
	 * Delegate fired when a user file delete from the network platform's storage is complete
	 *
	 * @param bWasSuccessful whether the file delete was successful or not
	 * @param UserId User owning the storage
	 * @param FileName the name of the file this was for
	 */
	DEFINE_ONLINE_DELEGATE_THREE_PARAM(OnDeleteUserFileComplete, bool, const FUniqueNetId&, const FString&);

	/**
	 * Starts an asynchronous request to get the usage statistics from the cloud storage service
	 *
	 * @param UserId      IN      User owning the storage
	 *
	 * @return true if the call starts successfully, false otherwise
	 */
	virtual bool RequestUsageInfo(const FUniqueNetId& UserId) = 0;

	/**
	 * Delegate fired when getting usage statistics from the network platform's storage is complete
	 *
	 * @param bWasSuccessful  Whether the operation was successful or not
	 * @param UserId          User owning the storage
	 * @param BytesUsed       Will be set to the total number of bytes used by the user
	 * @param TotalQuota      The total number of bytes the user is entitled to write
	 */
	DEFINE_ONLINE_DELEGATE_FOUR_PARAM(OnRequestUsageInfoComplete, bool, const FUniqueNetId&, int64, const TOptional<int64>&);

	/**
	 *	Print out the state of the cloud for this service
	 *
	 * @param UserId user to get state for
	 */
	virtual void DumpCloudState(const FUniqueNetId& UserId) = 0;
	
	/**
	 *	Print out the state of a file in the cloud for this service
	 *
	 * @param UserId user to get state for
	 * @param FileName filename to get state for
	 */
	virtual void DumpCloudFileState(const FUniqueNetId& UserId, const FString& FileName) = 0;

	/**
	 * Delegate fired when we would like to report an event to an analytics provider.
	 *
	 * @param EventName the name of the event
	 * @param Attributes the key/value pairs of analytics event attributes to include with the event
	 */
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnUserCloudAnalyticsEvent, const FString& /* EventName */, const TArray<FAnalyticsEventAttribute>& /* Attributes */);
};

typedef TSharedPtr<IOnlineUserCloud, ESPMode::ThreadSafe> IOnlineUserCloudPtr;
