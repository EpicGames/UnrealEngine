// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CoreOnline.h"
#include "OnlineDelegateMacros.h"

class FOnlineUser;

ONLINESUBSYSTEM_API DECLARE_LOG_CATEGORY_EXTERN(LogOnlineUser, Log, All);
#define UE_LOG_ONLINE_USER(Verbosity, Format, ...) \
{ \
	UE_LOGF(LogOnlineUser, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

#define UE_CLOG_ONLINE_USER(Conditional, Verbosity, Format, ...) \
{ \
	UE_CLOGF(Conditional, LogOnlineUser, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

/**
 * Delegate used when the user query request has completed
 *
 * @param LocalUserNum the controller number of the associated user that made the request
 * @param bWasSuccessful true if the async action completed without error, false if there was an error
 * @param UserIds list of user ids that were queried
 * @param ErrorStr string representing the error condition
 */
DECLARE_MULTICAST_DELEGATE_FourParams(FOnQueryUserInfoComplete, int32, bool, const TArray< FUniqueNetIdRef >&, const FString&);
typedef FOnQueryUserInfoComplete::FDelegate FOnQueryUserInfoCompleteDelegate;

struct FExternalIdQueryOptions
{
	FExternalIdQueryOptions()
		: bLookupByDisplayName(false)
	{
	}

	FExternalIdQueryOptions(const FString& InAuthType, const bool bInLookupByDisplayName)
		: AuthType(InAuthType)
		, bLookupByDisplayName(bInLookupByDisplayName)
	{
	}

	FExternalIdQueryOptions(FString&& InAuthType, const bool bInLookupByDisplayName)
		: AuthType(MoveTemp(InAuthType))
		, bLookupByDisplayName(bInLookupByDisplayName)
	{
	}

	FString AuthType;
	bool bLookupByDisplayName; // Lookup by external display name as opposed to external id
};

/**
 *	Interface class for obtaining online User info
 */
class IOnlineUser
{

public:

	virtual ~IOnlineUser() = default;

	/**
	 * Starts an async task that queries/reads the info for a list of users
	 *
	 * @param LocalUserNum the user requesting the query
	 * @param UserIds list of users to read info about
	 *
	 * @return true if the read request was started successfully, false otherwise
	 */
	virtual bool QueryUserInfo(int32 LocalUserNum, const TArray<FUniqueNetIdRef >& UserIds) = 0;

	/**
	 * Delegate used when the user query request has completed
	 *
	 * @param LocalUserNum the controller number of the associated user that made the request
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 * @param UserIds list of user ids that were queried
	 * @param ErrorStr string representing the error condition
	 */
	DEFINE_ONLINE_PLAYER_DELEGATE_THREE_PARAM(MAX_LOCAL_PLAYERS, OnQueryUserInfoComplete, bool, const TArray< FUniqueNetIdRef >&, const FString&);

	/**
	 * Obtains the cached list of online user info
	 *
	 * @param LocalUserNum the local user that queried for online user data
	 * @param OutUsers [out] array that receives the copied data
	 *
	 * @return true if user info was found
	 */
	virtual bool GetAllUserInfo(int32 LocalUserNum, TArray< TSharedRef<class FOnlineUser> >& OutUsers) = 0;

	/**
	 * Get the cached user entry for a specific user id if found
	 *
	 * @param LocalUserNum the local user that queried for online user data
	 * @param UserId id to use for finding the cached online user
	 *
	 * @return user info or null ptr if not found
	 */
	virtual TSharedPtr<FOnlineUser> GetUserInfo(int32 LocalUserNum, const class FUniqueNetId& UserId) = 0;

	/**
	 * Called when done querying for a UserId mapping from a requested display name
	 *
	 * @param bWasSuccessful true if server was contacted and a valid result received
	 * @param UserId user id initiating the request
	 * @param DisplayNameOrEmail the name string that was being queried
	 * @param FoundUserId the user id matched for the passed name string
	 * @param Error string representing the error condition
	 */
	DECLARE_DELEGATE_FiveParams(FOnQueryUserMappingComplete, bool /*bWasSuccessful*/, const FUniqueNetId& /*UserId*/, const FString& /*DisplayNameOrEmail*/, const FUniqueNetId& /*FoundUserId*/, const FString& /*Error*/);

	/**
	 * Contacts server to obtain a user id from an arbitrary user-entered name string, eg. display name
	 *
	 * @param UserId id of the user that is requesting the name string lookup
	 * @param DisplayNameOrEmail a string of a display name or email to attempt to map to a user id
	 *
	 * @return true if the operation was started successfully
	 */
	virtual bool QueryUserIdMapping(const FUniqueNetId& UserId, const FString& DisplayNameOrEmail, const FOnQueryUserMappingComplete& Delegate = FOnQueryUserMappingComplete()) = 0;

	/**
	 * Called when done querying for UserId mappings from external ids
	 *
	 * @param bWasSuccessful true if server was contacted and a valid result received
	 * @param UserId user id initiating the request
	 * @param QueryOptions Options specifying how to treat the External IDs and other query-related settings
	 * @param ExternalIds array of external ids to attempt to map to user ids
	 * @param Error string representing the error condition
	 */
	DECLARE_DELEGATE_FiveParams(FOnQueryExternalIdMappingsComplete, bool /*bWasSuccessful*/, const FUniqueNetId& /*UserId*/, const FExternalIdQueryOptions& /*QueryOptions*/, const TArray<FString>& /*ExternalIds*/, const FString& /*Error*/);

	/**
	 * Contacts server to obtain user ids from external ids
	 *
	 * @param UserId id of the user that is requesting the name string lookup
	 * @param QueryOptions Options specifying how to treat the External IDs and other query-related settings
	 * @param ExternalIds array of external ids to attempt to map to user ids
	 *
	 * @return true if the operation was started successfully
	 */
	virtual bool QueryExternalIdMappings(const FUniqueNetId& UserId, const FExternalIdQueryOptions& QueryOptions, const TArray<FString>& ExternalIds, const FOnQueryExternalIdMappingsComplete& Delegate = FOnQueryExternalIdMappingsComplete()) = 0;

	/**
	 * Get the cached user ids for the specified external ids
	 *
	 * @param QueryOptions Options specifying how to treat the External IDs and other query-related settings
	 * @param ExternalIds array of external ids to map to user ids
	 * @param OutIds array of user ids that map to the specified external ids (can contain null entries)
	 */
	virtual void GetExternalIdMappings(const FExternalIdQueryOptions& QueryOptions, const TArray<FString>& ExternalIds, TArray<FUniqueNetIdPtr>& OutIds) = 0;

	/**
	 * Get the cached user id for the specified external id
	 *
	 * @param QueryOptions Options specifying how to treat the External IDs and other query-related settings
	 * @param ExternalId external id to obtain user id for

	 * @return user info or null ptr if not found
	 */
	virtual FUniqueNetIdPtr GetExternalIdMapping(const FExternalIdQueryOptions& QueryOptions, const FString& ExternalId) = 0;
};
