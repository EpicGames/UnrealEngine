// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "TurnBasedMatchInterface.generated.h"

UINTERFACE(Blueprintable)
class UTurnBasedMatchInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class ITurnBasedMatchInterface
{
	GENERATED_IINTERFACE_BODY()

public:
	UFUNCTION(BlueprintImplementableEvent, Category = "Online|TurnBased")
	void OnMatchReceivedTurn(const FString& Match, bool bDidBecomeActive);

	UFUNCTION(BlueprintImplementableEvent, Category = "Online|TurnBased")
	void OnMatchEnded(const FString& Match);
};
