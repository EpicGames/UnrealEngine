// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CoreOnline.h"
#include "OnlineDelegateMacros.h"
#include "OnlineSubsystemPackage.h"
#include "DSP/MultithreadedPatching.h"

class FVoicePacket;

ONLINESUBSYSTEM_API DECLARE_LOG_CATEGORY_EXTERN(LogVoiceEngine, Log, All);
#define UE_LOG_ONLINE_VOICEENGINE(Verbosity, Format, ...) \
{ \
	UE_LOGF(LogVoiceEngine, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

#define UE_CLOG_ONLINE_VOICEENGINE(Conditional, Verbosity, Format, ...) \
{ \
	UE_CLOGF(Conditional, LogVoiceEngine, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

ONLINESUBSYSTEM_API DECLARE_LOG_CATEGORY_EXTERN(LogOnlineVoice, Log, All);

#define UE_LOG_ONLINE_VOICE(Verbosity, Format, ...) \
{ \
	UE_LOGF(LogOnlineVoice, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

#define UE_CLOG_ONLINE_VOICE(Conditional, Verbosity, Format, ...) \
{ \
	UE_CLOGF(Conditional, LogOnlineVoice, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

/** Enable to pipe local voice data back to this client as remote data */
#define VOICE_LOOPBACK !UE_BUILD_SHIPPING

/**
 * Delegate called when a player is talking either remotely or locally
 * Called once for each active talker each frame
 *
 * @param TalkerId the player whose talking state has changed
 * @param bIsTalking if true, player is now talking, otherwise they have now stopped
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPlayerTalkingStateChanged, FUniqueNetIdRef, bool);
typedef FOnPlayerTalkingStateChanged::FDelegate FOnPlayerTalkingStateChangedDelegate;

/**
 * This interface is an abstract mechanism for acquiring voice data from a hardware source. 
 * Each platform implements a specific version of this interface.
 */
class IVoiceEngine
{
PACKAGE_SCOPE:
	IVoiceEngine() {};

	/**
	 * Initialize the voice engine
	 * 
	 * @param MaxLocalTalkers maximum number of local talkers to support
	 * @param MaxRemoteTalkers maximum number of remote talkers to support
	 */
	virtual bool Init(int32 MaxLocalTalkers, int32 MaxRemoteTalkers) = 0;

public:
	/** Virtual destructor to force proper child cleanup */
	virtual ~IVoiceEngine() = default;

	/**
	 * Starts local voice processing for the specified user index
	 *
	 * @param UserIndex the user to start processing for
	 *
	 * @return 0 upon success, an error code otherwise
	 */
	virtual uint32 StartLocalVoiceProcessing(uint32 LocalUserNum) = 0;

	/**
	 * Stops local voice processing for the specified user index
	 *
	 * @param UserIndex the user to stop processing of
	 *
	 * @return 0 upon success, an error code otherwise
	 */
	virtual uint32 StopLocalVoiceProcessing(uint32 LocalUserNum) = 0;

	/**
	 * Starts remote voice processing for the specified user
	 *
	 * @param UniqueId the unique id of the user that will be talking
	 *
	 * @return 0 upon success, an error code otherwise
	 */
	virtual uint32 StartRemoteVoiceProcessing(const FUniqueNetId& UniqueId) = 0;

	/**
	 * Stops remote voice processing for the specified user
	 *
	 * @param UniqueId the unique id of the user that will no longer be talking
	 *
	 * @return 0 upon success, an error code otherwise
	 */
	virtual uint32 StopRemoteVoiceProcessing(const FUniqueNetId& UniqueId) = 0;

	/**
	 * Registers the user index as a local talker (interested in voice data)
	 *
	 * @param UserIndex the user index that is going to be a talker
	 *
	 * @return 0 upon success, an error code otherwise
	 */
	virtual uint32 RegisterLocalTalker(uint32 LocalUserNum) = 0;

	/**
	 * Unregisters the user index as a local talker (not interested in voice data)
	 *
	 * @param UserIndex the user index that is no longer a talker
	 *
	 * @return 0 upon success, an error code otherwise
	 */
	virtual uint32 UnregisterLocalTalker(uint32 LocalUserNum) = 0;

	/**
	 * Registers the unique player id as a remote talker (submitted voice data only)
	 *
	 * @param UniqueId the id of the remote talker
	 *
	 * @return 0 upon success, an error code otherwise
	 */
	virtual uint32 RegisterRemoteTalker(const FUniqueNetId& UniqueId) = 0;

	/**
	 * Unregisters the unique player id as a remote talker
	 *
	 * @param UniqueId the id of the remote talker
	 *
	 * @return 0 upon success, an error code otherwise
	 */
	virtual uint32 UnregisterRemoteTalker(const FUniqueNetId& UniqueId) = 0;

	/**
	 * Checks whether a local user index has a headset present or not
	 *
	 * @param UserIndex the user to check status for
	 *
	 * @return true if there is a headset, false otherwise
	 */
	virtual bool IsHeadsetPresent(uint32 LocalUserNum) = 0;

	/**
	 * Determines whether a local user index is currently talking or not
	 *
	 * @param UserIndex the user to check status for
	 *
	 * @return true if the user is talking, false otherwise
	 */
	virtual bool IsLocalPlayerTalking(uint32 LocalUserNum) = 0;

	/**
	 * Determines whether a remote talker is currently talking or not
	 *
	 * @param UniqueId the unique id of the talker to check status on
	 *
	 * @return true if the user is talking, false otherwise
	 */
	virtual bool IsRemotePlayerTalking(const FUniqueNetId& UniqueId) = 0;

	/**
	 * Returns which local talkers have data ready to be read from the voice system
	 *
	 * @return Bit mask of talkers that have data to be read (1 << UserIndex)
	 */
	virtual uint32 GetVoiceDataReadyFlags() const = 0;

	/**
	 * Sets the playback priority of a remote talker for the given user. A
	 * priority of 0xFFFFFFFF indicates that the player is muted. All other
	 * priorities sorted from zero being most important to higher numbers
	 * being less important.
	 *
	 * @param UserIndex the local talker that is setting the priority
	 * @param UniqueId the id of the remote talker that is having priority changed
	 * @param Priority the new priority to apply to the talker
	 *
	 * @return 0 upon success, an error code otherwise
	 */
	virtual uint32 SetPlaybackPriority(uint32 LocalUserNum, const FUniqueNetId& RemoteTalkerId, uint32 Priority) = 0;

	/**
	 * Reads local voice data for the specified local talker. The size field
	 * contains the buffer size on the way in and contains the amount read on
	 * the way out
	 *
	 * @param UserIndex the local talker that is having their data read
	 * @param Data the buffer to copy the voice data into
	 * @param Size in: the size of the buffer, out: the amount of data copied
	 * @param OutSampleCount: Optional parameter for the starting sample of this voice data.
	 *        Used for sorting packets received out of order.
	 *
	 * @return 0 upon success, an error code otherwise
	 */
	virtual uint32 ReadLocalVoiceData(uint32 LocalUserNum, uint8* Data, uint32* Size) = 0;
	virtual uint32 ReadLocalVoiceData(uint32 LocalUserNum, uint8* Data, uint32* Size, uint64* OutSampleCount) { return ReadLocalVoiceData(LocalUserNum, Data, Size); };

	/**
	 * Submits remote voice data for playback by the voice system. No playback
	 * occurs if the priority for this remote talker is 0xFFFFFFFF. Size
	 * indicates how much data to submit for processing. It's also an out
	 * value in case the system could only process a smaller portion of the data
	 *
	 * @param RemoteTalkerId the remote talker that sent this data
	 * @param Data the buffer to copy the voice data into
	 * @param Size in: the size of the buffer, out: the amount of data copied
	 * @param InSampleCount optional parameter for the starting sample of this voice packet.
	 *        Used for sorting packets received out of order.
	 *
	 * @return 0 upon success, an error code otherwise
	 */
	virtual uint32 SubmitRemoteVoiceData(const FUniqueNetId& RemoteTalkerId, uint8* Data, uint32* Size) = 0;
	virtual uint32 SubmitRemoteVoiceData(const FUniqueNetIdWrapper& RemoteTalkerId, uint8* Data, uint32* Size, uint64& InSampleCount) { return SubmitRemoteVoiceData(*RemoteTalkerId, Data, Size); };

	/**
	 * Used when receiving remote voice data to set the amplitude of a remote talker's incoming audio.
	 * 
	 * @param RemoteTalkerId the remote talker that sent the audio we are tracking the amplitude of.
	 * @param InAmplitude Current loudness of a given remote talker.
	 * 
	 * @return 0 upon success, an error code otherwise
	 */
	virtual uint32 SetRemoteVoiceAmplitude(const FUniqueNetIdWrapper& RemoteTalkerId, float InAmplitude) { return INDEX_NONE; }

	/**
	 * Allows for platform specific servicing of devices, etc.
	 *
	 * @param DeltaTime the amount of time that has elapsed since the last update
	 */
	virtual void Tick(float DeltaTime) = 0;

	/**
	 * Get information about the voice state for display
	 */
	virtual FString GetVoiceDebugState() const = 0;

	/**
	 * This can be used to get arbitrarily tap outgoing microphone audio from local players to use elsewhere in your application. 
	 * for information on usage, see MultithreadedPatching.h.
	 *
	 * @return nullptr if the VoiceEngine implementation being used does not support tapping audio output from the microphone.
	 */
	virtual Audio::FPatchOutputStrongPtr GetMicrophoneOutput()
	{
		return nullptr;
	};


	/**
	 * This can be used to get arbitrarily tap incoming VOIP audio from remote players to use elsewhere in your application.
	 * for information on usage, see MultithreadedPatching.h.
	 * @return nullptr if the VoiceEngine implementation being used does not support tapping incoming audio.
	 */
	virtual Audio::FPatchOutputStrongPtr GetRemoteTalkerOutput()
	{
		return nullptr;
	};

	/**
	 * This can be used to get how loud the given microphone input is, typically on a scale between 0.0 and 1.0.
	 * @return -1.0 if getting the mic amplitude is not supported on this platform or LocalUserNum was invalid, 0.0-1.0 otherwise.
	 */
	virtual float GetMicrophoneAmplitude(int32 LocalUserNum)
	{
		return -1.0f;
	}

	/**
	 * This can be used to get how loud the given microphone input is, typically on a scale between 0.0 and 1.0.
	 * @return -1.0 if getting the mic amplitude is not supported on this platform or RemoteUserId is invalid, 0.0-1.0 otherwise.
	 */
	virtual float GetIncomingAudioAmplitude(const FUniqueNetIdWrapper& RemoteUserId)
	{
		return -1.0f;
	}

	/**
	 * This can be used to patch incoming audio to a different audio device.
	 * @param InDeviceName- name of the device to patch to.
	 * @param bMuteInGameOutput- if set to true, the audio going out to the game will be muted.
	 */
	virtual bool PatchRemoteTalkerOutputToEndpoint(const FString& InDeviceName, bool bMuteInGameOutput = true)
	{
		return false;
	}
	
	/**
	 * This can be used to patch outgoing audio to a different audio device.
	 * @param InDeviceName- name of the device to patch to.
	 */
	virtual bool PatchLocalTalkerOutputToEndpoint(const FString& InDeviceName)
	{
		return false;
	}

	/**
	 * When called, disconnects all additional audio endpoints we patched to using PatchVoiceOutputToDevice().
	 */
	virtual void DisconnectAllEndpoints()
	{
	}

};

typedef TSharedPtr<IVoiceEngine, ESPMode::ThreadSafe> IVoiceEnginePtr;

/**
 * This interface is an abstract mechanism for managing voice data. 
 * Each platform implements a specific version of this interface. 
 */
class IOnlineVoice
{
protected:
	IOnlineVoice() {};


PACKAGE_SCOPE:

	virtual IVoiceEnginePtr CreateVoiceEngine() = 0;

	/**
	 * Re-evaluates the muting list for all local talkers
	 */
	virtual void ProcessMuteChangeNotification() = 0;

public:
	virtual void Shutdown() {}

	/**
	* Initialize the voice interface
	*/
	virtual bool Init() { return true; }

	/** Virtual destructor to force proper child cleanup */
	virtual ~IOnlineVoice() = default;

	/**
	 * Tells the voice layer that networked processing of the voice data is allowed
	 * for the specified player. This allows for push-to-talk style voice communication
	 *
	 * @param LocalUserNum the local user to allow network transmission for
	 */
	virtual void StartNetworkedVoice(uint8 LocalUserNum) = 0;

	/**
	 * Tells the voice layer to stop processing networked voice support for the
	 * specified player. This allows for push-to-talk style voice communication
	 *
	 * @param LocalUserNum the local user to disallow network transmission for
	 */
	virtual void StopNetworkedVoice(uint8 LocalUserNum) = 0;

	/**
	 * Registers the user index as a local talker (interested in voice data)
	 *
	 * @param UserIndex the user index that is going to be a talker
	 *
	 * @return true if the call succeeded, false otherwise
	 */
	virtual bool RegisterLocalTalker(uint32 LocalUserNum) = 0;

	/**
	 * Registers all signed in local talkers
	 */
	virtual void RegisterLocalTalkers() = 0;

	/**
	 * Unregisters the user index as a local talker (not interested in voice data)
	 *
	 * @param UserIndex the user index that is no longer a talker
	 *
	 * @return true if the call succeeded, false otherwise
	 */
	virtual bool UnregisterLocalTalker(uint32 LocalUserNum) = 0;

	/**
	 * Unregisters all signed in local talkers
	 */
	virtual void UnregisterLocalTalkers() = 0;

	/**
	 * Registers the unique player id as a remote talker (submitted voice data only)
	 *
	 * @param UniqueId the id of the remote talker
	 *
	 * @return true if the call succeeded, false otherwise
	 */
	virtual bool RegisterRemoteTalker(const FUniqueNetId& UniqueId) = 0;

	/**
	 * Unregisters the unique player id as a remote talker
	 *
	 * @param UniqueId the id of the remote talker
	 *
	 * @return true if the call succeeded, false otherwise
	 */
	virtual bool UnregisterRemoteTalker(const FUniqueNetId& UniqueId) = 0;

	/**
	 * Iterates the current remote talker list unregistering them with the
	 * voice engine and our internal state
	 */
	virtual void RemoveAllRemoteTalkers() = 0;

	/**
	 * Checks whether a local user index has a headset present or not
	 *
	 * @param UserIndex the user to check status for
	 *
	 * @return true if there is a headset, false otherwise
	 */
	virtual bool IsHeadsetPresent(uint32 LocalUserNum) = 0;

	/**
	 * Determines whether a local user index is currently talking or not
	 *
	 * @param UserIndex the user to check status for
	 *
	 * @return true if the user is talking, false otherwise
	 */
	virtual bool IsLocalPlayerTalking(uint32 LocalUserNum) = 0;

	/**
	 * Determines whether a remote talker is currently talking or not
	 *
	 * @param UniqueId the unique id of the talker to check status on
	 *
	 * @return true if the user is talking, false otherwise
	 */
	virtual bool IsRemotePlayerTalking(const FUniqueNetId& UniqueId) = 0;

	/**
	 * Delegate called when a player is talking either remotely or locally
	 * Called once for each active talker each frame
	 *
	 * @param TalkerId the player whose talking state has changed
	 * @param bIsTalking if true, player is now talking, otherwise they have now stopped
	 */
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnPlayerTalkingStateChanged, FUniqueNetIdRef, bool);

	/**
	 * Checks that a unique player id is on the specified user's mute list
	 *
	 * @param LocalUserNum the controller number of the associated user
	 * @param UniqueId the id of the player being checked
	 *
	 * @return true if the specified user is muted, false otherwise
	 */
	virtual bool IsMuted(uint32 LocalUserNum, const FUniqueNetId& UniqueId) const = 0;

	/**
	 * Mutes a remote talker for the specified local player. NOTE: This only mutes them in the
	 * game unless the bIsSystemWide flag is true, which attempts to mute them globally
	 *
	 * @param LocalUserNum the user that is muting the remote talker
	 * @param PlayerId the remote talker that is being muted
	 * @param bIsSystemWide whether to try to mute them globally or not
	 *
	 * @return true if the function succeeds, false otherwise
	 */
	virtual bool MuteRemoteTalker(uint8 LocalUserNum, const FUniqueNetId& PlayerId, bool bIsSystemWide) = 0;

	/**
	 * Allows a remote talker to talk to the specified local player. NOTE: This only unmutes them in the
	 * game unless the bIsSystemWide flag is true, which attempts to unmute them globally
	 *
	 * @param LocalUserNum the user that is allowing the remote talker to talk
	 * @param PlayerId the remote talker that is being restored to talking
	 * @param bIsSystemWide whether to try to unmute them globally or not
	 *
	 * @return TRUE if the function succeeds, FALSE otherwise
	 */
	virtual bool UnmuteRemoteTalker(uint8 LocalUserNum, const FUniqueNetId& PlayerId, bool bIsSystemWide) = 0;

	/**
	 * Convert generic network packet data back into voice data
	 *
	 * @param Ar archive to read data from
	 */
	virtual TSharedPtr<class FVoicePacket> SerializeRemotePacket(FArchive& Ar) = 0;

	/**
	 * Get the local voice packet intended for send
	 * 
	 * @param LocalUserNum user index voice packet to retrieve
	 */
	virtual TSharedPtr<class FVoicePacket> GetLocalPacket(uint32 LocalUserNum) = 0;

	/**
	 * @return total number of local talkers on this system
	 */
	virtual int32 GetNumLocalTalkers() = 0;

	/**
	 * Clears all voice packets currently queued for send
	 */
	virtual void ClearVoicePackets() = 0;

	/**
	 * Allows for platform specific servicing of devices, etc.
	 *
	 * @param DeltaTime the amount of time that has elapsed since the last update
	 */
	virtual void Tick(float DeltaTime) = 0;

	/**
	 * Get information about the voice state for display
	 */
	virtual FString GetVoiceDebugState() const = 0;

	/**
	 * This can be used to get arbitrarily tap outgoing microphone audio from local players to use elsewhere in your application.
	 * for information on usage, see MultithreadedPatching.h.
	 *
	 * @return nullptr if the VoiceEngine implementation being used does not support tapping audio output from the microphone.
	 */
	virtual Audio::FPatchOutputStrongPtr GetMicrophoneOutput()
	{
		return nullptr;
	};


	/**
	 * This can be used to get arbitrarily tap incoming VOIP audio from remote players to use elsewhere in your application.
	 * for information on usage, see MultithreadedPatching.h.
	 * @return nullptr if the VoiceEngine implementation being used does not support tapping incoming audio.
	 */
	virtual Audio::FPatchOutputStrongPtr GetRemoteTalkerOutput()
	{
		return nullptr;
	};

	/**
	 * This returns the current amplitude of a given player's incoming audio.
	 * @return -1.0 if the PlayerId was not found or if this isn't supported with the current Voice Interface, and a value from 0.0 to 1.0 otherwise.
	 */
	virtual float GetAmplitudeOfRemoteTalker(const FUniqueNetId& PlayerId)
	{
		return -1.0f;
	}
    
    /*
	 * This can be used to patch incoming audio to a different audio device.
	 * @param InDeviceName- name of the device to patch to.
	 * @param bMuteInGameOutput- if set to true, the audio going out to the game will be muted.
	 */
	virtual bool PatchRemoteTalkerOutputToEndpoint(const FString& InDeviceName, bool bMuteInGameOutput = true)
	{
		return false;
	}

	/**
	 * This can be used to patch outgoing audio to a different audio device.
	 * @param InDeviceName- name of the device to patch to.
	 */
	virtual bool PatchLocalTalkerOutputToEndpoint(const FString& InDeviceName)
	{
		return false;
	}


	/**
	 * When called, disconnects all additional audio endpoints we patched to using PatchRemoteTalkerOutputToEndpoint().
	 */
	virtual void DisconnectAllEndpoints()
	{
	}
};

typedef TSharedPtr<IOnlineVoice, ESPMode::ThreadSafe> IOnlineVoicePtr;

/**
 * Definition of a local player's talking state
 */
struct FLocalTalker
{
	/** Whether this player was already registered with the voice interface or not */
	bool bIsRegistered;
	/** Whether the talker should send network data */
	bool bHasNetworkedVoice;
	/** Used to trigger talking delegates only after a certain period of time has passed */
	float LastNotificationTime;
	/** Whether the local talker was speaking last frame */
	bool bWasTalking;
	/** Whether the local talker is speaking this frame */
	bool bIsTalking;

	/** Constructors */
	FLocalTalker() :
		bIsRegistered(false),
		bHasNetworkedVoice(false),
		LastNotificationTime(0.0f),
		bWasTalking(false),
		bIsTalking(false)
	{}
};

/**
 * Definition of a remote player's talking state
 */
struct FRemoteTalker
{
	/** The unique id for this talker */
	FUniqueNetIdPtr TalkerId;
	/** Used to trigger talking delegates only after a certain period of time has passed */
	float LastNotificationTime;
	/** Whether the remote talker was speaking last frame */
	bool bWasTalking;
	/** Whether the remote talker is speaking this frame */
	bool bIsTalking;

	/** Constructors */
	FRemoteTalker() :
		TalkerId(NULL),
		LastNotificationTime(0.0f),
		bWasTalking(false),
		bIsTalking(false)
	{}
};
