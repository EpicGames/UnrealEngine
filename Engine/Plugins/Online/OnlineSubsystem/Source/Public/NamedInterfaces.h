// Copyright Epic Games, Inc. All Rights Reserved.

//
// Named interfaces for Online Subsystem
//

#pragma once

#include "NamedInterfaces.generated.h"

/** Holds a named object interface for dynamically bound interfaces */
USTRUCT()
struct FNamedInterface
{
	GENERATED_USTRUCT_BODY()

	FNamedInterface() :
		InterfaceName(NAME_None),
		InterfaceObject(NULL)
	{
	}

	/** The name to bind this object to */
	UPROPERTY()
	FName InterfaceName;
	/** The object to store at this location */
	UPROPERTY()
	TObjectPtr<UObject> InterfaceObject;
};

/** Holds a name to class name mapping for adding the named interfaces automatically */
USTRUCT()
struct FNamedInterfaceDef
{
	GENERATED_USTRUCT_BODY()

	FNamedInterfaceDef() : 
		InterfaceName(NAME_None)
	{
	}

	/** The name to bind this object to */
	UPROPERTY()
	FName InterfaceName;
	/** The class to load and create for the named interface */
	UPROPERTY()
	FString InterfaceClassName;
};

/**
 * Named interfaces are a registry of UObjects accessible by an FName key that will persist for the lifetime of the process
 */
UCLASS(transient, config=Engine)
class UNamedInterfaces: public UObject
{
	GENERATED_UCLASS_BODY()

	/**
	 * Cleanup delegate fired on destruction
	 */
	DECLARE_MULTICAST_DELEGATE(FOnCleanup);

	// UObject interface begin
	virtual void BeginDestroy() override;
	// UObject interface end

	/**
	 * Initialize the named interface and create any predefined interfaces
	 */
	void Initialize();

	/** @return an interface stored by name */
	class UObject* GetNamedInterface(FName InterfaceName) const;

	/** set an interface stored by name, replacing any existing */
	void SetNamedInterface(FName InterfaceName, class UObject* NewInterface);

	/** get number of current named interfaces */
	int32 GetNumInterfaces() { return NamedInterfaces.Num(); }

	/** @return delegate fired on cleanup */
	FOnCleanup& OnCleanup() { return CleanupDelegates; }

private:

	/** Holds the set of registered named interfaces */
	UPROPERTY()
	TArray<FNamedInterface> NamedInterfaces;

	/** The list of named interfaces to automatically create and store */
	UPROPERTY(config)
	TArray<FNamedInterfaceDef> NamedInterfaceDefs;

	FOnCleanup CleanupDelegates;
};
