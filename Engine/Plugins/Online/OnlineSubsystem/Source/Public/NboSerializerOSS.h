// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/NboSerializer.h"

#include "OnlineSessionSettings.h"

/**
 * Serializes data in network byte order form into a buffer
 */
class FNboSerializeToBufferOSS : public FNboSerializeToBuffer
{
public:
	/** Default constructor zeros num bytes*/
	FNboSerializeToBufferOSS() :
		FNboSerializeToBuffer(512)
	{
	}

	/** Constructor specifying the size to use */
	FNboSerializeToBufferOSS(uint32 Size) :
		FNboSerializeToBuffer(Size)
	{
	}

	/**
	 * Writes a list of key value pairs to buffer
	 */
	template<class KeyType, class ValueType>
	friend inline FNboSerializeToBufferOSS& operator<<(FNboSerializeToBufferOSS& Ar, const FOnlineKeyValuePairs<KeyType,ValueType>& KeyValuePairs)
	{
		((FNboSerializeToBuffer&)Ar) << KeyValuePairs.Num();
		for (typename FOnlineKeyValuePairs<KeyType,ValueType>::TConstIterator It(KeyValuePairs); It; ++It)
		{
			((FNboSerializeToBuffer&)Ar) << It.Key();
			Ar << It.Value();
		}
		return Ar;
	}

	/**
	 * Adds a key value pair to the buffer
	 */
	friend inline FNboSerializeToBufferOSS& operator<<(FNboSerializeToBufferOSS& Ar,const FVariantData& KeyValuePair)
	{
		// Write the type
		uint8 Type = KeyValuePair.GetType();
		((FNboSerializeToBuffer&)Ar) << Type;
		// Now write out the held data
		switch (KeyValuePair.GetType())
		{
		case EOnlineKeyValuePairDataType::Float:
			{
				float Value;
				KeyValuePair.GetValue(Value);
				((FNboSerializeToBuffer&)Ar) << Value;
				break;
			}
		case EOnlineKeyValuePairDataType::Int32:
			{
				int32 Value;
				KeyValuePair.GetValue(Value);
				((FNboSerializeToBuffer&)Ar) << Value;
				break;
			}
		case EOnlineKeyValuePairDataType::Int64:
			{
				uint64 Value;
				KeyValuePair.GetValue(Value);
				((FNboSerializeToBuffer&)Ar) << Value;
				break;
			}
		case EOnlineKeyValuePairDataType::Double:
			{
				double Value;
				KeyValuePair.GetValue(Value);
				((FNboSerializeToBuffer&)Ar) << Value;
				break;
			}
		case EOnlineKeyValuePairDataType::Blob:
			{
				TArray<uint8> Value;
				KeyValuePair.GetValue(Value);

				// Write the length
				((FNboSerializeToBuffer&)Ar) << Value.Num();
				// Followed by each byte of data
				for (int32 Index = 0; Index < Value.Num(); Index++)
				{
					((FNboSerializeToBuffer&)Ar) << Value[Index];
				}
				break;
			}
		case EOnlineKeyValuePairDataType::String:
			{
				FString Value;
				KeyValuePair.GetValue(Value);
				// This will write a length prefixed string
				((FNboSerializeToBuffer&)Ar) << *Value;
				break;
			}
		case EOnlineKeyValuePairDataType::Bool:
			{
				bool Value;
				KeyValuePair.GetValue(Value);
				((FNboSerializeToBuffer&)Ar) << (uint8)(Value ? 1 : 0);
				break;
			}
		case EOnlineKeyValuePairDataType::Empty:
			break;
		default:
			checkfSlow(false, TEXT("Unsupported EOnlineKeyValuePairDataType: %d"), Type);
		}
		return Ar;
	}

	/**
	 * Adds a single online session setting to the buffer
	 */
	friend inline FNboSerializeToBufferOSS& operator<<(FNboSerializeToBufferOSS& Ar,const FOnlineSessionSetting& Setting)
	{
		Ar << Setting.Data;
		
		uint8 Type;
		Type = Setting.AdvertisementType;
		((FNboSerializeToBuffer&)Ar) << Type;

		return Ar;
	}
};

/**
 * Class used to read data from a NBO data buffer
 */
class FNboSerializeFromBufferOSS : public FNboSerializeFromBuffer
{
public:
	/**
	 * Initializes the buffer, size, and zeros the read offset
	 */
	FNboSerializeFromBufferOSS(const uint8* Packet,int32 Length) :
		FNboSerializeFromBuffer(Packet,Length)
	{
	}

	/**
	 * Reads a list of key value pairs from the buffer
	 */
	template<class KeyType, class ValueType>
	friend inline FNboSerializeFromBufferOSS& operator>>(FNboSerializeFromBufferOSS& Ar, FOnlineKeyValuePairs<KeyType,ValueType>& KeyValuePairs)
	{
		int32 NumValues;
		Ar >> NumValues;		
		for (int32 Idx=0; Idx < NumValues; Idx++)
		{
			KeyType Key;
			ValueType Value;
			Ar >> Key;
			Ar >> Value;

			KeyValuePairs.Add(Key,Value);
		}
		return Ar;
	}
	
	/**
	 * Reads a key value pair from the buffer
	 */
	friend inline FNboSerializeFromBufferOSS& operator>>(FNboSerializeFromBufferOSS& Ar,FVariantData& KeyValuePair)
	{
		if (!Ar.HasOverflow())
		{
			// Read the type
			uint8 Type;
			Ar >> Type;

			// Now read in the held data
			switch ((EOnlineKeyValuePairDataType::Type)Type)
			{
			case EOnlineKeyValuePairDataType::Float:
				{
					float Value;
					Ar >> Value;
					KeyValuePair.SetValue(Value);
					break;
				}
			case EOnlineKeyValuePairDataType::Int32:
				{
					int32 Value;
					Ar >> Value;
					KeyValuePair.SetValue(Value);
					break;
				}
			case EOnlineKeyValuePairDataType::Int64:
				{
					uint64 Value;
					Ar >> Value;
					KeyValuePair.SetValue(Value);
					break;
				}
			case EOnlineKeyValuePairDataType::Double:
				{
					double Value;
					Ar >> Value;
					KeyValuePair.SetValue(Value);
					break;
				}
			case EOnlineKeyValuePairDataType::Blob:
				{
					int32 Length;
					Ar >> Length;

					// Check this way to trust NumBytes and CurrentOffset to be more accurate than the packet Len value
					const bool bSizeOk = (Length >= 0) && (Length <= (Ar.NumBytes - Ar.CurrentOffset));
					if (!Ar.HasOverflow() && bSizeOk)
					{
						// Now directly copy the blob data
						KeyValuePair.SetValue(Length, &Ar.Data[Ar.CurrentOffset]);
						Ar.CurrentOffset += Length;
					}
					else
					{
						Ar.bHasOverflowed = true;
					}

					break;
				}
			case EOnlineKeyValuePairDataType::String:
				{
					FString Value;
					Ar >> Value;
					KeyValuePair.SetValue(Value);
					break;
				}
			case EOnlineKeyValuePairDataType::Bool:
				{
					uint8 Value;
					Ar >> Value;
					KeyValuePair.SetValue(Value != 0);
					break;
				}
			case EOnlineKeyValuePairDataType::Empty:
				break;
			default:
				checkfSlow(false, TEXT("Unsupported EOnlineKeyValuePairDataType: %d"), Type);
			}
		}

		return Ar;
	}

	/**
	 * Reads a single session setting from the buffer
	 */
	friend inline FNboSerializeFromBufferOSS& operator>>(FNboSerializeFromBufferOSS& Ar,FOnlineSessionSetting& Setting)
	{
		Ar >> Setting.Data;

		if (!Ar.HasOverflow())
		{
			uint8 Type;
			Ar >> Type;
			Setting.AdvertisementType = (EOnlineDataAdvertisementType::Type) Type;
		}

		return Ar;
	}
};
