// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystem.h"

#include "Interfaces/OnlineIdentityInterface.h"
#include "Interfaces/OnlineFriendsInterface.h"
#include "Interfaces/OnlineSessionInterface.h"
#include "Interfaces/OnlineExternalUIInterface.h"
#include "Interfaces/VoiceInterface.h"
#include "Interfaces/OnlineTitleFileInterface.h"
#include "Interfaces/OnlineEntitlementsInterface.h"
#include "Interfaces/OnlineUserCloudInterface.h"

/** Macro to handle the boilerplate of accessing the proper online subsystem and getting the requested interface */
#define IMPLEMENT_GET_INTERFACE(InterfaceType) \
static IOnline##InterfaceType##Ptr Get##InterfaceType##Interface(const FName SubsystemName = NAME_None) \
{ \
	IOnlineSubsystem* OSS = IOnlineSubsystem::Get(SubsystemName); \
	return (OSS == NULL) ? NULL : OSS->Get##InterfaceType##Interface(); \
} \
static IOnline##InterfaceType##Ptr Get##InterfaceType##InterfaceChecked(const FName SubsystemName = NAME_None) \
{ \
	IOnlineSubsystem* OSS = IOnlineSubsystem::Get(SubsystemName); \
	check(OSS); \
	IOnline##InterfaceType##Ptr InterfacePtr = OSS->Get##InterfaceType##Interface(); \
	check(InterfacePtr.IsValid()); \
	return InterfacePtr; \
} 

/** Helpers for accessing all the online features available in the online subsystem */
namespace Online
{
	/** 
	 * Get the interface for accessing the session services
	 * @param SubsystemName - Name of the requested online service
	 * @return Interface pointer for the appropriate session service
	 */
	IMPLEMENT_GET_INTERFACE(Session);

	/**
	 * Get the interface for accessing the party services
	 * @param SubsystemName - Name of the requested online service
	 * @return Interface pointer for the appropriate party service
	 */
	IMPLEMENT_GET_INTERFACE(Party);

	/**
	 * Get the interface for accessing the chat services
	 * @param SubsystemName - Name of the requested online service
	 * @return Interface pointer for the appropriate party service
	 */
	IMPLEMENT_GET_INTERFACE(Chat);

	/** 
	 * Get the interface for accessing the player friends services
	 * @param SubsystemName - Name of the requested online service
	 * @return Interface pointer for the appropriate friend service
	 */
	IMPLEMENT_GET_INTERFACE(Friends);

	/** 
	 * Get the interface for accessing user information by uniqueid
	 * @param SubsystemName - Name of the requested online service
	 * @return Interface pointer for the appropriate user service
	 */
	IMPLEMENT_GET_INTERFACE(User);

	/** 
	 * Get the interface for sharing user files in the cloud
	 * @return Interface pointer for the appropriate cloud service
	 */
	IMPLEMENT_GET_INTERFACE(SharedCloud);

	/** 
	 * Get the interface for accessing user files in the cloud
	 * @return Interface pointer for the appropriate cloud service
	 */
	IMPLEMENT_GET_INTERFACE(UserCloud);

	/** 
	 * Get the interface for accessing voice services
	 * @param SubsystemName - Name of the requested online service
	 * @return Interface pointer for the appropriate voice service
	 */
	IMPLEMENT_GET_INTERFACE(Voice);

	/** 
	 * Get the interface for accessing the external UIs of a service
	 * @param SubsystemName - Name of the requested online service
	 * @return Interface pointer for the appropriate external UI service
	 */
	IMPLEMENT_GET_INTERFACE(ExternalUI);

	/** 
	 * Get the interface for accessing the server time from an online service
	 * @param SubsystemName - Name of the requested online service
	 * @return Interface pointer for the appropriate server time service
	 */
	IMPLEMENT_GET_INTERFACE(Time);

	/** 
	 * Get the interface for accessing identity online services
	 * @param SubsystemName - Name of the requested online service
	 * @return Interface pointer for the appropriate identity service
	 */
	IMPLEMENT_GET_INTERFACE(Identity);
	
	/** 
	 * Get the interface for accessing title file online services
	 * @param SubsystemName - Name of the requested online service
	 * @return Interface pointer for the appropriate service
	 */
	IMPLEMENT_GET_INTERFACE(TitleFile);

	/** 
	 * Get the interface for accessing entitlements online services
	 * @param SubsystemName - Name of the requested online service
	 * @return Interface pointer for the appropriate service
	 */
	IMPLEMENT_GET_INTERFACE(Entitlements);

	/** 
	 * Get the interface for accessing platform leaderboards
	 * @param SubsystemName - Name of the requested online service
	 * @return Interface pointer for the appropriate leaderboard service
	 */
	IMPLEMENT_GET_INTERFACE(Leaderboards);

	/** 
	 * Get the interface for accessing entitlements online services
	 * @param SubsystemName - Name of the requested online service
	 * @return Interface pointer for the appropriate service
	 */
	IMPLEMENT_GET_INTERFACE(Achievements);

	/** 
	 * Get the interface for accessing online events
	 * @param SubsystemName - Name of the requested online service
	 * @return Interface pointer for the appropriate service
	 */
	IMPLEMENT_GET_INTERFACE(Events);

	/** 
	 * Get the interface for accessing rich presence online services
	 * @param SubsystemName - Name of the requested online service
	 * @return Interface pointer for the appropriate service
	 */
	IMPLEMENT_GET_INTERFACE(Presence);

	/**
	 * Get the interface for accessing the stats services
	 * @param SubsystemName - Name of the requested online service
	 * @return Interface pointer for the appropriate party service
	 */
	IMPLEMENT_GET_INTERFACE(Stats);

	/**
	 * Get the interface for accessing the purchasing services
	 * @param SubsystemName - Name of the requested online service
	 * @return Interface pointer for the appropriate party service
	 */
	IMPLEMENT_GET_INTERFACE(Purchase);
};

#undef IMPLEMENT_GET_INTERFACE
