// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreGlobals.h"
#include "HAL/PlatformTime.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Runnable.h"
#include "HAL/ThreadSafeBool.h"
#include "Misc/SingleThreadRunnable.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "OnlineSubsystemPackage.h"
#include "Containers/Queue.h"

#define UE_API ONLINESUBSYSTEM_API

/**
 * Base class of any async task that can be returned to the game thread by the async task manager
 * May originate on the game thread, or generated by an external platform service callback from the online thread itself
 */
class FOnlineAsyncItem
{
protected:
	/** Time the task was created */
	double StartTime;

	/** Hidden on purpose */
	FOnlineAsyncItem() 
	{
		StartTime = FPlatformTime::Seconds();
	}

public:
	
	virtual ~FOnlineAsyncItem() = default; 

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const = 0;

	/**
	 * Updates the amount of elapsed time this task has taken
	 */
	double GetElapsedTime()
	{
		return FPlatformTime::Seconds() - StartTime;
	}

	/**
	 * Give the async task a chance to marshal its data back to the game thread
	 * Can only be called on the game thread by the async task manager
	 */
	virtual void Finalize()
	{
		// assert that we're on the game thread
		check(IsInGameThread());
	}

	/**
	 *	Async task is given a chance to trigger it's delegates
	 */
	virtual void TriggerDelegates() 
	{
		// assert that we're on the game thread
		check(IsInGameThread());
	}
};

/**
 * An event triggered by the online subsystem to be routed to the game thread for processing
 * Originates on the online thread
 */
template<class T>
class FOnlineAsyncEvent : public FOnlineAsyncItem
{
PACKAGE_SCOPE:

	/** Reference to online subsystem */
	T* Subsystem;

	/** Hidden on purpose */
	FOnlineAsyncEvent() : 
		Subsystem(nullptr)
	{
	}

public:

	FOnlineAsyncEvent(T* InSubsystem) 
		: Subsystem(InSubsystem)
	{
	}
};

/**
 * A single task for an online service to be queued with the async task manager
 * Originates on the game thread
 */
class FOnlineAsyncTask : public FOnlineAsyncItem
{
protected:

	/** Hidden on purpose */
	FOnlineAsyncTask() 
	{
	}

public:

	virtual ~FOnlineAsyncTask() = default;

	/**
	 * Initialize the task - called on game thread when queued
	 */
	virtual void Initialize() {}

	/**
	 * Check the state of the async task
	 * @return true if complete, false otherwise
	 */
	virtual bool IsDone() const = 0;

	/**
	 * Check the success of the async task
	 * @return true if successful, false otherwise
	 */
	virtual bool WasSuccessful() const = 0;

	/**
	 * Give the async task time to do its work
	 * Can only be called on the async task manager thread
	 */
	virtual void Tick()
	{
		// assert that we're not on the game thread
		check(!IsInGameThread() || !FPlatformProcess::SupportsMultithreading());
	}

	/**
	 * If cancel is supported, cancel when timeout if timeout is set for this task
	 * Not putting this in Tick directly because existing subclasses could be overriding Tick without calling super::Tick
	 */
	virtual void CancelWhenTimeout() {}
};


/**
 * An async task that can execute any callable type with no parameters.
 * For example, l lambda, or an object with an operator().
 * Useful for calling simple functions that need to run on the game thread,
 * but are invoked from an online service thread.
 */
template<class CallableType>
class FOnlineAsyncTaskGenericCallable : public FOnlineAsyncTask
{
public:
	/**
	 * Constructor.
	 *
	 * @param InCallable any object that can be called with no parameters, usually a lambda
	 */
	explicit FOnlineAsyncTaskGenericCallable(const CallableType& InCallable)
		: CallableObject(InCallable) {}

	virtual void Finalize() override
	{
		CallableObject();
	}

	virtual FString ToString() const override { return FString("FOnlineAsyncTaskGenericCallable"); }

	virtual bool IsDone() const override { return true; }
	virtual bool WasSuccessful() const override { return true; }

private:
	/** Stored copy of the object to invoke on the game thread. */
	CallableType CallableObject;
};

/**
 * An async task that can execute any callable type with no parameters.
 * For example, l lambda, or an object with an operator().
 * Useful for calling simple functions that need to run on the ONLINE thread.
 */
template<class CallableType>
class FOnlineAsyncTaskThreadedGenericCallable : public FOnlineAsyncTask
{
public:
	/**
	* Constructor.
	*
	* @param InCallable any object that can be called with no parameters, usually a lambda
	*/
	explicit FOnlineAsyncTaskThreadedGenericCallable(const FString& InCallableName, const CallableType& InCallable)
		: bHasTicked(false)
		, CallableObject(InCallable)
		, CallableName(InCallableName)
	{
	}

	virtual void Tick() override
	{
		CallableObject();
		bHasTicked = true;
	}

	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncTaskThreadedGenericCallable (%s)"), *CallableName); }

	virtual bool IsDone() const override { return bHasTicked; }
	virtual bool WasSuccessful() const override { return true; }

private:
	/** True after it has ticked once and run the Callable on the online thread */
	bool bHasTicked;
	/** Stored copy of the object to invoke on the game thread. */
	CallableType CallableObject;
	/** Name of the task to help logging */
	FString CallableName;
};

template<class T>
class FOnlineAsyncTaskBasic : public FOnlineAsyncTask
{
PACKAGE_SCOPE:

	/** Reference to online subsystem */
	T* Subsystem;
	/** Has the task completed */
	FThreadSafeBool bIsComplete;
	/** Has the task complete successfully */
	FThreadSafeBool bWasSuccessful;

	/** Hidden on purpose */
	FOnlineAsyncTaskBasic()
		: Subsystem(nullptr)
		, bIsComplete(false)
		, bWasSuccessful(false)
	{
	}

public:

	FOnlineAsyncTaskBasic(T* const InSubsystem)
		: Subsystem(InSubsystem)
		, bIsComplete(false)
		, bWasSuccessful(false)
	{
	}

	virtual ~FOnlineAsyncTaskBasic() = default;

	/**
	 * Copies the contents of this task to another
	 * Use sparingly, it is not atomic.
	 */
	FOnlineAsyncTaskBasic& operator= (const FOnlineAsyncTaskBasic& Other)
	{
		if (this != &Other)
		{
			Subsystem = Other.Subsystem;
			bIsComplete = !!Other.bIsComplete;
			bWasSuccessful = !!Other.bWasSuccessful;
		}

		return *this;
	}

	/**
	 * Check the state of the async task
	 * @return true if complete, false otherwise
	 */
	virtual bool IsDone() const override
	{
		return bIsComplete;
	}

	/**
	 * Check the success of the async task
	 * @return true if successful, false otherwise
	 */
	virtual bool WasSuccessful() const override
	{
		return bWasSuccessful;
	}
};

/**
 *	The foundation of all async operations in every online subsystem
 *
 * A task manager ticks on its own thread, managing both a serial and parallel queue of FOnlineAsyncTasks
 * Each task works through the serial queue in the following manner
 *	GameThread
 *	- Initialize()
 *	OnlineThread
 *	-- Tick() until IsDone()
 *	-- Add task to OutQueue
 *	GameThread
 *	- Finalize()
 *	- TriggerDelegates() 
 *
 * Parallel task queue works in a similar flow, except the tasks don't wait for any previous tasks to complete
 */
class FOnlineAsyncTaskManager : public FRunnable, FSingleThreadRunnable 
{
private:
	
	/** The current active task processed serially by the async task manager */
	FOnlineAsyncTask* ActiveSerialTask = nullptr;
	/** Critical section for modifying the active serial task */
	FCriticalSection ActiveSerialTaskLock;
	/** When the active serial task was started on the game thread */
	double ActiveSerialTaskStartTime = 0.0;
	/** Has a breach been reported for the active task */
	std::atomic<bool> bActiveSerialTaskBreachReported = false;

	struct FOnlineThreadTickInfo
	{
		/* Is the online thread currently ticking */
		bool bIsTicking = false;
		/* Time the online thread began this tick */
		double TickStartTime = 0.0;
		/* Task the online thread is currently ticking */
		FOnlineAsyncTask* CurrentTask = nullptr;
		/* */
		bool bBreachReported = false;
	};
	/** Info on what the online thread is currently doing */
	FOnlineThreadTickInfo OnlineThreadTickInfo;
	/** Critical section for modifying this tick info */
	FCriticalSection OnlineThreadTickInfoLock;

	/** Should we report a breach or not */
	bool bEnableReportBreach = false;
	/** Time to allow a serial task / the online thread tick to run before reporting */
	float ConfigBreachTimeSeconds = 60.0;

protected:

	/** Game thread async tasks are queued up here for processing on the online thread */
	TArray<FOnlineAsyncTask*> QueuedSerialTasks;
	/** Critical section for thread safe operation of the event in queue */
	FTransactionallySafeCriticalSection QueuedSerialTasksLock;

	/** Number of tasks that can run in parallel */
	int32 MaxParallelTasks;
	/** Signal game thread to reload the MaxParallelTasks value from config */
	std::atomic<bool> bReloadMaxParallelTasksConfig;
	/** This queue is for tasks that are safe to run in parallel with one another */
	TQueue<FOnlineAsyncTask*, EQueueMode::Mpsc> QueuedParallelTasks;
	/** Tasks that are running in parallel */
	TArray<FOnlineAsyncTask*> ParallelTasks;
	/** Critical section for thread safe operation of the list */
	FCriticalSection ParallelTasksLock;

	/** Completed online requests are queued up here for processing on the game thread */
	TArray<FOnlineAsyncItem*> OutQueue;
    /** Critical section for thread safe operation of the out queue */
	FTransactionallySafeCriticalSection OutQueueLock;

	/** Trigger event to signal the queue has tasks that need processing */
	FEvent* WorkEvent;

	/** Min amount of time to poll for the current task to complete */
	uint32 PollingInterval;

	/** Should this manager and the thread exit */
	FThreadSafeBool bRequestingExit;

	/** Number of async task managers running currently */
	static UE_API int32 InvocationCount;

	/**
	 * Remove a parallel async task from the parallel queue
	 * @param OldTask - some request of the online services
	 */
	UE_API void RemoveFromParallelTasks(FOnlineAsyncTask* OldTask);

PACKAGE_SCOPE:

	/** Set by FOnlineAsyncTaskManager::Run */
	volatile uint32 OnlineThreadId;

public:

	UE_API FOnlineAsyncTaskManager();
	UE_API virtual ~FOnlineAsyncTaskManager();

	/**
	 * Init the online async task manager
	 *
	 * @return True if initialization was successful, false otherwise
	 */
	UE_API virtual bool Init();

	/**
	 * This is where all per object thread work is done. This is only called
	 * if the initialization was successful.
	 *
	 * @return The exit code of the runnable object
	 */
	UE_API virtual uint32 Run();

	/**
	 * This is called if a thread is requested to terminate early
	 */
	UE_API virtual void Stop();

	/**
	 * Called in the context of the aggregating thread to perform any cleanup.
	 */
	UE_API virtual void Exit();

	/**
	 *  FSingleThreadRunnable accessor for ticking this FRunnable when multi-threading is disabled. 
	 *  @return FSingleThreadRunnable Interface for this FRunnable object.
	 */
	virtual class FSingleThreadRunnable* GetSingleThreadInterface() { return this; }

	/**
	 * Add online async tasks that need processing onto the incoming queue
	 * @param NewTask - some request of the online services
	 */
	UE_API void AddToInQueue(FOnlineAsyncTask* NewTask);

	/**
	 * Add completed online async tasks that need processing onto the queue
	 * @param CompletedItem - some finished request of the online services
	 */
	UE_API void AddToOutQueue(FOnlineAsyncItem* CompletedItem);

	/**
	 * Add a new item to the out queue that will call InCallable on the game thread.
	 * Very useful when passing in lambdas as parameters, since this function will
	 * automatically deduce the template parameter type for FOnlineAsyncItemGenericCallable.
	 *
	 * @param InCallable the callable object to execute on the game thread.
	 */
	template<class CallableType>
	void AddGenericToOutQueue(const CallableType& InCallable)
	{
		AddToOutQueue(new FOnlineAsyncTaskGenericCallable<CallableType>(InCallable));
	}

	/**
	 * Add a new item to the in queue that will call InCallable on the game thread.
	 * Very useful when passing in lambdas as parameters, since this function will
	 * automatically deduce the template parameter type for FOnlineAsyncItemGenericCallable.
	 *
	 * Unlike AddGenericToOutQueue, this version is useful when you want to ensure that
	 * the callable will only execute after any existing tasks in the queue are complete.
	 *
	 * @param InCallable the callable object to execute on the game thread.
	 */
	template<class CallableType>
	void AddGenericToInQueue(const CallableType& InCallable)
	{
		AddToInQueue(new FOnlineAsyncTaskGenericCallable<CallableType>(InCallable));
	}

	/**
	* Add a new item to the in queue that will call InCallable on the ONLINE thread.
	* Very useful when passing in lambdas as parameters, since this function will
	* automatically deduce the template parameter type for FOnlineAsyncItemGenericCallable.
	*
	* Unlike AddGenericToInQueue, this version runs the task on the online thread in Tick(), 
	* instead of Finalize on the game thread.
	*
	* @param InCallable the callable object to execute on the game thread.
	*/
	template<class CallableType>
	void AddGenericToInQueueOnlineThread(const FString& CallableName, const CallableType& InCallable)
	{
		AddToInQueue(new FOnlineAsyncTaskThreadedGenericCallable<CallableType>(CallableName, InCallable));
	}

	/**
	 * Add a new online async task that is safe to run in parallel
	 * @param NewTask - some request of the online services
	 */
	UE_API void AddToParallelTasks(FOnlineAsyncTask* NewTask);

	/**
	 *	** CALL ONLY FROM GAME THREAD **
	 * Give the completed async tasks a chance to marshal their data back onto the game thread
	 * Calling delegates where appropriate
	 */
	UE_API void GameTick();

	/**
	 *	** CALL ONLY FROM ONLINE THREAD **
	 * Give the online service a chance to do work
	 */
	virtual void OnlineTick() = 0;

	// FSingleThreadRunnable interface
	/**
	 * Tick() is called by both multithreaded and single threaded runnable
	 */
	UE_API virtual void Tick();

};

#undef UE_API
