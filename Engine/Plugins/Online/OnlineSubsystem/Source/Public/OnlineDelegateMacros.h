// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#define DEFINE_ONLINE_DELEGATE_BASE(DelegateName) \
public: \
	F##DelegateName DelegateName##Delegates; \
public: \
	virtual FDelegateHandle Add##DelegateName##Delegate_Handle(const F##DelegateName##Delegate& Delegate) \
	{ \
		DelegateName##Delegates.Add(Delegate); \
		return Delegate.GetHandle(); \
	} \
	virtual void Clear##DelegateName##Delegate_Handle(FDelegateHandle& Handle) \
	{ \
		DelegateName##Delegates.Remove(Handle); \
		Handle.Reset(); \
	} \
	virtual void Clear##DelegateName##Delegates(FDelegateUserObject Object) \
	{ \
		DelegateName##Delegates.RemoveAll(Object); \
	}

#define DEFINE_ONLINE_DELEGATE(DelegateName) \
	DEFINE_ONLINE_DELEGATE_BASE(DelegateName) \
	virtual void Trigger##DelegateName##Delegates() \
	{ \
		DelegateName##Delegates.Broadcast(); \
	}

#define DEFINE_ONLINE_DELEGATE_ONE_PARAM(DelegateName, Param1Type) \
	DEFINE_ONLINE_DELEGATE_BASE(DelegateName) \
	virtual void Trigger##DelegateName##Delegates(Param1Type Param1) \
	{ \
		DelegateName##Delegates.Broadcast(Param1); \
	}
	
#define DEFINE_ONLINE_DELEGATE_TWO_PARAM(DelegateName, Param1Type, Param2Type) \
	DEFINE_ONLINE_DELEGATE_BASE(DelegateName) \
	virtual void Trigger##DelegateName##Delegates(Param1Type Param1, Param2Type Param2) \
	{ \
		DelegateName##Delegates.Broadcast(Param1, Param2); \
	}

#define DEFINE_ONLINE_DELEGATE_THREE_PARAM(DelegateName, Param1Type, Param2Type, Param3Type) \
	DEFINE_ONLINE_DELEGATE_BASE(DelegateName) \
	virtual void Trigger##DelegateName##Delegates(Param1Type Param1, Param2Type Param2, Param3Type Param3) \
	{ \
		DelegateName##Delegates.Broadcast(Param1, Param2, Param3); \
	}

#define DEFINE_ONLINE_DELEGATE_FOUR_PARAM(DelegateName, Param1Type, Param2Type, Param3Type, Param4Type) \
	DEFINE_ONLINE_DELEGATE_BASE(DelegateName) \
	virtual void Trigger##DelegateName##Delegates(Param1Type Param1, Param2Type Param2, Param3Type Param3, Param4Type Param4) \
	{ \
		DelegateName##Delegates.Broadcast(Param1, Param2, Param3, Param4); \
	}

#define DEFINE_ONLINE_DELEGATE_FIVE_PARAM(DelegateName, Param1Type, Param2Type, Param3Type, Param4Type, Param5Type) \
	DEFINE_ONLINE_DELEGATE_BASE(DelegateName) \
	virtual void Trigger##DelegateName##Delegates(Param1Type Param1, Param2Type Param2, Param3Type Param3, Param4Type Param4, Param5Type Param5) \
	{ \
		DelegateName##Delegates.Broadcast(Param1, Param2, Param3, Param4, Param5); \
	}

#define DEFINE_ONLINE_DELEGATE_SIX_PARAM(DelegateName, Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type) \
	DEFINE_ONLINE_DELEGATE_BASE(DelegateName) \
	virtual void Trigger##DelegateName##Delegates(Param1Type Param1, Param2Type Param2, Param3Type Param3, Param4Type Param4, Param5Type Param5, Param6Type Param6) \
	{ \
		DelegateName##Delegates.Broadcast(Param1, Param2, Param3, Param4, Param5, Param6); \
	}

#define DEFINE_ONLINE_DELEGATE_SEVEN_PARAM(DelegateName, Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type, Param7Type) \
	DEFINE_ONLINE_DELEGATE_BASE(DelegateName) \
	virtual void Trigger##DelegateName##Delegates(Param1Type Param1, Param2Type Param2, Param3Type Param3, Param4Type Param4, Param5Type Param5, Param6Type Param6, Param7Type Param7) \
	{ \
		DelegateName##Delegates.Broadcast(Param1, Param2, Param3, Param4, Param5, Param6, Param7); \
	}


#define DEFINE_ONLINE_PLAYER_DELEGATE_BASE(MaxPlayers, DelegateName) \
public: \
	F##DelegateName DelegateName##Delegates[MaxPlayers]; \
public: \
	virtual FDelegateHandle Add##DelegateName##Delegate_Handle(int32 LocalUserNum, const F##DelegateName##Delegate& Delegate) \
	{ \
		FDelegateHandle Result; \
		if (LocalUserNum >= 0 && LocalUserNum < MaxPlayers) \
		{ \
			Result = DelegateName##Delegates[LocalUserNum].Add(Delegate); \
		} \
		return Result; \
	} \
	virtual void Clear##DelegateName##Delegate_Handle(int32 LocalUserNum, FDelegateHandle& Handle) \
	{ \
		if (LocalUserNum >= 0 && LocalUserNum < MaxPlayers) \
		{ \
			DelegateName##Delegates[LocalUserNum].Remove(Handle); \
			Handle.Reset(); \
		} \
	} \
	virtual void Clear##DelegateName##Delegates(int32 LocalUserNum, FDelegateUserObject Object) \
	{ \
		if (LocalUserNum >= 0 && LocalUserNum < MaxPlayers) \
		{ \
			DelegateName##Delegates[LocalUserNum].RemoveAll(Object); \
		} \
	}

#define DEFINE_ONLINE_PLAYER_DELEGATE(MaxPlayers, DelegateName) \
	DEFINE_ONLINE_PLAYER_DELEGATE_BASE(MaxPlayers, DelegateName) \
	virtual void Trigger##DelegateName##Delegates(int32 LocalUserNum) \
	{ \
		if (LocalUserNum >= 0 && LocalUserNum < MaxPlayers) \
		{ \
			DelegateName##Delegates[LocalUserNum].Broadcast(); \
		} \
	}

#define DEFINE_ONLINE_PLAYER_DELEGATE_ONE_PARAM(MaxPlayers, DelegateName, Param1Type) \
	DEFINE_ONLINE_PLAYER_DELEGATE_BASE(MaxPlayers, DelegateName) \
	virtual void Trigger##DelegateName##Delegates(int32 LocalUserNum, Param1Type Param1) \
	{ \
		if (LocalUserNum >= 0 && LocalUserNum < MaxPlayers) \
		{ \
			DelegateName##Delegates[LocalUserNum].Broadcast(LocalUserNum, Param1); \
		} \
	}

#define DEFINE_ONLINE_PLAYER_DELEGATE_TWO_PARAM(MaxPlayers, DelegateName, Param1Type, Param2Type) \
	DEFINE_ONLINE_PLAYER_DELEGATE_BASE(MaxPlayers, DelegateName) \
	virtual void Trigger##DelegateName##Delegates(int32 LocalUserNum, Param1Type Param1, Param2Type Param2) \
	{ \
		if (LocalUserNum >= 0 && LocalUserNum < MaxPlayers) \
		{ \
			DelegateName##Delegates[LocalUserNum].Broadcast(LocalUserNum, Param1, Param2); \
		} \
	}

#define DEFINE_ONLINE_PLAYER_DELEGATE_THREE_PARAM(MaxPlayers, DelegateName, Param1Type, Param2Type, Param3Type) \
	DEFINE_ONLINE_PLAYER_DELEGATE_BASE(MaxPlayers, DelegateName) \
	virtual void Trigger##DelegateName##Delegates(int32 LocalUserNum, Param1Type Param1, Param2Type Param2, Param3Type Param3) \
	{ \
		if (LocalUserNum >= 0 && LocalUserNum < MaxPlayers) \
		{ \
			DelegateName##Delegates[LocalUserNum].Broadcast(LocalUserNum, Param1, Param2, Param3); \
		} \
	}

#define DEFINE_ONLINE_PLAYER_DELEGATE_FOUR_PARAM(MaxPlayers, DelegateName, Param1Type, Param2Type, Param3Type, Param4Type) \
	DEFINE_ONLINE_PLAYER_DELEGATE_BASE(MaxPlayers, DelegateName) \
	virtual void Trigger##DelegateName##Delegates(int32 LocalUserNum, Param1Type Param1, Param2Type Param2, Param3Type Param3, Param4Type Param4) \
	{ \
		if (LocalUserNum >= 0 && LocalUserNum < MaxPlayers) \
		{ \
			DelegateName##Delegates[LocalUserNum].Broadcast(LocalUserNum, Param1, Param2, Param3, Param4); \
		} \
	}

#define DEFINE_ONLINE_PLAYER_DELEGATE_FIVE_PARAM(MaxPlayers, DelegateName, Param1Type, Param2Type, Param3Type, Param4Type, Param5Type) \
	DEFINE_ONLINE_PLAYER_DELEGATE_BASE(MaxPlayers, DelegateName) \
	virtual void Trigger##DelegateName##Delegates(int32 LocalUserNum, Param1Type Param1, Param2Type Param2, Param3Type Param3, Param4Type Param4, Param5Type Param5) \
	{ \
		if (LocalUserNum >= 0 && LocalUserNum < MaxPlayers) \
		{ \
			DelegateName##Delegates[LocalUserNum].Broadcast(LocalUserNum, Param1, Param2, Param3, Param4, Param5); \
		} \
	}

#define DEFINE_ONLINE_PLAYER_DELEGATE_SIX_PARAM(MaxPlayers, DelegateName, Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type) \
	DEFINE_ONLINE_PLAYER_DELEGATE_BASE(MaxPlayers, DelegateName) \
	virtual void Trigger##DelegateName##Delegates(int32 LocalUserNum, Param1Type Param1, Param2Type Param2, Param3Type Param3, Param4Type Param4, Param5Type Param5, Param6Type Param6) \
	{ \
		if (LocalUserNum >= 0 && LocalUserNum < MaxPlayers) \
		{ \
			DelegateName##Delegates[LocalUserNum].Broadcast(LocalUserNum, Param1, Param2, Param3, Param4, Param5, Param6); \
		} \
	}

#define DEFINE_ONLINE_PLAYER_DELEGATE_SEVEN_PARAM(MaxPlayers, DelegateName, Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type, Param7Type) \
	DEFINE_ONLINE_PLAYER_DELEGATE_BASE(MaxPlayers, DelegateName) \
	virtual void Trigger##DelegateName##Delegates(int32 LocalUserNum, Param1Type Param1, Param2Type Param2, Param3Type Param3, Param4Type Param4, Param5Type Param5, Param6Type Param6, Param7Type Param7) \
	{ \
		if (LocalUserNum >= 0 && LocalUserNum < MaxPlayers) \
		{ \
			DelegateName##Delegates[LocalUserNum].Broadcast(LocalUserNum, Param1, Param2, Param3, Param4, Param5, Param6, Param7); \
		} \
	}
