// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

#define UE_API ONLINESUBSYSTEM_API

/**
 * Online subsystem delegates that are more external to the online subsystems themselves
 * This is NOT to replace the individual interfaces that have the Add/Clear/Trigger syntax
 */
class FOnlineSubsystemDelegates
{

public:

	/**
	 * Notification that a new online subsystem instance has been created
	 * 
	 * @param NewSubsystem the new instance created
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnOnlineSubsystemCreated, class IOnlineSubsystem* /*NewSubsystem*/);
	static UE_API FOnOnlineSubsystemCreated OnOnlineSubsystemCreated;
	
	/**
	 * Notification that the default subsystem instance is going to be deleted and reinstantiated
	 * 
	 * @param SubsystemToBeReloaded the instance to be destroyed
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnDefaultOnlineSubsystemPreReloaded, class IOnlineSubsystem* /*SubsystemToBeReloaded*/);
	static UE_API FOnDefaultOnlineSubsystemPreReloaded OnDefaultOnlineSubsystemPreReloaded;
};

#undef UE_API
