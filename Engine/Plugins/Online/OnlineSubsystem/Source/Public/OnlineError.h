// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Internationalization.h" // IWYU pragma: keep

#define UE_API ONLINESUBSYSTEM_API

#define ONLINE_ERROR_LEGACY 1

namespace EOnlineServerConnectionStatus {
	enum Type : uint8;
}

/**
 * Common error results
 */
enum class EOnlineErrorResult : uint8
{
	/** Successful result. no further error processing needed */
	Success,
	
	/** Failed due to no connection */
	NoConnection,
	/** */
	RequestFailure,
	/** */
	InvalidCreds,
	/** Failed due to invalid or missing user */
	InvalidUser,
	/** Failed due to invalid or missing auth for user */
	InvalidAuth,
	/** Failed due to invalid access */
	AccessDenied,
	/** Throttled due to too many requests */
	TooManyRequests,
	/** Async request was already pending */
	AlreadyPending,
	/** Invalid parameters specified for request */
	InvalidParams,
	/** Data could not be parsed for processing */
	CantParse,
	/** Invalid results returned from the request. Parsed but unexpected results */
	InvalidResults,
	/** Incompatible client for backend version */
	IncompatibleVersion,
	/** Not configured correctly for use */
	NotConfigured,
	/** Feature not available on this implementation */
	NotImplemented,
	/** Interface is missing */
	MissingInterface,
	/** Operation was canceled (likely by user) */
	Canceled,
	/** Extended error. More info can be found in the results or by looking at the ErrorCode */
	FailExtended,
	/** No game session found */
	NoGameSession,
	/** Failed due to mismatched or missing user between title and store */
	MismatchedUser,

	/** Default state */
	Unknown
};

#define ONLINE_ERROR_CONTEXT_SEPARATOR TEXT(":")

/** Generic Error response for OSS calls */
struct FOnlineError
{

private:

	/** Ctors. Use ONLINE_ERROR macro instead */
	
	UE_API explicit FOnlineError(EOnlineErrorResult InResult, const FString& InErrorCode, const FText& InErrorMessage);

public:

#if ONLINE_ERROR_LEGACY
	UE_API explicit FOnlineError(const TCHAR* const ErrorCode);
	UE_API explicit FOnlineError(const int32 ErrorCode);
	UE_API void SetFromErrorCode(const int32 ErrorCode);
	UE_API void SetFromErrorMessage(const FText& ErrorMessage, const int32 ErrorCode);
#endif

	UE_API explicit FOnlineError(EOnlineErrorResult InResult = EOnlineErrorResult::Unknown);

	/** Create factory for proper namespacing. Use ONLINE_ERROR macro  */
	static UE_API FOnlineError CreateError(const FString& ErrorNamespace, EOnlineErrorResult Result, const FString& ErrorCode, const FText& ErrorMessage = FText::GetEmpty());
	
	/** Use a default error code / display text */
	static UE_API FOnlineError CreateError(const FString& ErrorNamespace, EOnlineErrorResult Result);

	// helpers for the most common error types
	static UE_API const FOnlineError& Success();

	UE_API explicit FOnlineError(bool bSucceeded);
	UE_API explicit FOnlineError(const FString& ErrorCode);
	UE_API explicit FOnlineError(FString&& ErrorCode);
	UE_API explicit FOnlineError(const FText& ErrorMessage);

	/** Same as the Ctors but can be called any time (does NOT set bSucceeded to false) */
	UE_API void SetFromErrorCode(const FString& ErrorCode);
	UE_API void SetFromErrorCode(FString&& ErrorCode);
	UE_API void SetFromErrorMessage(const FText& ErrorMessage);

	/** Accessors */
	inline EOnlineErrorResult GetErrorResult() const { return Result; }
	inline const FText& GetErrorMessage() const { return ErrorMessage; }
	inline const FString& GetErrorRaw() const { return ErrorRaw; }
	inline const FString& GetErrorCode() const { return ErrorCode; }
	inline bool WasSuccessful() const { return bSucceeded || Result == EOnlineErrorResult::Success; }

	/** Setters for adding the raw error */
	inline FOnlineError& SetErrorRaw(const FString& Val) { ErrorRaw = Val; return *this; }

	/** Code useful when all you have is raw error info from old APIs */
	static UE_API const FString GetGenericErrorCode();

	/** prints out everything, need something like this!!! */
	UE_API FString GetErrorLegacy();
	/** Call this if you want to log this out (will pick the best string representation) */
	UE_API FString ToLogString() const;

	bool operator==(const FOnlineError& Other) const
	{
		return Result == Other.Result && ErrorCode == Other.ErrorCode;
	}

	bool operator!=(const FOnlineError& Other) const
	{
		return !(FOnlineError::operator==(Other));
	}

	FOnlineError operator+ (const FOnlineError& RHS) const
	{
		FOnlineError Copy(*this);
		Copy += RHS;
		return Copy;
	}

	FOnlineError operator+ (const FString& RHS) const
	{
		FOnlineError Copy(*this);
		Copy += RHS;
		return Copy;
	}

  	FOnlineError& operator+= (const FOnlineError& RHS)
  	{
 		ErrorRaw += ONLINE_ERROR_CONTEXT_SEPARATOR + RHS.ErrorRaw;
 		ErrorCode += ONLINE_ERROR_CONTEXT_SEPARATOR + RHS.ErrorCode;
  		return *this;
  	}
 
 	FOnlineError& operator+= (const FString& RHS)
 	{
 		ErrorCode += ONLINE_ERROR_CONTEXT_SEPARATOR + RHS;
 		return *this;
 	}

public:
	/** Did the request succeed fully. If this is true the rest of the struct probably doesn't matter */
	bool bSucceeded;

	/** The raw unparsed error message from server. Used for pass-through error processing by other systems. */
	FString ErrorRaw;

	/** Intended to be interpreted by code. */
	FString ErrorCode;

	/** Suitable for display to end user. Guaranteed to be in the current locale (or empty) */
	FText ErrorMessage;

protected:

	static UE_API FString DefaultErrorCode(EOnlineErrorResult Result);
	/** Default messaging for common errors */
	static UE_API FText DefaultErrorMsg(EOnlineErrorResult Result);
	/** Default namespace for online errors */
	static UE_API const FString& GetDefaultErrorNamespace();

	/** Setters for updating individual values directly */
	inline FOnlineError& SetResult(EOnlineErrorResult Val) { Result = Val; return *this; }
	inline FOnlineError& SetErrorCode(const FString& Val) { ErrorCode = Val; return *this; }
	inline FOnlineError& SetErrorMessage(const FText& Val) { ErrorMessage = Val; return *this; }

	/** Helpers for constructing errors */
	UE_API void SetFromErrorCode(EOnlineErrorResult InResult);
	UE_API void SetFromErrorCode(EOnlineErrorResult InResult, const FString& InErrorCode);
	UE_API void SetFromErrorCode(EOnlineErrorResult InResult, const FString& InErrorCode, const FText& InErrorText);

	/** If successful result then the rest of the struct probably doesn't matter */
	EOnlineErrorResult Result;
};

/** must be defined to a valid namespace for using ONLINE_ERROR factory macro */
#undef ONLINE_ERROR_NAMESPACE
#define ONLINE_ERROR(...) FOnlineError::CreateError(TEXT(ONLINE_ERROR_NAMESPACE), __VA_ARGS__)

#undef UE_API
