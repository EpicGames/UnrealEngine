// Copyright Epic Games, Inc. All Rights Reserved.

// HEADER_UNIT_SKIP - Is included in ways that does not support header units
// IWYU pragma: private

#ifndef ONLINE_ERROR_NAMESPACE
#error "ONLINE_ERROR_NAMESPACE not defined"
#endif

#ifndef LOCTEXT_NAMESPACE
#error "LOCTEXT_NAMESPACE not defined"
#endif

namespace Errors
{
	// Configured namspace
	inline const TCHAR* BaseNamespace() { return TEXT(ONLINE_ERROR_NAMESPACE); }

	// Configuration
	inline FOnlineError NotConfigured() { return ONLINE_ERROR(EOnlineErrorResult::NotConfigured); }

	// Input validation
	inline FOnlineError InvalidUser() { return ONLINE_ERROR(EOnlineErrorResult::InvalidUser); }
	inline FOnlineError InvalidParams() { return ONLINE_ERROR(EOnlineErrorResult::InvalidParams); }

	// System failures
	inline FOnlineError MissingInterface() { return ONLINE_ERROR(EOnlineErrorResult::MissingInterface); }
	inline FOnlineError MissingSubsystem() { return ONLINE_ERROR(EOnlineErrorResult::MissingInterface); }

	// Failures
	inline FOnlineError RequestFailure() { return ONLINE_ERROR(EOnlineErrorResult::RequestFailure); }

	// Response failures
	inline FOnlineError ParseError() { return ONLINE_ERROR(EOnlineErrorResult::CantParse); }
	inline FOnlineError ResultsError() { return ONLINE_ERROR(EOnlineErrorResult::InvalidResults); }
	inline FOnlineError AccessDenied() { return ONLINE_ERROR(EOnlineErrorResult::AccessDenied); }
}
