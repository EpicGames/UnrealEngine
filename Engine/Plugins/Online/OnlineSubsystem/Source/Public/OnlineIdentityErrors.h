// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "OnlineError.h" // IWYU pragma: keep
#define LOCTEXT_NAMESPACE "OnlineIdentity"
#define ONLINE_ERROR_NAMESPACE "errors.com.epicgames.oss.identity"

namespace OnlineIdentity
{
	#include "OnlineErrorMacros.inl"

	namespace Errors
	{
		inline FOnlineError InvalidCreds() { return ONLINE_ERROR(EOnlineErrorResult::InvalidCreds); }
		inline FOnlineError InvalidAuth() { return ONLINE_ERROR(EOnlineErrorResult::InvalidAuth); }

		inline FOnlineError UserNotFound() { return ONLINE_ERROR(EOnlineErrorResult::InvalidUser, TEXT("user_not_found")); }

		inline FOnlineError LoginPending() { return ONLINE_ERROR(EOnlineErrorResult::AlreadyPending, TEXT("login_pending")); }

		inline FOnlineError InvalidResult() { return ONLINE_ERROR(EOnlineErrorResult::InvalidResults, TEXT("invalid_result")); }

		inline FOnlineError PinGrantFailure() { return ONLINE_ERROR(EOnlineErrorResult::FailExtended, TEXT("pin_grant_failure")); }
		inline FOnlineError PinGrantTimeout() { return ONLINE_ERROR(EOnlineErrorResult::FailExtended, TEXT("pin_grant_timeout")); }

		inline FOnlineError Canceled() { return ONLINE_ERROR(EOnlineErrorResult::Canceled); }
	}
}


#undef LOCTEXT_NAMESPACE
#undef ONLINE_ERROR_NAMESPACE


#define UE_ONLINE_ERROR_PARAM_IDENTITY_AUTH_LOGIN TEXT("auth_login")
#define UE_ONLINE_ERROR_PARAM_IDENTITY_AUTH_TYPE TEXT("auth_type")
#define UE_ONLINE_ERROR_PARAM_IDENTITY_AUTH_PASSWORD TEXT("auth_password")

#define UE_ONLINE_ERROR_PARAM_IDENTITY_NO_USER_ID TEXT("no_user_id")
#define UE_ONLINE_ERROR_PARAM_IDENTITY_NO_AUTH_TOKEN TEXT("no_auth_token")
#define UE_ONLINE_ERROR_PARAM_IDENTITY_NO_AUTH_TYPE TEXT("no_auth_type")

