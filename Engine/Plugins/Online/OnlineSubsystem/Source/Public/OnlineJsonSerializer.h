// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"
#include "Serialization/JsonSerializerMacros.h"

UE_DEPRECATED_HEADER(5.7, "OnlineJsonSerializer.h is deprecated, please use the regular JSON functionality in Serialization/JsonSerializerMacros.h");

class FVariantData;

#define BEGIN_ONLINE_JSON_SERIALIZER																BEGIN_JSON_SERIALIZER
#define END_ONLINE_JSON_SERIALIZER																	END_JSON_SERIALIZER
#define ONLINE_JSON_SERIALIZE(JsonName, JsonValue)													JSON_SERIALIZE(JsonName, JsonValue)
#define ONLINE_JSON_SERIALIZE_OPTIONAL(JsonName, OptionalJsonValue)									JSON_SERIALIZE_OPTIONAL(JsonName, OptionalJsonValue)
#define ONLINE_JSON_SERIALIZE_ARRAY(JsonName, JsonArray)											JSON_SERIALIZE_ARRAY(JsonName, JsonArray) 
#define ONLINE_JSON_SERIALIZE_MAP(JsonName, JsonMap)												JSON_SERIALIZE_MAP(JsonName, JsonMap)
#define ONLINE_JSON_SERIALIZE_SIMPLE_COPY(JsonMap)													JSON_SERIALIZE_SIMPLECOPY(JsonMap)
#define ONLINE_JSON_SERIALIZE_SERIALIZABLE(JsonName, JsonValue)	UE_DEPRECATED_MACRO(5.6, "ONLINE_JSON_SERIALIZE_SERIALIZABLE has been deprecated, use JSON_SERIALIZE_MEMBERS_OF instead") JSON_SERIALIZE_MEMBERS_OF(JsonValue) 
#define ONLINE_JSON_SERIALIZE_RAW_JSON_STRING(JsonName, JsonValue)									JSON_SERIALIZE_RAW_JSON_STRING(JsonName, JsonValue)
#define ONLINE_JSON_SERIALIZE_ARRAY_SERIALIZABLE(JsonName, JsonArray, ElementType)					JSON_SERIALIZE_ARRAY_SERIALIZABLE(JsonName, JsonArray, ElementType)
#define ONLINE_JSON_SERIALIZE_OPTIONAL_ARRAY_SERIALIZABLE(JsonName, OptionalJsonArray, ElementType)	JSON_SERIALIZE_OPTIONAL_ARRAY_SERIALIZABLE(JsonName, OptionalJsonArray, ElementType)
#define ONLINE_JSON_SERIALIZE_MAP_SERIALIZABLE(JsonName, JsonMap, ElementType)						JSON_SERIALIZE_MAP_SERIALIZABLE(JsonName, JsonMap, ElementType) 
#define ONLINE_JSON_SERIALIZE_MAP_ARRAY_SERIALIZABLE(JsonName, JsonMap, ElementType)				JSON_SERIALIZE_MAP_ARRAY_SERIALIZABLE(JsonName, JsonMap, ElementType) 
#define ONLINE_JSON_SERIALIZE_OBJECT_SERIALIZABLE(JsonName, JsonSerializableObject)					JSON_SERIALIZE_OBJECT_SERIALIZABLE(JsonName, JsonSerializableObject) 
#define ONLINE_JSON_SERIALIZE_DATETIME_UNIX_TIMESTAMP(JsonName, JsonDateTime)						JSON_SERIALIZE_DATETIME_UNIX_TIMESTAMP(JsonName, JsonDateTime)
#define ONLINE_JSON_SERIALIZE_ENUM(JsonName, JsonEnum)												JSON_SERIALIZE_ENUM(JsonName, JsonEnum)
#define ONLINE_JSON_SERIALIZE_DATETIME_UNIX_TIMESTAMP_MILLISECONDS(JsonName, JsonDateTime)			JSON_SERIALIZE_DATETIME_UNIX_TIMESTAMP_MILLISECONDS(JsonName, JsonDateTime)

typedef FJsonSerializable FOnlineJsonSerializable;
typedef FJsonSerializerWriter<> FOnlineJsonSerializerWriter;

typedef TMap<FString, FVariantData> FJsonSerializeableKeyValueMapVariant;
