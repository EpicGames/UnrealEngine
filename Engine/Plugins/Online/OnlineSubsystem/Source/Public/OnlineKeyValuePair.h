// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/SharedPointer.h"

#define UE_API ONLINESUBSYSTEM_API

class FJsonObject;
class UStruct;

namespace EOnlineKeyValuePairDataType
{
	enum Type : uint8
	{
		/** Means the data in the OnlineData value fields should be ignored */
		Empty,
		/** 32 bit integer */
		Int32,
		/** 32 bit unsigned integer */
		UInt32,
		/** 64 bit integer */
		Int64,
		/** 64 bit unsigned integer */
		UInt64,
		/** Double (8 byte) */
		Double,
		/** Unicode string */
		String,
		/** Float (4 byte) */
		Float,
		/** Binary data */
		Blob,
		/** bool data (1 byte) */
		Bool,
		/** Serialized json text */
		Json,
		MAX
	};

	/** @return the stringified version of the enum passed in */
	inline const TCHAR* ToString(EOnlineKeyValuePairDataType::Type EnumVal)
	{
		switch (EnumVal)
		{
		case Empty:
			return TEXT("Empty");
		case Int32:
			return TEXT("Int32");
		case UInt32:
			return TEXT("UInt32");
		case Int64:
			return TEXT("Int64");
		case UInt64:
			return TEXT("UInt64");
		case Double:
			return TEXT("Double");
		case String:
			return TEXT("String");
		case Float:
			return TEXT("Float");
		case Blob:
			return TEXT("Blob");
		case Bool:
			return TEXT("Bool");
		case Json:
			return TEXT("Json");
		default:
			return TEXT("");
		}		
	}

	/** @return the correspondent enum for the stringified version passed in */
	inline EOnlineKeyValuePairDataType::Type FromString(const FString& EnumStr)
	{
		if (EnumStr.Equals(TEXT("Empty"),ESearchCase::IgnoreCase))
		{
			return Empty;
		}
		else if (EnumStr.Equals(TEXT("Int32"),ESearchCase::IgnoreCase))
		{
			return Int32;
		}
		else if (EnumStr.Equals(TEXT("UInt32"),ESearchCase::IgnoreCase))
		{
			return UInt32;
		}
		else if (EnumStr.Equals(TEXT("Int64"),ESearchCase::IgnoreCase))
		{
			return Int64;
		}
		else if (EnumStr.Equals(TEXT("UInt64"),ESearchCase::IgnoreCase))
		{
			return UInt64;
		}
		else if (EnumStr.Equals(TEXT("Double"),ESearchCase::IgnoreCase))
		{
			return Double;
		}
		else if (EnumStr.Equals(TEXT("String"),ESearchCase::IgnoreCase))
		{
			return String;
		}
		else if (EnumStr.Equals(TEXT("Float"),ESearchCase::IgnoreCase))
		{
			return Float;
		}
		else if (EnumStr.Equals(TEXT("Blob"),ESearchCase::IgnoreCase))
		{
			return Blob;
		}
		else if (EnumStr.Equals(TEXT("Bool"),ESearchCase::IgnoreCase))
		{
			return Bool;
		}
		else if (EnumStr.Equals(TEXT("Json"),ESearchCase::IgnoreCase))
		{
			return Json;
		}
		else
		{
			return MAX;
		}
	}
}

/**
 *	Associative container for key value pairs
 */
template<class KeyType, class ValueType>
class FOnlineKeyValuePairs : public TMap<KeyType,ValueType>
{
	typedef TMap<KeyType, ValueType> Super;

public:

	inline FOnlineKeyValuePairs() {}
	inline FOnlineKeyValuePairs(FOnlineKeyValuePairs&& Other) : Super(MoveTemp(Other)) {}
	inline FOnlineKeyValuePairs(const FOnlineKeyValuePairs&  Other) : Super(Other) {}
	inline FOnlineKeyValuePairs& operator=(FOnlineKeyValuePairs&& Other) { Super::operator=(MoveTemp(Other)); return *this; }
	inline FOnlineKeyValuePairs& operator=(const FOnlineKeyValuePairs&  Other) { Super::operator=(Other); return *this; }

};

/**
 *	Container for storing data of variable type
 */
class FVariantData
{

private:

	/** Current data type */
	EOnlineKeyValuePairDataType::Type Type;
	/** Union of all possible types that can be stored */
	union ValueUnion
	{
		bool AsBool;
		int32 AsInt;
		uint32 AsUInt;
		float AsFloat;
		int64 AsInt64;
		uint64 AsUInt64;
		double AsDouble;
		TCHAR* AsTCHAR;
		struct 
		{ 
			uint8* BlobData;
			uint32 BlobSize;
		} AsBlob;

		ValueUnion() { FMemory::Memset( this, 0, sizeof( ValueUnion ) ); }
	} Value;

	/** Cached estimated maximum size of the value when encoded as an escaped string. */
	mutable int CachedEstimatedMaxEscapedStringSize = 0;

public:

	/** Constructor */
	FVariantData() :
		Type(EOnlineKeyValuePairDataType::Empty)
	{
	}

	/** Constructor starting with an initialized value/type */
	template<typename ValueType>
	FVariantData(const ValueType& InData)
		: Type(EOnlineKeyValuePairDataType::Empty)
	{
		SetValue(InData);
	}

	/**
	 * Copy constructor. Copies the other into this object
	 *
	 * @param Other the other structure to copy
	 */
	UE_API FVariantData(const FVariantData& Other);

	/**
	 * Move constructor. Moves the other into this object
	 *
	 * @param Other the other structure to move
	 */
	UE_API FVariantData(FVariantData&& Other);

	/**
	 * Assignment operator. Copies the other into this object
	 *
	 * @param Other the other structure to copy
	 */
	UE_API FVariantData& operator=(const FVariantData& Other);

	/**
	 * Move Assignment operator. Moves the other into this object
	 *
	 * @param Other the other structure to move
	 */
	UE_API FVariantData& operator=(FVariantData&& Other);

	/**
	 * Cleans up the data to prevent leaks
	 */
	~FVariantData()
	{
		Empty();
	}

	/**
	 * Cleans up the existing data and sets the type to ODT_Empty
	 */
	UE_API void Empty();

	/**
	 *	Get the key for this key value pair
	 */
	const EOnlineKeyValuePairDataType::Type GetType() const
	{
		return Type;
	}

	/**
	 * Copies the data and sets the type
	 *
	 * @param InData the new data to assign
	 */
	UE_API void SetValue(const FString& InData);

	/**
	 * Copies the data and sets the type
	 *
	 * @param InData the new data to assign
	 */
	UE_API void SetValue(const TCHAR* InData);

	/**
	 * Copies the data and sets the type
	 *
	 * @param InData the new data to assign
	 */
	UE_API void SetValue(int32 InData);

	/**
	 * Copies the data and sets the type
	 *
	 * @param InData the new data to assign
	 */
	UE_API void SetValue(uint32 InData);

	/**
	 * Copies the data and sets the type
	 *
	 * @param InData the new data to assign
	 */
	UE_API void SetValue(bool InData);

	/**
	 * Copies the data and sets the type
	 *
	 * @param InData the new data to assign
	 */
	UE_API void SetValue(double InData);

	/**
	 * Copies the data and sets the type
	 *
	 * @param InData the new data to assign
	 */
	UE_API void SetValue(float InData);

	/**
	 * Copies the data and sets the type
	 *
	 * @param InData the new data to assign
	 */
	UE_API void SetValue(const TArray<uint8>& InData);

	/**
	 * Copies the data and sets the type
	 *
	 * @param Size the length of the buffer to copy
	 * @param InData the new data to assign
	 */
	UE_API void SetValue(uint32 Size, const uint8* InData);

	/**
	 * Copies the data and sets the type
	 *
	 * @param InData the new data to assign
	 */
	UE_API void SetValue(int64 InData);

	/**
	 * Copies the data and sets the type
	 *
	 * @param InData the new data to assign
	 */
	UE_API void SetValue(uint64 InData);

	/**
	* Copies the data and sets the type
	*
	* @param InData the new data to assign
	*/
	UE_API void SetValue(const TSharedRef<class FJsonObject>& InData);

	/**
	* Copies the data and sets the type
	*
	* @param InData the new data to assign
	*/
	UE_API void SetJsonValueFromString(const FString& InData);

	/**
	 * Copies the data after verifying the type
	 *
	 * @param OutData out value that receives the copied data
	 */
	UE_API void GetValue(FString& OutData) const;

	/**
	 * Copies the data after verifying the type
	 *
	 * @param OutData out value that receives the copied data
	 */
	UE_API void GetValue(int32& OutData) const;

	/**
	 * Copies the data after verifying the type
	 *
	 * @param OutData out value that receives the copied data
	 */
	UE_API void GetValue(uint32& OutData) const;

	/**
	 * Copies the data after verifying the type
	 *
	 * @param OutData out value that receives the copied data
	 */
	UE_API void GetValue(bool& OutData) const;

	/**
	  Copies the data after verifying the type
	 *
	 * @param OutData out value that receives the copied data
	 */
	UE_API void GetValue(int64& OutData) const;

	/**
	  Copies the data after verifying the type
	 *
	 * @param OutData out value that receives the copied data
	 */
	UE_API void GetValue(uint64& OutData) const;

	/**
	 * Copies the data after verifying the type
	 *
	 * @param OutData out value that receives the copied data
	 */
	UE_API void GetValue(float& OutData) const;

	/**
	 * Copies the data after verifying the type
	 *
	 * @param OutData out value that receives the copied data
	 */
	UE_API void GetValue(TArray<uint8>& OutData) const;

	/**
	 * Copies the data after verifying the type.
	 * NOTE: Performs a deep copy so you are responsible for freeing the data
	 *
	 * @param OutSize out value that receives the size of the copied data
	 * @param OutData out value that receives the copied data
	 */
	UE_API void GetValue(uint32& OutSize,uint8** OutData) const;

	/**
	 * Copies the data after verifying the type
	 *
	 * @param OutData out value that receives the copied data
	 */
	UE_API void GetValue(double& OutData) const;

	/**
	* Copies the data after verifying the type
	*
	* @param OutData out value that receives the copied data
	*/
	UE_API void GetValue(TSharedPtr<class FJsonObject>& OutData) const;

	/**
	* Copies the data after verifying the type
	*
	* @param OutData out value that receives the copied data
	*/
	UE_API void GetValue(TArray<TSharedPtr<class FJsonValue>>& OutData) const;

	/** Returns the estimated maximum size of the value when encoded as an escaped string. */
	UE_API int GetEstimatedMaxEscapedStringSize() const;

	/**
	 * Returns true if Type is numeric
	 */
	bool IsNumeric() const
	{
		return Type == EOnlineKeyValuePairDataType::Int32 || Type == EOnlineKeyValuePairDataType::Int64 ||
				  Type == EOnlineKeyValuePairDataType::UInt32 || Type == EOnlineKeyValuePairDataType::UInt64 ||
				  Type == EOnlineKeyValuePairDataType::Float || Type == EOnlineKeyValuePairDataType::Double;
	}

	/**
	 * Increments the value by the specified amount
	 * 
	 * @param IncBy the amount to increment by
	 */
	template<typename TYPE, EOnlineKeyValuePairDataType::Type ENUM_TYPE>
	inline void Increment(TYPE IncBy)
	{
		checkSlow(IsNumeric());
		if (Type == ENUM_TYPE)
		{
			*(TYPE*)&Value += IncBy;
		}
	}

	/**
	 * Increments the numeric value by 1
	 */
	UE_API void Increment();

	/**
	 * Decrements the value by the specified amount
	 *
	 * @param DecBy the amount to decrement by
	 */
	template<typename TYPE, EOnlineKeyValuePairDataType::Type ENUM_TYPE>
	inline void Decrement(TYPE DecBy)
	{
		checkSlow(IsNumeric());
		if (Type == ENUM_TYPE)
		{
			*(TYPE*)&Value -= DecBy;
		}
	}

	/**
	 * Decrements the numeric value by 1
	 */
	UE_API void Decrement();

	/**
	 * Converts the data into a string representation
	 */
	UE_API FString ToString() const;

	/**
	 * Converts the string to the specified type of data for this setting
	 *
	 * @param NewValue the string value to convert
	 *
	 * @return true if it was converted, false otherwise
	 */
	UE_API bool FromString(const FString& NewValue);

	/** @return The type as a string */
	const TCHAR* GetTypeString() const
	{
		return EOnlineKeyValuePairDataType::ToString(Type);
	}

	/**
	 * Convert variant data to json object with "type,value" fields
	 *
	 * @return json object representation
	 */
	UE_API TSharedRef<class FJsonObject> ToJson() const;

	/**
	 * Convert json object to variant data from "type,value" fields
	 *
	 * @param JsonObject json to convert from
	 * @return true if conversion was successful
	 */
	UE_API bool FromJson(const TSharedRef<class FJsonObject>& JsonObject);

	/**
	 * Combine the type suffix given this variant's type
	 *
	 * @return the the type suffix
	 */
	UE_API FString GetTypeSuffix() const;
	
	/**
	* Insert variant data into json object
	*
	* @param JsonObject json object to insert data into
	* @param Name field name for value
	* @param bWithTypeSuffix True if you would like the type suffix appended to the name
	*/
	UE_API void AddToJsonObject(const TSharedRef<FJsonObject>& JsonObject, const FString& Name, const bool bWithTypeSuffix = true) const;

	/**
	* Convert json object to variant data from Name_
	*
	* @param Name name of json field that includes the type suffix
	* @param JsonValue json value to convert from
	* @param OutName returns the name with the type suffix stripped
	* @return true if conversion was successful
	*/
	UE_API bool FromJsonValue(FStringView Name, const TSharedRef<class FJsonValue>& JsonValue, FString& OutName);

	/**
	 * Comparison of two settings data classes
	 *
	 * @param Other the other settings data to compare against
	 *
	 * @return true if they are equal, false otherwise
	 */
	UE_API bool operator==(const FVariantData& Other) const;
	UE_API bool operator!=(const FVariantData& Other) const;
};

/**
 * Helper class for converting from UStruct to FVariantData and back
 * only very basic flat UStructs with POD types are supported
 */
class FVariantDataConverter
{
public:
	/**
	 * Convert a UStruct into a variant mapping table
	 *
	 * @param StructDefinition layout of the UStruct
	 * @param Struct actual UStruct data
	 * @param OutVariantMap container for outgoing data
	 * @param CheckFlags property must have this flag to be serialized
	 * @param SkipFlags property cannot have this flag to be serialized
	 * 
	 * @return true if it was successful, false otherwise
	 */
	static UE_API bool UStructToVariantMap(const UStruct* StructDefinition, const void* Struct, FOnlineKeyValuePairs<FString, FVariantData>& OutVariantMap, int64 CheckFlags, int64 SkipFlags);

	/**
	 * Convert a single FProperty to an FVariantData
	 *
	 * @param Property definition of the property
	 * @param Value actual property data
	 * @param CheckFlags property must have this flag to be serialized
	 * @param SkipFlags property cannot have this flag to be serialized
	 * @param OutVariantData container for outgoing data
	 * 
	 * @return true if it was successful, false otherwise
	 */
	static UE_API bool FPropertyToVariantData(FProperty* Property, const void* Value, int64 CheckFlags, int64 SkipFlags, FVariantData& OutVariantData);

public:

	/**
	 * Convert a map of FVariantData elements to a UStruct
	 *
	 * @param VariantMap Input variant data
	 * @param StructDefinition layout of the UStruct
	 * @param OutStruct output container for UStruct data
	 * @param CheckFlags property must have this flag to be serialized
	 * @param SkipFlags property cannot have this flag to be serialized
	 * 
	 * @return true if it was successful, false otherwise
	 */
	static UE_API bool VariantMapToUStruct(const FOnlineKeyValuePairs<FString, FVariantData>& VariantMap, const UStruct* StructDefinition, void* OutStruct, int64 CheckFlags, int64 SkipFlags);

	/**
	 * Apply FVariantData elements to properties of a UStruct that are identical with the previous value.
	 *
	 * @param VariantMap Input variant data
	 * @param StructDefinition layout of the UStruct
	 * @param OutStruct output container for UStruct data
	 * @param CompareStruct the struct to compare OutStruct to when deciding if a property should be updated.
	 * @param CheckFlags property must have this flag to be serialized
	 * @param SkipFlags property cannot have this flag to be serialized
	 * 
	 * @return true if it was successful, false otherwise
	 */
	static UE_API bool ApplyVariantMapToUnchangedProperties(const FOnlineKeyValuePairs<FString, FVariantData>& VariantMap, const UStruct* StructDefinition, void* OutStruct, const void* CompareStruct, int64 CheckFlags, int64 SkipFlags);

	/**
	 * Convert an FVariantData to a FProperty
	 *
	 * @param Variant Input variant data
	 * @param Property definition of the property
	 * @param OutValue outgoing property data container
	 * @param CheckFlags property must have this flag to be serialized
	 * @param SkipFlags property cannot have this flag to be serialized
	 * 
	 * @return true if it was successful, false otherwise
	 */
	static UE_API bool VariantDataToFProperty(const FVariantData* Variant, FProperty* Property, void* OutValue, int64 CheckFlags, int64 SkipFlags);
	
private:

	/**
	 * Convert a single FProperty to an FVariantData
	 *
	 * @param Property definition of the property
	 * @param Value actual property data
	 * @param OutVariantData container for outgoing data
	 * @param CheckFlags property must have this flag to be serialized
	 * @param SkipFlags property cannot have this flag to be serialized
	 * 
	 * @return true if it was successful, false otherwise
	 */
	static bool ConvertScalarFPropertyToVariant(FProperty* Property, const void* Value, FVariantData& OutVariantData, int64 CheckFlags, int64 SkipFlags);

	/**
	 * Convert an FVariantData to a FProperty
	 *
	 * @param Variant Input variant data
	 * @param Property definition of the property
	 * @param OutValue outgoing property data container
	 * @param CheckFlags property must have this flag to be serialized
	 * @param SkipFlags property cannot have this flag to be serialized
	 * 
	 * @return true if it was successful, false otherwise
	 */
	static bool ConvertScalarVariantToFProperty(const FVariantData* Variant, FProperty* Property, void* OutValue, int64 CheckFlags, int64 SkipFlags);
};

#undef UE_API
