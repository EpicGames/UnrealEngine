// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CoreOnlineFwd.h"

#define UE_API ONLINESUBSYSTEM_API

class FJsonObject;
class UStruct;

class FJsonValue;

/** Notification object, used to send messages between systems */
struct FOnlineNotification
{
	/**
	 * Default constructor
	 */
	FOnlineNotification()
	{
	}

	/**
	 * Constructor from type and FJsonValue
	 * System message unless ToUserId is specified; FromUserId optional
	 */
	UE_API FOnlineNotification(
		const FString& InTypeStr,
		const TSharedPtr<FJsonValue>& InPayload,
		FUniqueNetIdPtr InToUserId = nullptr,
		FUniqueNetIdPtr InFromUserId = nullptr,
		const FString & InClientRequestIdStr = TEXT("")
	);

	/**
	 * Constructor from type and FJsonObject
	 * System message unless ToUserId is specified; FromUserId optional
	 */
	FOnlineNotification(
		const FString& InTypeStr,
		const TSharedPtr<FJsonObject>& InPayload,
		FUniqueNetIdPtr InToUserId = nullptr,
		FUniqueNetIdPtr InFromUserId = nullptr,
		const FString& InClientRequestIdStr = TEXT("")
	)
	: TypeStr(InTypeStr)
	, Payload(InPayload)
	, ToUserId(InToUserId)
	, FromUserId(InFromUserId)
	, ClientRequestIdStr(InClientRequestIdStr)
	{
	}


	/**
	 * Parse a payload and assume there is a static const TypeStr member to use
	 */
	template <class FStruct>
	bool ParsePayload(FStruct& PayloadOut) const
	{
		return ParsePayload(FStruct::StaticStruct(), &PayloadOut);
	}

	/**
	 * Parse out Payload into the provided UStruct
	 */
	UE_API bool ParsePayload(UStruct* StructType, void* StructPtr) const;

	/**
	 * Does this notification have a valid payload?
	 */
	explicit operator bool() const
	{
		return Payload.IsValid();
	}

	/**
	 * Set up the type string for the case where the type is embedded in the payload
	 */
	UE_API void SetTypeFromPayload();

	/**
	 * Set up the ClientRequestIdStr for the case where it is embedded in the payload
	 */
	UE_API void SetClientRequestIdFromPayload();

	/** A string defining the type of this notification, used to determine how to parse the payload */
	FString TypeStr;

	/** The payload of this notification */
	TSharedPtr<FJsonObject> Payload;

	/** User to deliver the notification to.  Can be null for system notifications. */
	FUniqueNetIdPtr ToUserId;

	/** User who sent the notification, optional. */
	FUniqueNetIdPtr FromUserId;

	/** String representing the client_request_id for this notification. Used to tie a server request back to the client. Can be empty. */
	FString ClientRequestIdStr;
};

#undef UE_API
