// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

#define UE_API ONLINESUBSYSTEM_API

class FUniqueNetId;
struct FOnlineNotification;

/** Whether a handler function handled a particular notification */
enum class EOnlineNotificationResult
{
	None,		// No handling occurred
	Handled,	// Notification was handled
};

/**
* Delegate type for handling a notification
*
* The first parameter is a notification structure
* Return result code to indicate if notification has been handled
*/
DECLARE_DELEGATE_RetVal_OneParam(EOnlineNotificationResult, FHandleOnlineNotificationSignature, const FOnlineNotification&);


/** Struct to keep track of bindings */
struct FOnlineNotificationBinding
{
	/** Delegate to call when this binding is activated */
	FHandleOnlineNotificationSignature NotificationDelegate;

	FOnlineNotificationBinding()
	{}

	FOnlineNotificationBinding(const FHandleOnlineNotificationSignature& InNotificationDelegate)
		: NotificationDelegate(InNotificationDelegate)
	{}
};

/** This class is a static manager used to track notification transports and map the delivered notifications to subscribed notification handlers */
class FOnlineNotificationHandler
{

protected:

	typedef TMap< FString, TArray<FOnlineNotificationBinding> > NotificationTypeBindingsMap;

	/** Map from type of notification to the delegate to call */
	NotificationTypeBindingsMap SystemBindingMap;

	/** Map from player and type of notification to the delegate to call */
	TMap< FString, NotificationTypeBindingsMap > PlayerBindingMap;

public:

	/** Lifecycle is managed by OnlineSubSystem, all access should be made through there */
	FOnlineNotificationHandler()
	{
	}

	// SYSTEM NOTIFICATION BINDINGS

	/** Add a notification binding for a type */
	UE_API FDelegateHandle AddSystemNotificationBinding_Handle(FString NotificationType, const FOnlineNotificationBinding& NewBinding);

	/** Remove the notification handler for a type */
	UE_API void RemoveSystemNotificationBinding(FString NotificationType, FDelegateHandle RemoveHandle);

	/** Resets all system notification handlers */
	UE_API void ResetSystemNotificationBindings();

	// PLAYER NOTIFICATION BINDINGS

	/** Add a notification binding for a type */
	UE_API FDelegateHandle AddPlayerNotificationBinding_Handle(const FUniqueNetId& PlayerId, FString NotificationType, const FOnlineNotificationBinding& NewBinding);

	/** Remove the player notification handler for a type */
	UE_API void RemovePlayerNotificationBinding(const FUniqueNetId& PlayerId, FString NotificationType, FDelegateHandle RemoveHandle);

	/** Resets a player's notification handlers */
	UE_API void ResetPlayerNotificationBindings(const FUniqueNetId& PlayerId);

	/** Resets all player notification handlers */
	UE_API void ResetAllPlayerNotificationBindings();

	// RECEIVING NOTIFICATIONS

	/** Deliver a notification to the appropriate handler for that player/msg type.  Called by NotificationTransport implementations. */
	UE_API void DeliverNotification(const FOnlineNotification& Notification);
};

typedef TSharedPtr<FOnlineNotificationHandler, ESPMode::ThreadSafe> FOnlineNotificationHandlerPtr;
typedef TSharedRef<FOnlineNotificationHandler, ESPMode::ThreadSafe> FOnlineNotificationHandlerRef;
typedef TWeakPtr<FOnlineNotificationHandler, ESPMode::ThreadSafe> FOnlineNotificationHandlerWeakPtr;

#undef UE_API
