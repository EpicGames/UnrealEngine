// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineNotificationTransportInterface.h"

#define UE_API ONLINESUBSYSTEM_API

struct FOnlineNotification;

//forward declare
typedef TSharedPtr<class IOnlineNotificationTransport, ESPMode::ThreadSafe> IOnlineNotificationTransportPtr;
class IOnlineNotificationTransportMessage;
struct FOnlineNotification;


/** This class is a static manager used to track notification transports and map the delivered notifications to subscribed notification handlers */
class FOnlineNotificationTransportManager
{

protected:

	/** Map from a transport type to the transport object */
	TMap< FNotificationTransportId, IOnlineNotificationTransportPtr > TransportMap;

public:

	/** Lifecycle is managed by OnlineSubSystem, all access should be through there */
	FOnlineNotificationTransportManager()
	{
	}

	UE_API virtual ~FOnlineNotificationTransportManager();
	
	/** Send a notification using a specific transport */
	UE_API bool SendNotification(FNotificationTransportId TransportType, const FOnlineNotification& Notification);

	/** Receive a message from a specific transport, convert to notification, and pass on for delivery */
	UE_API bool ReceiveTransportMessage(FNotificationTransportId TransportType, const IOnlineNotificationTransportMessage& TransportMessage);

	// NOTIFICATION TRANSPORTS

	/** Get a notification transport of a specific type */
	UE_API IOnlineNotificationTransportPtr GetNotificationTransport(FNotificationTransportId TransportType);

	/** Add a notification transport */
	UE_API void AddNotificationTransport(IOnlineNotificationTransportPtr Transport);

	/** Remove a notification transport */
	UE_API void RemoveNotificationTransport(FNotificationTransportId TransportType);

	/** Resets all transports */
	UE_API void ResetNotificationTransports();

	/** Base function for letting the notifications flow */
	virtual FOnlineTransportTapHandle OpenTap(const FUniqueNetId& User, const FOnlineTransportTap& Tap)
	{
		return FOnlineTransportTapHandle();
	}

	/** Base function for stanching the notifications */
	virtual void CloseTap(FOnlineTransportTapHandle TapHandle)
	{
	}
};

typedef TSharedPtr<FOnlineNotificationTransportManager, ESPMode::ThreadSafe> FOnlineNotificationTransportManagerPtr;

#undef UE_API
