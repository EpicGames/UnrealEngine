// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineError.h" // IWYU pragma: keep
#define LOCTEXT_NAMESPACE "OnlinePresence"
#define ONLINE_ERROR_NAMESPACE "errors.com.epicgames.oss.presence"

namespace OnlinePresence
{
	#include "OnlineErrorMacros.inl"

	namespace Errors
	{
		inline FOnlineError InvalidResult() { return ONLINE_ERROR(EOnlineErrorResult::InvalidResults, TEXT("invalid_result")); }
	}
}

#undef LOCTEXT_NAMESPACE
#undef ONLINE_ERROR_NAMESPACE




