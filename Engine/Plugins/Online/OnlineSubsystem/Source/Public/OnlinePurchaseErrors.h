// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineError.h" // IWYU pragma: keep
#define LOCTEXT_NAMESPACE "OnlinePurchase"
#define ONLINE_ERROR_NAMESPACE "errors.com.epicgames.oss.purchase"

namespace OnlinePurchase
{
	#include "OnlineErrorMacros.inl"

	namespace Errors
	{
		inline FOnlineError InvalidResult() { return ONLINE_ERROR(EOnlineErrorResult::InvalidResults, TEXT("invalid_result")); }
	}
}


#undef LOCTEXT_NAMESPACE
#undef ONLINE_ERROR_NAMESPACE




