// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineKeyValuePair.h"
#include "OnlineSubsystem.h"

#define UE_API ONLINESUBSYSTEM_API

// NOTE:  Session setting names have been moved to OnlineBase in OnlineSessionNames.h

#define INVALID_SESSION_SETTING_ID -1

/**
 *	One setting describing an online session
 * contains a key, value and how this setting is advertised to others, if at all
 */
struct FOnlineSessionSetting
{
public:
	/** Settings value */
	FVariantData Data;
	/** How is this session setting advertised with the backend or searches */
	EOnlineDataAdvertisementType::Type AdvertisementType;
	/** Optional ID used in some platforms as the index instead of the session name */
	int32 ID;

public:
	/** Default constructor, used when serializing a network packet */
	FOnlineSessionSetting()
		: AdvertisementType(EOnlineDataAdvertisementType::DontAdvertise)
		, ID(INVALID_SESSION_SETTING_ID)
	{
	}

	/** Constructor for settings created/defined on the host for a session */
	template<typename Type>
	FOnlineSessionSetting(Type&& InData)
		: Data(Forward<Type>(InData))
		, AdvertisementType(EOnlineDataAdvertisementType::DontAdvertise)
		, ID(INVALID_SESSION_SETTING_ID)
	{
	}

	/** Constructor for settings created/defined on the host for a session */
	template<typename Type>
	FOnlineSessionSetting(Type&& InData, EOnlineDataAdvertisementType::Type InAdvertisementType)
		: Data(Forward<Type>(InData))
		, AdvertisementType(InAdvertisementType)
		, ID(INVALID_SESSION_SETTING_ID)
	{
	}

	/** Constructor for settings created/defined on the host for a session */
	template<typename Type>
	FOnlineSessionSetting(Type&& InData, EOnlineDataAdvertisementType::Type InAdvertisementType, int32 InID)
		: Data(Forward<Type>(InData))
		, AdvertisementType(InAdvertisementType)
		, ID(InID)
	{
	}

	/**
	*	Comparison operator
	*/
	bool operator==(const FOnlineSessionSetting& Other) const
	{
		// Advertisement type not compared because it is not passed to clients
		return Data == Other.Data;
	}

	bool operator!=(const FOnlineSessionSetting& Other) const
	{
		return !operator==(Other);
	}

	FString ToString() const
	{
		if (ID == INVALID_SESSION_SETTING_ID)
		{
			return FString::Printf(TEXT("%s : %s"), *Data.ToString(), EOnlineDataAdvertisementType::ToString(AdvertisementType));
		}
		else
		{
			return FString::Printf(TEXT("%s : %s : %d"), *Data.ToString(), EOnlineDataAdvertisementType::ToString(AdvertisementType), ID);
		}
	}
};

/** Type defining an array of session settings accessible by key */
typedef FOnlineKeyValuePairs<FName, FOnlineSessionSetting> FSessionSettings;

/**
 *	One search parameter in an online session query
 *  contains a value and how this setting is compared
 */
class FOnlineSessionSearchParam
{
private:
	/** Hidden on purpose */
	FOnlineSessionSearchParam() = delete;

public:
	/** Search value */
	FVariantData Data;
	/** How is this session setting compared on the backend searches */
	EOnlineComparisonOp::Type ComparisonOp;
	/** Optional ID used on some platform to index the session setting */
	int32 ID;

public:
	/** Constructor for setting search parameters in a query */
	template<typename Type>
	FOnlineSessionSearchParam(Type&& InData)
		: Data(Forward<Type>(InData))
		, ComparisonOp(EOnlineComparisonOp::Equals)
		, ID(INVALID_SESSION_SETTING_ID)
	{
	}

	/** Constructor for setting search parameters in a query */
	template<typename Type>
	FOnlineSessionSearchParam(Type&& InData, EOnlineComparisonOp::Type InComparisonOp)
		: Data(Forward<Type>(InData))
		, ComparisonOp(InComparisonOp)
		, ID(INVALID_SESSION_SETTING_ID)
	{
	}

	/** Constructor for setting search parameters in a query */
	template<typename Type>
	FOnlineSessionSearchParam(Type&& InData, EOnlineComparisonOp::Type InComparisonOp, int32 InID)
		: Data(Forward<Type>(InData))
		, ComparisonOp(InComparisonOp)
		, ID(InID)
	{
	}

	/**
	 *	Comparison operator
	 */
	bool operator==(const FOnlineSessionSearchParam& Other) const
	{
		// Don't compare ComparisonOp so we don't get the same data with different ops
		return Data == Other.Data;
	}

	bool operator!=(const FOnlineSessionSearchParam& Other) const
	{
		return !operator==(Other);
	}

	FString ToString() const
	{
		return FString::Printf(TEXT("Value=%s : %s : %d"), *Data.ToString(), EOnlineComparisonOp::ToString(ComparisonOp), ID);
	}
};

/** Type defining an array of search parameters accessible by key */
typedef FOnlineKeyValuePairs<FName, FOnlineSessionSearchParam> FSearchParams;

/**
 *	Container for all parameters describing a single session search
 */
class FOnlineSearchSettings
{
public:
	/** Array of custom search settings */
	FSearchParams SearchParams;

public:
	FOnlineSearchSettings() = default;

	FOnlineSearchSettings(FOnlineSearchSettings&&) = default;
	FOnlineSearchSettings(const FOnlineSearchSettings&) = default;
	FOnlineSearchSettings& operator=(FOnlineSearchSettings&&) = default;
	FOnlineSearchSettings& operator=(const FOnlineSearchSettings&) = default;

	virtual ~FOnlineSearchSettings() = default;

	/**
	 *	Sets a key value pair combination that defines a search parameter
	 *
	 * @param Key key for the setting
	 * @param Value value of the setting
	 * @param InType type of comparison
	 * @param ID ID of comparison
	 */
	template<typename ValueType>
	ONLINESUBSYSTEM_API void Set(FName Key, const ValueType& Value, EOnlineComparisonOp::Type InType, int32 ID);

	/**
	 *	Sets a key value pair combination that defines a search parameter
	 *
	 * @param Key key for the setting
	 * @param Value value of the setting
	 * @param InType type of comparison
	 */
	template<typename ValueType>
	ONLINESUBSYSTEM_API void Set(FName Key, const ValueType& Value, EOnlineComparisonOp::Type InType);

	/**
	 *	Gets a key value pair combination that defines a search parameter
	 *
	 * @param Key key for the setting
	 * @param Value value of the setting
	 *
	 * @return true if found, false otherwise
	 */
	template<typename ValueType>
	ONLINESUBSYSTEM_API bool Get(FName Key, ValueType& Value) const;

	/**
	 * Retrieve a search parameter comparison op
	 *
	 * @param Key key of the setting
	 *
	 * @return the comparison op for the setting
	 */
	UE_API EOnlineComparisonOp::Type GetComparisonOp(FName Key) const;
};

/**
 *	Container for all settings describing a single online session
 */
class FOnlineSessionSettings
{
public:
	/** The number of publicly available connections advertised */
	int32 NumPublicConnections;
	/** The number of connections that are private (invite/password) only */
	int32 NumPrivateConnections;
	/** Whether this match is publicly advertised on the online service */
	bool bShouldAdvertise;
	/** Whether joining in progress is allowed or not */
	bool bAllowJoinInProgress;
	/** This game will be lan only and not be visible to external players */
	bool bIsLANMatch;
	/** Whether the server is dedicated or player hosted */
	bool bIsDedicated;
	/** Whether the match should gather stats or not */
	bool bUsesStats;
	/** Whether the match allows invitations for this session or not */
	bool bAllowInvites;
	/** Whether to display user presence information or not */
	bool bUsesPresence;
	/** Whether joining via player presence is allowed or not */
	bool bAllowJoinViaPresence;
	/** Whether joining via player presence is allowed for friends only or not */
	bool bAllowJoinViaPresenceFriendsOnly;
	/** Whether the server employs anti-cheat (punkbuster, vac, etc) */
	bool bAntiCheatProtected;
	/** Whether to prefer lobbies APIs if the platform supports them */
	bool bUseLobbiesIfAvailable;
	/** Whether to create (and auto join) a voice chat room for the lobby, if the platform supports it */
	bool bUseLobbiesVoiceChatIfAvailable;
	/** Manual override for the Session Id instead of having one assigned by the backend. Its size may be restricted depending on the platform */
	FString SessionIdOverride;

	/** Used to keep different builds from seeing each other during searches */
	int32 BuildUniqueId;
	/** Array of custom session settings */
	FSessionSettings Settings;

	/** Map of custom settings per session member (Not currently used by every OSS) */
	TUniqueNetIdMap<FSessionSettings> MemberSettings;

public:
	/** Default constructor, used when serializing a network packet */
	FOnlineSessionSettings()
		: NumPublicConnections(0)
		, NumPrivateConnections(0)
		, bShouldAdvertise(false)
		, bAllowJoinInProgress(false)
		, bIsLANMatch(false)
		, bIsDedicated(false)
		, bUsesStats(false)
		, bAllowInvites(false)
		, bUsesPresence(false)
		, bAllowJoinViaPresence(false)
		, bAllowJoinViaPresenceFriendsOnly(false)
		, bAntiCheatProtected(false)
		, bUseLobbiesIfAvailable(false)
		, bUseLobbiesVoiceChatIfAvailable(false)
		, BuildUniqueId(GetBuildUniqueId())
	{
		// Example usage of settings
//		Set(SETTING_MAPNAME, FString(TEXT("")), EOnlineDataAdvertisementType::ViaOnlineService);
//		Set(SETTING_NUMBOTS, 0, EOnlineDataAdvertisementType::ViaOnlineService);
//		Set(SETTING_GAMEMODE, FString(TEXT("")), EOnlineDataAdvertisementType::ViaOnlineService);
	}

	virtual ~FOnlineSessionSettings() = default;

	FOnlineSessionSettings(FOnlineSessionSettings&&) = default;
	FOnlineSessionSettings(const FOnlineSessionSettings&) = default;
	FOnlineSessionSettings& operator=(FOnlineSessionSettings&&) = default;
	FOnlineSessionSettings& operator=(const FOnlineSessionSettings&) = default;

	/**
	 *	Sets a key value pair combination that defines a session setting with an ID
	 *
	 * @param Key key for the setting
	 * @param Value value of the setting
	 * @param InType type of online advertisement
	 * @param ID ID for this session setting
	 */
	template<typename ValueType>
	ONLINESUBSYSTEM_API void Set(FName Key, const ValueType& Value, EOnlineDataAdvertisementType::Type InType, int32 InID);

	/**
	 *	Sets a key value pair combination that defines a session setting
	 *
	 * @param Key key for the setting
	 * @param Value value of the setting
	 * @param InType type of online advertisement
	 */
	template<typename ValueType>
	ONLINESUBSYSTEM_API void Set(FName Key, const ValueType& Value, EOnlineDataAdvertisementType::Type InType);

	/**
	 *	Sets a key value pair combination that defines a session setting
	 * from an existing session setting
	 *
	 * @param Key key for the setting
	 * @param SrcSetting setting values
	 */
	ONLINESUBSYSTEM_API void Set(FName Key, const FOnlineSessionSetting& SrcSetting);

	/**
	 *	Gets a key value pair combination that defines a session setting
	 *
	 * @param Key key for the setting
	 * @param Value value of the setting
	 *
	 * @return true if found, false otherwise
	 */
	template<typename ValueType>
	ONLINESUBSYSTEM_API bool Get(FName Key, ValueType& Value) const;

	/**
	 *  Removes a key value pair combination
	 *
	 * @param Key key to remove
	 *
	 * @return true if found and removed, false otherwise
	 */
	ONLINESUBSYSTEM_API bool Remove(FName Key);

	/**
	 * Retrieve a session setting's advertisement type
	 *
	 * @param Key key of the setting
	 *
	 * @return the advertisement type for the setting
	 */
	ONLINESUBSYSTEM_API EOnlineDataAdvertisementType::Type GetAdvertisementType(FName Key) const;

	/**
	* Retrieve a session setting's ID
	*
	* @param Key key of the setting
	*
	* @return the ID for the setting
	*/
	ONLINESUBSYSTEM_API int32 GetID(FName Key) const;

};

/** Basic session information serializable into a NamedSession or SearchResults */
class FOnlineSession
{
public:
	/** Owner of the session */
	FUniqueNetIdPtr OwningUserId;
	/** Owner name of the session */
	FString OwningUserName;
	/** The settings associated with this session */
	FOnlineSessionSettings SessionSettings;
	/** The platform specific session information */
	TSharedPtr<class FOnlineSessionInfo> SessionInfo;
	/** The number of private connections that are available (read only) */
	int32 NumOpenPrivateConnections;
	/** The number of publicly available connections that are available (read only) */
	int32 NumOpenPublicConnections;

public:
	/** Default constructor, used when serializing a network packet */
	FOnlineSession() :
		OwningUserId(nullptr),
		SessionInfo(nullptr),
		NumOpenPrivateConnections(0),
		NumOpenPublicConnections(0)
	{
	}

	/** Constructor */
	FOnlineSession(const FOnlineSessionSettings& InSessionSettings) :
		OwningUserId(nullptr),
		SessionSettings(InSessionSettings),
		SessionInfo(nullptr),
		NumOpenPrivateConnections(0),
		NumOpenPublicConnections(0)
	{
	}

	FOnlineSession(FOnlineSession&&) = default;
	FOnlineSession(const FOnlineSession&) = default;
	FOnlineSession& operator=(FOnlineSession&&) = default;
	FOnlineSession& operator=(const FOnlineSession&) = default;

	virtual ~FOnlineSession() = default;

	/** @return the session id for the session as a string */
	FString GetSessionIdStr() const
	{
		if (SessionInfo.IsValid() && SessionInfo->IsValid())
		{
			return SessionInfo->GetSessionId().ToString();
		}

		return TEXT("InvalidSession");
	}
};

/** Holds the per session information for named sessions */
class FNamedOnlineSession : public FOnlineSession
{
protected:
	FNamedOnlineSession() :
		SessionName(NAME_None),
		HostingPlayerNum(INDEX_NONE),
		SessionState(EOnlineSessionState::NoSession)
	{
	}

public:
	/** The name of the session */
	const FName SessionName;
	/** Index of the player who created the session [host] or joined it [client] */
	int32 HostingPlayerNum;
	/** Whether or not the local player is hosting this session */
	bool bHosting;

	/** NetId of the local player that created this named session.  Could be the host, or a player joining a session. Will entirely replace HostingPlayerNum */
	FUniqueNetIdPtr LocalOwnerId;

	/** List of players registered in the session */
	TArray< FUniqueNetIdRef > RegisteredPlayers;
	/** State of the session (game thread write only) */
	EOnlineSessionState::Type SessionState;

	/** Constructor used to create a named session directly */
	FNamedOnlineSession(FName InSessionName, const FOnlineSessionSettings& InSessionSettings)
		: FOnlineSession(InSessionSettings)
		, SessionName(InSessionName)
		, HostingPlayerNum(INDEX_NONE)
		, bHosting(false)
		, SessionState(EOnlineSessionState::NoSession)
	{
	}

	/** Constructor used to create a named session directly */
	FNamedOnlineSession(FName InSessionName, const FOnlineSession& Session)
		: FOnlineSession(Session)
		, SessionName(InSessionName)
		, HostingPlayerNum(INDEX_NONE)
		, bHosting(false)
		, SessionState(EOnlineSessionState::NoSession)
	{
	}

	virtual ~FNamedOnlineSession() = default;

	FNamedOnlineSession(const FNamedOnlineSession& Other) = default;
	FNamedOnlineSession(FNamedOnlineSession&& Other) = default;

	// We delete the equals operator as SessionName is immutable
	FNamedOnlineSession& operator=(const FNamedOnlineSession& Other) = delete;
	FNamedOnlineSession& operator=(FNamedOnlineSession&& Other) = delete;

	/**
	 * Calculate the possible joinability state of this session
	 * check the values from left to right in order of precedence
	 *
	 * @param bPublicJoinable [out] is the game joinable by anyone at all
	 * @param bFriendJoinable [out] is the game joinable by friends via presence (doesn't require invite)
	 * @param bInviteOnly [out] is the game joinable via explicit invites
	 * @param bAllowInvites [out] are invites possible (use with bInviteOnly to determine if invites are available right now)
	 *
	 * @return true if the out params are valid, false otherwise
	 */
	bool GetJoinability(bool& bPublicJoinable, bool& bFriendJoinable, bool& bInviteOnly, bool& bAllowInvites) const
	{
		// Only states that have a valid session are considered
		if (SessionState != EOnlineSessionState::NoSession && SessionState != EOnlineSessionState::Creating && SessionState != EOnlineSessionState::Destroying)
		{
			bool bAllowJIP = SessionSettings.bAllowJoinInProgress || (SessionState != EOnlineSessionState::Starting && SessionState != EOnlineSessionState::InProgress);
			if (bAllowJIP)
			{
				bPublicJoinable = SessionSettings.bShouldAdvertise || SessionSettings.bAllowJoinViaPresence;
				bFriendJoinable = SessionSettings.bAllowJoinViaPresenceFriendsOnly;
				bInviteOnly = !bPublicJoinable && !bFriendJoinable && SessionSettings.bAllowInvites;
				bAllowInvites = SessionSettings.bAllowInvites;
			}
			else
			{
				bPublicJoinable = false;
				bFriendJoinable = false;
				bInviteOnly = false;
				bAllowInvites = false;
			}

			// Valid session, joinable or otherwise
			return true;
		}

		// Invalid session
		return false;
	}
};

using FNamedOnlineSessionPtr = TSharedPtr<FNamedOnlineSession, ESPMode::ThreadSafe>;
using FNamedOnlineSessionRef = TSharedRef<FNamedOnlineSession, ESPMode::ThreadSafe>;

/** Value returned on unreachable or otherwise bad search results */
#define MAX_QUERY_PING 9999

/** Representation of a single search result from a FindSession() call */
class FOnlineSessionSearchResult
{
public:
	/** All advertised session information */
	FOnlineSession Session;
	/** Ping to the search result, MAX_QUERY_PING is unreachable */
	int32 PingInMs;

	FOnlineSessionSearchResult()
		: PingInMs(MAX_QUERY_PING)
	{
	}

	/**
	 *	@return true if the search result is valid, false otherwise
	 */
	bool IsValid() const
	{
		return (Session.OwningUserId.IsValid() && IsSessionInfoValid());
	}

	/** 
	 * Check if the session info is valid, for cases where we don't need the OwningUserId
	 * @return true if the session info is valid, false otherwise
	 */
	bool IsSessionInfoValid() const
	{
		return (Session.SessionInfo.IsValid() && Session.SessionInfo->IsValid());
	}

	/** @return the session id for a given session search result */
	FString GetSessionIdStr() const
	{
		return Session.GetSessionIdStr();
	}
};

/**
 * Encapsulation of a search for sessions request.
 * Contains all the search parameters and any search results returned after
 * the OnFindSessionsCompleteDelegate has triggered
 * Check the SearchState for Done/Failed state before using the data
 */
class FOnlineSessionSearch
{
public:
	/** Array of all sessions found when searching for the given criteria */
	TArray<FOnlineSessionSearchResult> SearchResults;
	/** State of the search */
	EOnlineAsyncTaskState::Type SearchState;
	/** Max number of queries returned by the matchmaking service */
	int32 MaxSearchResults;
	/** The query to use for finding matching servers */
	FOnlineSearchSettings QuerySettings;
	/** Whether the query is intended for LAN matches or not */
	bool bIsLanQuery;
	/**
	 * Used to sort games into buckets since a the difference in terms of feel for ping
	 * in the same bucket is often not a useful comparison and skill is better
	 */
	int32 PingBucketSize;
	/** Search hash used by the online subsystem to disambiguate search queries, stamped every time FindSession is called */
	int32 PlatformHash;
	/** Amount of time to wait for the search results. May not apply to all platforms. */
	float TimeoutInSeconds;

public:
	/** Constructor */
	FOnlineSessionSearch()
		: SearchState(EOnlineAsyncTaskState::NotStarted)
		, MaxSearchResults(1)
		, bIsLanQuery(false)
		, PingBucketSize(0)
		, PlatformHash(0)
		, TimeoutInSeconds(0.0f)
	{
		// Examples of setting search parameters
		// QuerySettings.Set(SETTING_MAPNAME, FString(), EOnlineComparisonOp::Equals);
		// QuerySettings.Set(SEARCH_DEDICATED_ONLY, false, EOnlineComparisonOp::Equals);
		// QuerySettings.Set(SEARCH_EMPTY_SERVERS_ONLY, false, EOnlineComparisonOp::Equals);
		// QuerySettings.Set(SEARCH_SECURE_SERVERS_ONLY, false, EOnlineComparisonOp::Equals);
	}

	virtual ~FOnlineSessionSearch() = default;

	FOnlineSessionSearch(FOnlineSessionSearch&&) = default;
	FOnlineSessionSearch(const FOnlineSessionSearch&) = default;
	FOnlineSessionSearch& operator=(FOnlineSessionSearch&&) = default;
	FOnlineSessionSearch& operator=(const FOnlineSessionSearch&) = default;

	/**
	 *	Give the game a chance to sort the returned results
	 */
	virtual void SortSearchResults() {}

	/**
	 * Get the default session settings for this search type
	 * Allows games to set reasonable defaults that aren't advertised
	 * but would be setup for each instantiated search result
	 */
	virtual TSharedPtr<FOnlineSessionSettings> GetDefaultSessionSettings() const
	{
		return MakeShared<FOnlineSessionSettings>();
	}
};

/**
 * Logs session properties used from the session settings
 *
 * @param NamedSession the session to log the information for
 */
void ONLINESUBSYSTEM_API DumpNamedSession(const FNamedOnlineSession* NamedSession);

/**
 * Logs session properties used from the session settings
 *
 * @param Session the session to log the information for
 */
void ONLINESUBSYSTEM_API DumpSession(const FOnlineSession* Session);

/**
 * Logs session properties used from the session settings
 *
 * @param SessionSettings the session to log the information for
 */
void ONLINESUBSYSTEM_API DumpSessionSettings(const FOnlineSessionSettings* SessionSettings);

#undef UE_API
