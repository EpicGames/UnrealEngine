// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemTypes.h"
#include "OnlineKeyValuePair.h"

#define UE_API ONLINESUBSYSTEM_API

// typedef FOnlineKeyValuePairs<FString, FVariantData> FStatsColumnArray;
// Temporary class to assist in deprecation of changing the key from FName to FString. After the deprecation period, the class will be deleted and replaced by the line above.
/** Representation of a single column and its data */ 
class FStatsColumnArray : public FOnlineKeyValuePairs<FString, FVariantData>
{
	typedef FOnlineKeyValuePairs<FString, FVariantData> Super;

public:
	inline FStatsColumnArray() {}
	inline FStatsColumnArray(FStatsColumnArray&& Other) : Super(MoveTemp(Other)) {}
	inline FStatsColumnArray(const FStatsColumnArray& Other) : Super(Other) {}
	inline FStatsColumnArray& operator=(FStatsColumnArray&& Other) { Super::operator=(MoveTemp(Other)); return *this; }
	inline FStatsColumnArray& operator=(const FStatsColumnArray& Other) { Super::operator=(Other); return *this; }
};


// typedef FOnlineKeyValuePairs<FString, FVariantData> FStatPropertyArray;
// Temporary class to assist in deprecation of changing the key from FName to FString. After the deprecation period, the class will be deleted and replaced by the line above.
/** Representation of a single stat value to post to the backend */
class FStatPropertyArray : public FOnlineKeyValuePairs<FString, FVariantData>
{
	typedef FOnlineKeyValuePairs<FString, FVariantData> Super;

public:
	inline FStatPropertyArray() {}
	inline FStatPropertyArray(FStatPropertyArray&& Other) : Super(MoveTemp(Other)) {}
	inline FStatPropertyArray(const FStatPropertyArray& Other) : Super(Other) {}
	inline FStatPropertyArray& operator=(FStatPropertyArray&& Other) { Super::operator=(MoveTemp(Other)); return *this; }
	inline FStatPropertyArray& operator=(const FStatPropertyArray& Other) { Super::operator=(Other); return *this; }
};

class FNameArrayDeprecationWrapper : public TArray<FString>
{
	typedef TArray<FString> Super;

public:
	inline FNameArrayDeprecationWrapper() {}
	inline FNameArrayDeprecationWrapper(FNameArrayDeprecationWrapper&& Other) : Super(MoveTemp(Other)) {}
	inline FNameArrayDeprecationWrapper(const FNameArrayDeprecationWrapper& Other) : Super(Other) {}
	inline FNameArrayDeprecationWrapper& operator=(FNameArrayDeprecationWrapper&& Other) { Super::operator=(MoveTemp(Other)); return *this; }
	inline FNameArrayDeprecationWrapper& operator=(const FNameArrayDeprecationWrapper& Other) { Super::operator=(Other); return *this; }

	inline FNameArrayDeprecationWrapper(TArray<FString>&& Other) : Super(MoveTemp(Other)) {}
	inline FNameArrayDeprecationWrapper(const TArray<FString>& Other) : Super(Other) {}
	inline FNameArrayDeprecationWrapper& operator=(TArray<FString>&& Other) { Super::operator=(MoveTemp(Other)); return *this; }
	inline FNameArrayDeprecationWrapper& operator=(const TArray<FString>& Other) { Super::operator=(Other); return *this; }
};

class FNameDeprecationWrapper : public FString
{
	typedef FString Super;

public:
	inline FNameDeprecationWrapper() {}
	inline FNameDeprecationWrapper(FNameDeprecationWrapper&& Other) : Super(MoveTemp(Other)) {}
	inline FNameDeprecationWrapper(const FNameDeprecationWrapper& Other) : Super(Other) {}
	inline FNameDeprecationWrapper& operator=(FNameDeprecationWrapper&& Other) { Super::operator=(MoveTemp(Other)); return *this; }
	inline FNameDeprecationWrapper& operator=(const FNameDeprecationWrapper& Other) { Super::operator=(Other); return *this; }

	inline FNameDeprecationWrapper(FString&& Other) : Super(MoveTemp(Other)) {}
	inline FNameDeprecationWrapper(const FString& Other) : Super(Other) {}
	inline FNameDeprecationWrapper& operator=(FString&& Other) { Super::operator=(MoveTemp(Other)); return *this; }
	inline FNameDeprecationWrapper& operator=(const FString& Other) { Super::operator=(Other); return *this; }
};

/**
 * An interface used to collect and manage online stats
 */
class FOnlineStats
{
public:

	/** Array of stats we are gathering */
	FStatPropertyArray Properties;

	/**
	 *	Get a key value pair by key name
	 * @param StatName key name to search for
	 * @return KeyValuePair if found, NULL otherwise
	 */
	UE_API class FVariantData* FindStatByName(const FString& StatName);

	/**
	 * Sets a stat of type SDT_Float to the value specified. Does nothing
	 * if the stat is not of the right type.
	 *
	 * @param StatName the stat to change the value of
	 * @param Value the new value to assign to the stat
	 */
	UE_API virtual void SetFloatStat(const FString& StatName, float Value);

	/**
	 * Sets a stat of type SDT_Int to the value specified. Does nothing
	 * if the stat is not of the right type.
	 *
	 * @param StatName the stat to change the value of
	 * @param Value the new value to assign to the stat
	 */
	UE_API virtual void SetIntStat(const FString& StatName, int32 Value);

	/**
	 * Increments a stat of type float by the value specified. Does nothing
	 * if the stat is not of the right type.
	 *
	 * @param StatName the stat to increment
	 * @param IncBy the value to increment by
	 */
	UE_API virtual void IncrementFloatStat(const FString& StatName, float IncBy = 1.0f);

	/**
	 * Increments a stat of type int32 by the value specified. Does nothing
	 * if the stat is not of the right type.
	 *
	 * @param StatName the stat to increment
	 * @param IncBy the value to increment by
	 */
	UE_API virtual void IncrementIntStat(const FString& StatName, int32 IncBy = 1);

	/**
	 * Decrements a stat of type float by the value specified. Does nothing
	 * if the stat is not of the right type.
	 *
	 * @param StatName the stat to decrement
	 * @param DecBy the value to decrement by
	 */
	UE_API virtual void DecrementFloatStat(const FString& StatName, float DecBy = 1.0f);

	/**
	 * Decrements a stat of type int32 by the value specified. Does nothing
	 * if the stat is not of the right type.
	 *
	 * @param StatName the stat to decrement
	 * @param DecBy the value to decrement by
	 */
	UE_API virtual void DecrementIntStat(const FString& StatName, int32 DecBy = 1);

	/**
	 * Destructor
	 */
	virtual ~FOnlineStats() = default;
};

/**
 *	Interface for storing/writing data to a leaderboard
 */
class FOnlineLeaderboardWrite : public FOnlineStats
{
public:

	/** Sort Method */
	ELeaderboardSort::Type SortMethod;
	/** Display Type */
	ELeaderboardFormat::Type DisplayFormat;
	/** Update Method */
	ELeaderboardUpdateMethod::Type UpdateMethod;

	/** Names of the leaderboards to write to */
	FNameArrayDeprecationWrapper LeaderboardNames;

	/** Name of the stat that the leaderboard is rated by */
	FNameDeprecationWrapper RatedStat;

	FOnlineLeaderboardWrite() :
		SortMethod(ELeaderboardSort::None),
		DisplayFormat(ELeaderboardFormat::Number),
		UpdateMethod(ELeaderboardUpdateMethod::KeepBest)
	{
	}
};

/**
 *	Representation of a single row in a retrieved leaderboard
 */
struct FOnlineStatsRow
{
private:
    /** Hidden on purpose */
    FOnlineStatsRow() : NickName() {}

public:
	/** Name of player in this row */
	const FString NickName;
	/** Unique Id for the player in this row */
    const FUniqueNetIdPtr PlayerId;
	/** Player's rank in this leaderboard */
    int32 Rank;
	/** All requested data on the leaderboard for this player */
	FStatsColumnArray Columns;

	FOnlineStatsRow(const FString& InNickname, const FUniqueNetIdRef& InPlayerId) :
		NickName(InNickname),
		PlayerId(InPlayerId)
	{
	}

	FString ToLogString() const;
};

/**
 *	Representation of a single column of data in a leaderboard
 */
struct FColumnMetaData
{
private:
	FColumnMetaData() :
		DataType(EOnlineKeyValuePairDataType::Empty)
	{}

public:

	/** Name of the column to retrieve */
	const FNameDeprecationWrapper ColumnName;
	/** Type of data this column represents */
	const EOnlineKeyValuePairDataType::Type DataType;

	FColumnMetaData(const FString& InColumnName, EOnlineKeyValuePairDataType::Type InDataType) :
		ColumnName(InColumnName),
		DataType(InDataType)
	{
	}
};

/**
 *	Interface for reading data from a leaderboard service
 */
class FOnlineLeaderboardRead
{
public:
	/** Name of the leaderboard read */
	FNameDeprecationWrapper LeaderboardName;
	/** Column this leaderboard is sorted by */
	FNameDeprecationWrapper SortedColumn;
	/** Column metadata for this leaderboard */
	TArray<FColumnMetaData> ColumnMetadata;
	/** Array of ranked users retrieved (not necessarily sorted yet) */
	TArray<FOnlineStatsRow> Rows;
	/** Indicates an error reading data occurred while processing */
	EOnlineAsyncTaskState::Type ReadState;

	FOnlineLeaderboardRead() :
		ReadState(EOnlineAsyncTaskState::NotStarted)
	{
	}

	/**
	 *	Retrieve a single record from the leaderboard for a given user
	 *
	 * @param UserId user id to retrieve a record for
	 * @return the requested user row or NULL if not found
	 */
	FOnlineStatsRow* FindPlayerRecord(const FUniqueNetId& UserId)
	{
		for (int32 UserIdx=0; UserIdx<Rows.Num(); UserIdx++)
		{
			if (*Rows[UserIdx].PlayerId == UserId)
			{
				return &Rows[UserIdx];
			}
		}

		return NULL;
	}

	UE_API FString ToLogString() const;
};

typedef TSharedRef<FOnlineLeaderboardRead, ESPMode::ThreadSafe> FOnlineLeaderboardReadRef;
typedef TSharedPtr<FOnlineLeaderboardRead, ESPMode::ThreadSafe> FOnlineLeaderboardReadPtr;

// TODO ONLINE
class FOnlinePlayerScore
{

};

/**
 * The interface for writing achievement stats to the server.
 */
class FOnlineAchievementsWrite : public FOnlineStats
{
public:
	/**
	 * Constructor
	 */
	FOnlineAchievementsWrite() :
		WriteState(EOnlineAsyncTaskState::NotStarted)
	{

	}

	/** Indicates an error reading data occurred while processing */
	EOnlineAsyncTaskState::Type WriteState;
};

typedef TSharedRef<FOnlineAchievementsWrite, ESPMode::ThreadSafe> FOnlineAchievementsWriteRef;
typedef TSharedPtr<FOnlineAchievementsWrite, ESPMode::ThreadSafe> FOnlineAchievementsWritePtr;

#undef UE_API
