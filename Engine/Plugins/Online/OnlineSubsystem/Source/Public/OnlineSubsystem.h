// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "OnlineSubsystemModule.h"
#include "OnlineSubsystemTypes.h"
#include "OnlineDelegateMacros.h"
#include "OnlineSubsystemNames.h" // IWYU pragma: keep

#define UE_API ONLINESUBSYSTEM_API

class IModuleInterface;

class FOnlineNotificationHandler;
class FOnlineNotificationTransportManager;
class IMessageSanitizer;
class IOnlineAchievements;
class IOnlineChat;
class IOnlineEntitlements;
class IOnlineEvents;
class IOnlineExternalUI;
class IOnlineFriends;
class IOnlineGroups;
class IOnlineIdentity;
class IOnlineLeaderboards;
class IOnlineMessage;
class IOnlinePartySystem;
class IOnlinePresence;
class IOnlinePurchase;
class IOnlineSession;
class IOnlineSharedCloud;
class IOnlineSharing;
class IOnlineStoreV2;
class IOnlineTime;
class IOnlineTitleFile;
class IOnlineTurnBased;
class IOnlineUser;
class IOnlineUserCloud;
class IOnlineVoice;
class IOnlineStats;
class IOnlineGameActivity;
class IOnlineGameItemStats;
class IOnlineGameMatches;
class IOnlineContentAgeRestriction;

LLM_DECLARE_TAG_API(OnlineSubsystem, ONLINESUBSYSTEM_API);

ONLINESUBSYSTEM_API DECLARE_LOG_CATEGORY_EXTERN(LogOnline, Log, All);
ONLINESUBSYSTEM_API DECLARE_LOG_CATEGORY_EXTERN(LogOnlineGame, Log, All);

/** Online subsystem stats */
DECLARE_STATS_GROUP(TEXT("Online"), STATGROUP_Online, STATCAT_Advanced);
/** Total async thread time */
DECLARE_CYCLE_STAT_EXTERN(TEXT("OnlineAsync"), STAT_Online_Async, STATGROUP_Online, ONLINESUBSYSTEM_API);
/** Number of async tasks in queue */
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("NumTasks"), STAT_Online_AsyncTasks, STATGROUP_Online, ONLINESUBSYSTEM_API);
/** Total time to process session interface */
DECLARE_CYCLE_STAT_EXTERN(TEXT("SessionInt"), STAT_Session_Interface, STATGROUP_Online, ONLINESUBSYSTEM_API);
/** Total time to process both local/remote voice */
DECLARE_CYCLE_STAT_EXTERN(TEXT("VoiceInt"), STAT_Voice_Interface, STATGROUP_Online, ONLINESUBSYSTEM_API);

#ifndef ONLINE_LOG_PREFIX
#define ONLINE_LOG_PREFIX TEXT("OSS: ")
#endif

#define UE_LOG_ONLINE(Verbosity, Format, ...) \
{ \
	UE_LOGF(LogOnline, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

#define UE_CLOG_ONLINE(Conditional, Verbosity, Format, ...) \
{ \
	UE_CLOGF(Conditional, LogOnline, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

#define UE_LOG_ONLINE_ONCE(Verbosity, Format, ...) \
{ \
	static bool bLogged = false; \
	UE_CLOG_ONLINE(!bLogged, Verbosity, Format, ##__VA_ARGS__); \
	bLogged = true; \
}

/** Forward declarations of all interface classes */
typedef TSharedPtr<class IOnlineSession, ESPMode::ThreadSafe> IOnlineSessionPtr;
typedef TSharedPtr<class IOnlineFriends, ESPMode::ThreadSafe> IOnlineFriendsPtr;
typedef TSharedPtr<class IOnlinePartySystem, ESPMode::ThreadSafe> IOnlinePartyPtr;
typedef TSharedPtr<class IMessageSanitizer, ESPMode::ThreadSafe> IMessageSanitizerPtr;
typedef TSharedPtr<class IOnlineGroups, ESPMode::ThreadSafe> IOnlineGroupsPtr;
typedef TSharedPtr<class IOnlineSharedCloud, ESPMode::ThreadSafe> IOnlineSharedCloudPtr;
typedef TSharedPtr<class IOnlineUserCloud, ESPMode::ThreadSafe> IOnlineUserCloudPtr;
typedef TSharedPtr<class IOnlineEntitlements, ESPMode::ThreadSafe> IOnlineEntitlementsPtr;
typedef TSharedPtr<class IOnlineLeaderboards, ESPMode::ThreadSafe> IOnlineLeaderboardsPtr;
typedef TSharedPtr<class IOnlineVoice, ESPMode::ThreadSafe> IOnlineVoicePtr;
typedef TSharedPtr<class IOnlineExternalUI, ESPMode::ThreadSafe> IOnlineExternalUIPtr;
typedef TSharedPtr<class IOnlineTime, ESPMode::ThreadSafe> IOnlineTimePtr;
typedef TSharedPtr<class IOnlineIdentity, ESPMode::ThreadSafe> IOnlineIdentityPtr;
typedef TSharedPtr<class IOnlineTitleFile, ESPMode::ThreadSafe> IOnlineTitleFilePtr;
typedef TSharedPtr<class IOnlineStoreV2, ESPMode::ThreadSafe> IOnlineStoreV2Ptr;
typedef TSharedPtr<class IOnlinePurchase, ESPMode::ThreadSafe> IOnlinePurchasePtr;
typedef TSharedPtr<class IOnlineEvents, ESPMode::ThreadSafe> IOnlineEventsPtr;
typedef TSharedPtr<class IOnlineAchievements, ESPMode::ThreadSafe> IOnlineAchievementsPtr;
typedef TSharedPtr<class IOnlineSharing, ESPMode::ThreadSafe> IOnlineSharingPtr;
typedef TSharedPtr<class IOnlineUser, ESPMode::ThreadSafe> IOnlineUserPtr;
typedef TSharedPtr<class IOnlineMessage, ESPMode::ThreadSafe> IOnlineMessagePtr;
typedef TSharedPtr<class IOnlinePresence, ESPMode::ThreadSafe> IOnlinePresencePtr;
typedef TSharedPtr<class IOnlineChat, ESPMode::ThreadSafe> IOnlineChatPtr;
typedef TSharedPtr<class IOnlineTurnBased, ESPMode::ThreadSafe> IOnlineTurnBasedPtr;
typedef TSharedPtr<class IOnlineTournament, ESPMode::ThreadSafe> IOnlineTournamentPtr;
typedef TSharedPtr<class FOnlineNotificationHandler, ESPMode::ThreadSafe> FOnlineNotificationHandlerPtr;
typedef TSharedPtr<class FOnlineNotificationTransportManager, ESPMode::ThreadSafe> FOnlineNotificationTransportManagerPtr;
typedef TSharedPtr<class IOnlineStats, ESPMode::ThreadSafe> IOnlineStatsPtr;
typedef TSharedPtr<IOnlineGameActivity, ESPMode::ThreadSafe> IOnlineGameActivityPtr;
typedef TSharedPtr<IOnlineGameItemStats, ESPMode::ThreadSafe> IOnlineGameItemStatsPtr;
typedef TSharedPtr<IOnlineGameMatches, ESPMode::ThreadSafe> IOnlineGameMatchesPtr;
typedef TSharedPtr<IOnlineContentAgeRestriction, ESPMode::ThreadSafe> IOnlineContentAgeRestrictionPtr;

/**
 * Called when the connection state as reported by the online platform changes
 *
 * @param ServiceName the name of the service that is reporting (platform dependent)
 * @param LastConnectionState last state of the connection
 * @param ConnectionState current state of the connection
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnConnectionStatusChanged, const FString& /*ServiceName*/, EOnlineServerConnectionStatus::Type /*LastConnectionState*/, EOnlineServerConnectionStatus::Type /*ConnectionState*/);
typedef FOnConnectionStatusChanged::FDelegate FOnConnectionStatusChangedDelegate;

/**
 * Delegate fired when the PSN environment changes
 *
 * @param LastEnvironment - old online environment
 * @param Environment - current online environment
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnOnlineEnvironmentChanged, EOnlineEnvironment::Type /*LastEnvironment*/, EOnlineEnvironment::Type /*Environment*/);
typedef FOnOnlineEnvironmentChanged::FDelegate FOnOnlineEnvironmentChangedDelegate;

/**
 * Delegate fired when a configuration change was detected as a result of
 * ReloadConfigs being called. Allows application code to inject custom
 * configuration data before the new configuration is used.
 *
 * @param ConfigSections list of ini sections that need to be reloaded
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnConfigChanged, const TSet<FString>& /*ConfigSections*/);
typedef FOnConfigChanged::FDelegate FOnConfigChangedDelegate;

/**
 *	OnlineSubsystem - Series of interfaces to support communicating with various web/platform layer services
 */
class IOnlineSubsystem
{
protected:
	/** Hidden on purpose */
	IOnlineSubsystem() {}

	FOnlineNotificationHandlerPtr OnlineNotificationHandler;
	FOnlineNotificationTransportManagerPtr OnlineNotificationTransportManager;

public:
	
	virtual ~IOnlineSubsystem() = default;

	/** 
	 * Get the online subsystem for a given service
	 * @param SubsystemName - Name of the requested online service
	 * @return pointer to the appropriate online subsystem
	 */
	static IOnlineSubsystem* Get(const FName& SubsystemName = NAME_None)
	{
		static const FName OnlineSubsystemModuleName = TEXT("OnlineSubsystem");
		FOnlineSubsystemModule& OSSModule = FModuleManager::GetModuleChecked<FOnlineSubsystemModule>(OnlineSubsystemModuleName);
		return OSSModule.GetOnlineSubsystem(SubsystemName);
	}

	/** 
	 * Get the online subsystem native to the current hardware
	 *
	 * @param bAutoLoad - load the module if not already loaded
	 *
	 * @return pointer to the appropriate online subsystem
	 */
	static IOnlineSubsystem* GetByPlatform(bool bAutoLoad=true)
	{
		static const FName OnlineSubsystemModuleName = TEXT("OnlineSubsystem");
		if (bAutoLoad || FModuleManager::Get().IsModuleLoaded(OnlineSubsystemModuleName))
		{
			FOnlineSubsystemModule& OSSModule = FModuleManager::GetModuleChecked<FOnlineSubsystemModule>(OnlineSubsystemModuleName);
			return OSSModule.GetNativeSubsystem(bAutoLoad);
		}
		return nullptr;
	}

	/**
	* Get the online subsystem associated with the given config string
	*
	* @param ConfigString - Key to query for
	* @param bAutoLoad - load the module if not already loaded
	*
	* @return pointer to the appropriate online subsystem
	*/
	static IOnlineSubsystem* GetByConfig(const FString& ConfigString, bool bAutoLoad = true)
	{
		static const FName OnlineSubsystemModuleName = TEXT("OnlineSubsystem");
		if (bAutoLoad || FModuleManager::Get().IsModuleLoaded(OnlineSubsystemModuleName))
		{
			FOnlineSubsystemModule& OSSModule = FModuleManager::GetModuleChecked<FOnlineSubsystemModule>(OnlineSubsystemModuleName);
			return OSSModule.GetSubsystemByConfig(ConfigString, bAutoLoad);
		}
		return nullptr;
	}

	/** 
	 * Destroy a single online subsystem instance
	 * @param SubsystemName - Name of the online service to destroy
	 */
	static void Destroy(FName SubsystemName)
	{
		static const FName OnlineSubsystemModuleName = TEXT("OnlineSubsystem");
		FOnlineSubsystemModule& OSSModule = FModuleManager::GetModuleChecked<FOnlineSubsystemModule>(OnlineSubsystemModuleName);
		return OSSModule.DestroyOnlineSubsystem(SubsystemName);
	}

	/**
	 * Unload the current default subsystem and attempt to reload the configured default subsystem
	 * May be different if the fallback subsystem was created an startup
	 *
	 * **NOTE** This is intended for editor use only, attempting to use this at the wrong time can result
	 * in unexpected crashes/behavior
	 */
	static void ReloadDefaultSubsystem()
	{
		static const FName OnlineSubsystemModuleName = TEXT("OnlineSubsystem");
		FOnlineSubsystemModule& OSSModule = FModuleManager::GetModuleChecked<FOnlineSubsystemModule>(OnlineSubsystemModuleName);
		return OSSModule.ReloadDefaultSubsystem();
	}

	/**
	 * Determine if an instance of the subsystem already exists
	 * @param SubsystemName - Name of the requested online service
	 * @return true if instance exists, false otherwise
	 */
	FORCENOINLINE static bool DoesInstanceExist(const FName& SubsystemName = NAME_None)
	{
		if (FModuleManager::Get().IsModuleLoaded("OnlineSubsystem"))
		{
			FOnlineSubsystemModule& OSSModule = FModuleManager::GetModuleChecked<FOnlineSubsystemModule>("OnlineSubsystem");
			return OSSModule.DoesInstanceExist(SubsystemName);
		}
		return false;
	}

	/** 
	 * Determine if the subsystem for a given interface is already loaded
	 * @param SubsystemName - Name of the requested online service
	 * @return true if module for the subsystem is loaded
	 */
	static bool IsLoaded(const FName& SubsystemName = NAME_None)
	{
		if (FModuleManager::Get().IsModuleLoaded("OnlineSubsystem"))
		{
			FOnlineSubsystemModule& OSSModule = FModuleManager::GetModuleChecked<FOnlineSubsystemModule>("OnlineSubsystem"); 
			return OSSModule.IsOnlineSubsystemLoaded(SubsystemName); 
		}
		return false;
	}

	/** 
	 * Determine if the subsystem for a given interface is enabled by config and command line
	 * @param SubsystemName - Name of the requested online service
	 * @return true if the subsystem is enabled by config
	 */
	static UE_API bool IsEnabled(const FName& SubsystemName, const FName& InstanceName = NAME_None);

	/**
	 * Return the name of the subsystem @see OnlineSubsystemNames.h
	 *
	 * @return the name of the subsystem, as used in calls to IOnlineSubsystem::Get()
	 */
	virtual FName GetSubsystemName() const = 0;

	/**
	 * Get the instance name, which is typically "default" or "none" but distinguishes
	 * one instance from another in "Play In Editor" mode.  Most platforms can't do this
	 * because of third party requirements that only allow one login per machine instance
	 *
	 * @return the instance name of this subsystem
	 */
	virtual FName GetInstanceName() const = 0;

	/**
	 * Get the local online platform based on compile time determination of hardware.
	 * @see OnlineSubsystemNames.h OSS_PLATFORM_NAME_*
	 * @return string representation of online platform name
	 */
	static UE_API FString GetLocalPlatformName();

	/** @return true if the subsystem is enabled, false otherwise */
	virtual bool IsEnabled() const = 0;

	/** 
	 * Get the interface for accessing the session management services
	 * @return Interface pointer for the appropriate session service
	 */
	virtual IOnlineSessionPtr GetSessionInterface() const = 0;
	
	/** 
	 * Get the interface for accessing the player friends services
	 * @return Interface pointer for the appropriate friend service
	 */
	virtual IOnlineFriendsPtr GetFriendsInterface() const = 0;

	/**
	 * Get the interface for accessing the message sanitizer service
	 * @param LocalUserNum the controller number of the associated user
	 * @param OutAuthTypeToExclude platform to exclude in sanitization requests
	 * @return Interface pointer for the appropriate sanitizer service
	 */
	virtual IMessageSanitizerPtr GetMessageSanitizer(int32 LocalUserNum, FString& OutAuthTypeToExclude) const = 0;

	/**
	 * Get the interface for accessing the groups services
	 * @return Interface pointer for appropriate groups service
	 */
	virtual IOnlineGroupsPtr GetGroupsInterface() const = 0;

	/** 
	 * Get the interface for accessing the player party services
	 * @return Interface pointer for the appropriate party service
	 */
	virtual IOnlinePartyPtr GetPartyInterface() const = 0;

	/** 
	 * Get the interface for sharing user files in the cloud
	 * @return Interface pointer for the appropriate cloud service
	 */
	virtual IOnlineSharedCloudPtr GetSharedCloudInterface() const = 0;

	/**
	* Get the interface for accessing user files in the cloud
	* @return Interface pointer for the appropriate cloud service
	*/
	virtual IOnlineUserCloudPtr GetUserCloudInterface() const = 0;

	/**
	 * Get the interface for accessing user entitlements
	 * @return Interface pointer for the appropriate entitlements service
	 */
	virtual IOnlineEntitlementsPtr GetEntitlementsInterface() const = 0;

	/** 
	 * Get the interface for accessing leaderboards/rankings of a service
	 * @return Interface pointer for the appropriate leaderboard service
	 */
	virtual IOnlineLeaderboardsPtr GetLeaderboardsInterface() const = 0;

	/** 
	 * Get the interface for accessing voice related data
	 * @return Interface pointer for the appropriate voice service
	 */
	virtual IOnlineVoicePtr GetVoiceInterface() const = 0;

	/** 
	 * Get the interface for accessing the external UIs of a service
	 * @return Interface pointer for the appropriate external UI service
	 */
	virtual IOnlineExternalUIPtr GetExternalUIInterface() const = 0;

	/** 
	 * Get the interface for accessing the server time from an online service
	 * @return Interface pointer for the appropriate server time service
	 */
	virtual IOnlineTimePtr GetTimeInterface() const = 0;

	/** 
	 * Get the interface for accessing identity online services
	 * @return Interface pointer for the appropriate identity service
	 */
	virtual IOnlineIdentityPtr GetIdentityInterface() const = 0;

	/** 
	 * Get the interface for accessing title file online services
	 * @return Interface pointer for the appropriate title file service
	 */
	virtual IOnlineTitleFilePtr GetTitleFileInterface() const = 0;

	/** 
	 * Get the interface for accessing an online store
	 * @return Interface pointer for the appropriate online store service
	 */
	virtual IOnlineStoreV2Ptr GetStoreV2Interface() const = 0;

	/** 
	 * Get the interface for purchasing 
	 * @return Interface pointer for the appropriate purchase service
	 */
	virtual IOnlinePurchasePtr GetPurchaseInterface() const = 0;
	
	/** 
	 * Get the interface for accessing online achievements
	 * @return Interface pointer for the appropriate online achievements service
	 */
	virtual IOnlineEventsPtr GetEventsInterface() const = 0;

	/** 
	 * Get the interface for accessing online achievements
	 * @return Interface pointer for the appropriate online achievements service
	 */
	virtual IOnlineAchievementsPtr GetAchievementsInterface() const = 0;
	
	/** 
	 * Get the interface for accessing online sharing
	 * @return Interface pointer for the appropriate online sharing service
	 */
	virtual IOnlineSharingPtr GetSharingInterface() const = 0;
	
	/** 
	 * Get the interface for accessing online user information
	 * @return Interface pointer for the appropriate online user service
	 */
	virtual IOnlineUserPtr GetUserInterface() const = 0;

	/** 
	 * Get the interface for accessing online messages
	 * @return Interface pointer for the appropriate online message service
	 */
	virtual IOnlineMessagePtr GetMessageInterface() const = 0;

	/** 
	 * Get the interface for managing rich presence information
	 * @return Interface pointer for the appropriate online presence service
	 */
	virtual IOnlinePresencePtr GetPresenceInterface() const = 0;

	/** 
	 * Get the interface for user-user and user-room chat functionality
	 * @return Interface pointer for the appropriate online chat service
	 */
	virtual IOnlineChatPtr GetChatInterface() const = 0;
	
	/** 
	 * Get the interface for accessing online stats
	 * @return Interface pointer for the appropriate online stat service
	 */
	virtual IOnlineStatsPtr GetStatsInterface() const = 0;

	/** 
	 * Get interface for accessing the game activity
	 * @return Interface pointer for the game activity interface
	 */
	virtual IOnlineGameActivityPtr GetGameActivityInterface() const = 0;

	/**
	 * Get interface for accessing the game item stats
	 * @return Interface pointer for the game item stats interface
	 */
	virtual IOnlineGameItemStatsPtr GetGameItemStatsInterface() const = 0;

	/**
	 * Get interface for accessing game matches
	 * @return Interface pointer for the game matches interface
	 */
	virtual IOnlineGameMatchesPtr GetGameMatchesInterface() const = 0;

	/**
	 * Get interface for accessing the online content age restriction
	 * @return Interface pointer for online content age restriction
	 */
	 virtual IOnlineContentAgeRestrictionPtr GetOnlineContentAgeRestrictionInterface() const = 0;

	/**
	 * Get the notification handler instance for this subsystem
	 * @return Pointer for the appropriate notification handler
	 */
	FOnlineNotificationHandlerPtr GetOnlineNotificationHandler() const
	{
		return OnlineNotificationHandler;
	}

	/**
	 * Get the interface for managing turn based multiplayer games
	 * @return Interface pointer for the appropriate online turn-based service
	 */
	virtual IOnlineTurnBasedPtr GetTurnBasedInterface() const = 0;

	/**
	 * Get the interface for managing tournament information
	 * @return Interface pointer for the appropriate online tournament service
	 */
	virtual IOnlineTournamentPtr GetTournamentInterface() const = 0;

	/**
	 * Get the transport manager instance for this subsystem
	 * @return Pointer for the appropriate transport manager
	 */
	FOnlineNotificationTransportManagerPtr GetOnlineNotificationTransportManager() const
	{
		return OnlineNotificationTransportManager;
	}

	/**
	 * Get custom UObject data preserved by the online subsystem
	 *
	 * @param InterfaceName key to the custom data
	 */
	virtual class UObject* GetNamedInterface(FName InterfaceName) = 0;

	/**
	 * Set a custom UObject to be preserved by the online subsystem
	 *
	 * @param InterfaceName key to the custom data
	 * @param NewInterface object to preserve
	 */
	virtual void SetNamedInterface(FName InterfaceName, class UObject* NewInterface) = 0;

	/**
	 * Is the online subsystem associated with the game/editor/engine running as dedicated.
	 * May be forced into this mode by EditorPIE, but basically asks if the OSS is serving
	 * in a dedicated capacity
	 *
	 * @return true if the online subsystem is in dedicated server mode, false otherwise
	 */
	virtual bool IsDedicated() const = 0;

	/**
	 * Is this instance of the game running as a server (dedicated OR listen)
	 * checks the Engine if possible for netmode status
	 *
	 * @return true if this is the server, false otherwise
	 */
	virtual bool IsServer() const = 0;

	/**
	 * Force the online subsystem to behave as if it's associated with running a dedicated server
	 *
	 * @param bForce force dedicated mode if true
	 */
	virtual void SetForceDedicated(bool bForce) = 0;

	/**
	 * Is a player local to this machine by unique id
	 *
	 * @param UniqueId UniqueId of the player
	 *
	 * @return true if unique id is local to this machine, false otherwise
	 */
	virtual bool IsLocalPlayer(const FUniqueNetId& UniqueId) const = 0;

	/** 
	 * Initialize the underlying subsystem APIs
	 * @return true if the subsystem was successfully initialized, false otherwise
	 */
	virtual bool Init() = 0;

	/** 
	 * Perform any shutdown actions prior to any other modules being unloaded/shutdown
	 */
	virtual void PreUnload() = 0;

	/** 
	 * Shutdown the underlying subsystem APIs
	 * @return true if the subsystem shutdown successfully, false otherwise
	 */
	virtual bool Shutdown() = 0;

	/**
	 * Each online subsystem has a global id for the app
	 *
	 * @return the app id for this app
	 */
	virtual FString GetAppId() const = 0;

	/**
	 * Exec handler that allows the online subsystem to process exec commands
	 *
	 * @param InWorld world
	 * @param Cmd the exec command being executed
	 * @param Ar the archive to log results to
	 *
	 * @return true if the handler consumed the input, false to continue searching handlers
	 */
	virtual bool Exec(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) = 0;

	/**
	 * Some platforms must know when the game is using Multiplayer features so they can do recurring authorization checks.
	 */
	virtual void SetUsingMultiplayerFeatures(const FUniqueNetId& UniqueId, bool bUsingMP) = 0;

	/**
	 * Called when the connection state as reported by the online platform changes
	 *
	 * @param ServiceName the name of the service that is reporting (platform dependent)
	 * @param LastConnectionState last state of the connection
	 * @param ConnectionState current state of the connection
	 */
	DEFINE_ONLINE_DELEGATE_THREE_PARAM(OnConnectionStatusChanged, const FString& /*ServiceName*/, EOnlineServerConnectionStatus::Type /*LastConnectionState*/, EOnlineServerConnectionStatus::Type /*ConnectionState*/);

	/**
	 * @return the current environment being used for the online platform
	 */
	virtual EOnlineEnvironment::Type GetOnlineEnvironment() const = 0;

	/**
	 * @return the current environment being used for the online platform as defined by the platform (not necessarily EOnlineEnvironment::ToString)
	 */
	virtual FString GetOnlineEnvironmentName() const = 0;

	/**
	 * Delegate fired when the online environment changes
	 *
	 * @param LastEnvironment - old online environment
	 * @param Environment - current online environment
	 */
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnOnlineEnvironmentChanged, EOnlineEnvironment::Type /*LastEnvironment*/, EOnlineEnvironment::Type /*Environment*/);

	/**
	 * @return The name of the online service this platform uses
	 */
	virtual FText GetOnlineServiceName() const = 0;

	/**
	 * @return The name of the social platform for this subsystem
	 */
	virtual FText GetSocialPlatformName() const = 0;

	/**
	 * Delegate fired when a configuration change was detected as a result of
	 * ReloadConfigs being called. Allows application code to inject custom
	 * configuration data before the new configuration is used.
	 *
	 * @param ConfigSections list of ini sections that need to be reloaded
	 */
	DEFINE_ONLINE_DELEGATE_ONE_PARAM(OnConfigChanged, const TSet<FString>& /*ConfigSections*/);

	/**
	 * Reload the configuration if it is relevant for this OSS instance
	 *
	 * @param ConfigSections list of ini sections that need to be reloaded
	 */
	virtual void ReloadConfigs(const TSet<FString>& ConfigSections) = 0;
};

/** Public references to the online subsystem pointer should use this */
typedef TSharedPtr<IOnlineSubsystem, ESPMode::ThreadSafe> IOnlineSubsystemPtr;

/**
 * Interface for creating the actual online subsystem instance for a given platform
 * all modules must implement this
 */
class IOnlineFactory
{
public:

	IOnlineFactory() {}
	virtual ~IOnlineFactory() = default;

	/**
	 * Create an instance of the platform subsystem
	 *
	 * @param InstanceName name of this single instance of the subsystem
	 * @return newly created and initialized online subsystem, NULL if failure
	 */
	virtual IOnlineSubsystemPtr CreateSubsystem(FName InstanceName) = 0;
};

/**
 * Generates a unique number based off of the current engine package
 *
 * @return the unique number from the current engine package
 */
ONLINESUBSYSTEM_API int32 GetBuildUniqueId();

/**
 * Common implementation for finding a player in a session
 *
 * @param SessionInt Session interface to use
 * @param SessionName Session name to check for player
 * @param UniqueId UniqueId of the player
 *
 * @return true if unique id found in session, false otherwise
 */
ONLINESUBSYSTEM_API bool IsPlayerInSessionImpl(class IOnlineSession* SessionInt, FName SessionName, const FUniqueNetId& UniqueId);

/**
 * Is the unique id local to this instance
 *
 * @param UniqueId unique to query
 *
 * @return true if unique id is found and logged in locally, false otherwise
 */
ONLINESUBSYSTEM_API bool IsUniqueIdLocal(const FUniqueNetId& UniqueId);

/**
 * Retrieve the beacon port from the specified session settings
 *
 * @param SessionSettings the session settings that contains the BEACON_PORT setting
 *
 * @return the port if found, otherwise DEFAULT_BEACON_PORT
 */
ONLINESUBSYSTEM_API int32 GetBeaconPortFromSessionSettings(const class FOnlineSessionSettings& SessionSettings);

/** Temp solution for some hardcoded access to logged in user 0, please avoid using this */
ONLINESUBSYSTEM_API FUniqueNetIdPtr GetFirstSignedInUser(IOnlineIdentityPtr IdentityInt);

#undef UE_API
