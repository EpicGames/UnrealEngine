// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

/*
 * Enable runtime checks by default in editor builds. Other targets should set this definition in
 * their respective target files.
 */
#ifndef ONLINE_SUBSYSTEM_FEATURE_RUNTIMECHECKS
#define ONLINE_SUBSYSTEM_FEATURE_RUNTIMECHECKS UE_EDITOR
#endif

/**
  * Feature checks allow an application to enable online subsystem features based on runtime
  * configuration such as the command-line. The behavior is split so that features can be toggled
  * on or off through a build script while allowing the application to override when
  * ONLINE_SUBSYSTEM_FEATURE_RUNTIMECHECKS is enabled.
  *
  * Application overrides must be registered before the OnlineSubsystem module is started by
  * using AdditionalModulesToLoad and are evaluated only once for the duration of the application.
  * 
  * Example application usage:
  * REGISTER_ONLINE_SUBSYSTEM_FEATURE_RUNTIMECHECK(FRIENDS, FOnlineSubsystemFeatureCheck::CreateStatic(FParse::Param(FCommandLine::Get(), TEXT("EnableFriends")))
*/ 

DECLARE_DELEGATE_RetVal(bool, FOnlineSubsystemFeatureCheck);

#if ONLINE_SUBSYSTEM_FEATURE_RUNTIMECHECKS

#define UE_REGISTER_ONLINE_SUBSYSTEM_FEATURE_RUNTIMECHECK(FeatureName, CheckFunction) \
UE::OnlineSubsystemFeature::FeatureRuntimeCheckDelegate_##FeatureName = CheckFunction;

#define UE_UNREGISTER_ONLINE_SUBSYSTEM_FEATURE_RUNTIMECHECK(FeatureName) \
UE::OnlineSubsystemFeature::FeatureRuntimeCheckDelegate_##FeatureName.Unbind();

#define UE_DECLARE_ONLINE_SUBSYSTEM_FEATURE(StorageClass, FeatureName) \
namespace UE::OnlineSubsystemFeature { \
	StorageClass extern FOnlineSubsystemFeatureCheck FeatureRuntimeCheckDelegate_##FeatureName; \
	StorageClass bool RuntimeCheckFeature_##FeatureName(); \
}

#define UE_DEFINE_ONLINE_SUBSYSTEM_FEATURE(FeatureName) \
namespace UE::OnlineSubsystemFeature { \
	FOnlineSubsystemFeatureCheck FeatureRuntimeCheckDelegate_##FeatureName; \
	bool RuntimeCheckFeature_##FeatureName() \
	{ \
		if constexpr (ONLINE_SUBSYSTEM_FEATURE_##FeatureName) \
		{ \
			static const bool Runtime##FeatureName##Enabled = FeatureRuntimeCheckDelegate_##FeatureName.IsBound() ? FeatureRuntimeCheckDelegate_##FeatureName.Execute() : true; \
			return Runtime##FeatureName##Enabled; \
		} \
		else \
		{ \
			return false; \
		} \
	} \
}

#define UE_ONLINE_SUBSYSTEM_FEATURE_ENABLED(FeatureName) \
UE::OnlineSubsystemFeature::RuntimeCheckFeature_##FeatureName()

#else

/* No-op implementation. */
#define UE_REGISTER_ONLINE_SUBSYSTEM_FEATURE_RUNTIMECHECK(FeatureName, CheckFunction)
#define UE_UNREGISTER_ONLINE_SUBSYSTEM_FEATURE_RUNTIMECHECK(FeatureName)
#define UE_DECLARE_ONLINE_SUBSYSTEM_FEATURE(StorageClass, FeatureName)
#define UE_DEFINE_ONLINE_SUBSYSTEM_FEATURE(FeatureName)

/* When runtime checks are disabled return the preprocessor value. */
#define UE_ONLINE_SUBSYSTEM_FEATURE_ENABLED(FeatureName) \
ONLINE_SUBSYSTEM_FEATURE_##FeatureName

#endif
