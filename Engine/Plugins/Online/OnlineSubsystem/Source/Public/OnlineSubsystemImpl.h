// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/IConsoleManager.h"
#include "OnlineSubsystem.h"
#include "Containers/Queue.h"
#include "Containers/Ticker.h"

#define UE_API ONLINESUBSYSTEM_API

struct FOnlineError;

DECLARE_DELEGATE(FNextTickDelegate);

namespace OSSConsoleVariables
{
	extern ONLINESUBSYSTEM_API TAutoConsoleVariable<int32> CVarVoiceLoopback;
}

/**
 *	FOnlineSubsystemImpl - common functionality to share across online platforms, not intended for direct use
 */
class FOnlineSubsystemImpl 
	: public IOnlineSubsystem
	, public FTSTickerObjectBase
{
private:

	/**
	 * Exec function handling for Exec() call
	 */
	UE_API bool HandleFriendExecCommands(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar);
	UE_API bool HandleIdentityExecCommands(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar);
	UE_API bool HandleSessionExecCommands(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar);
	UE_API bool HandlePresenceExecCommands(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar);
	UE_API bool HandlePurchaseExecCommands(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar);
	UE_API bool HandleStoreExecCommands(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar);
	
	/** Delegate fired when exec cheat related to receipts completes */
	UE_API void OnQueryReceiptsComplete(const FOnlineError& Result, FUniqueNetIdPtr UserId);
	
	/** Dump purchase receipts for a given user id */
	UE_API void DumpReceipts(const FUniqueNetId& UserId);
	/** Finalize purchases known to the client, will wipe real money purchases without fulfillment */
	UE_API void FinalizeReceipts(const FUniqueNetId& UserId);

protected:

	/** Hidden on purpose */
	FOnlineSubsystemImpl() = delete;

	UE_API FOnlineSubsystemImpl(FName InSubsystemName, FName InInstanceName);
	UE_API FOnlineSubsystemImpl(FName InSubsystemName, FName InInstanceName, FTSTicker& Ticker);

	/** Name of the subsystem @see OnlineSubsystemNames.h */
	FName SubsystemName;
	/** Instance name (disambiguates PIE instances for example) */
	FName InstanceName;

	/** Whether or not the online subsystem is in forced dedicated server mode */
	bool bForceDedicated;

	/** Holds all currently named interfaces */
	mutable class UNamedInterfaces* NamedInterfaces;

	/** Load in any named interfaces specified by the ini configuration */
	UE_API void InitNamedInterfaces();

	/** Delegate fired when named interfaces are cleaned up at exit */
	UE_API void OnNamedInterfaceCleanup();

	/** Queue to hold callbacks scheduled for next tick using ExecuteNextTick */
	TQueue<FNextTickDelegate, EQueueMode::Mpsc> NextTickQueue;

	/** Buffer to hold callbacks for the current tick (so it's safe to call ExecuteNextTick within a tick callback) */
	TArray<FNextTickDelegate> CurrentTickBuffer;

	/** Start Ticker */
	UE_API void StartTicker();

	/** Stop Ticker */
	UE_API void StopTicker();

	/** Is the ticker started */
	bool bTickerStarted;

public:
	
	UE_API virtual ~FOnlineSubsystemImpl();

	// IOnlineSubsystem
	UE_API virtual IOnlineGroupsPtr GetGroupsInterface() const override;
	UE_API virtual IOnlinePartyPtr GetPartyInterface() const override;
	UE_API virtual IOnlineSharedCloudPtr GetSharedCloudInterface() const override;
	UE_API virtual IOnlineUserCloudPtr GetUserCloudInterface() const override;
	UE_API virtual IOnlineEntitlementsPtr GetEntitlementsInterface() const override;
	UE_API virtual IOnlineLeaderboardsPtr GetLeaderboardsInterface() const override;
	UE_API virtual IOnlineVoicePtr GetVoiceInterface() const override;
	UE_API virtual IOnlineExternalUIPtr GetExternalUIInterface() const override;
	UE_API virtual IOnlineTimePtr GetTimeInterface() const override;
	UE_API virtual IOnlineIdentityPtr GetIdentityInterface() const override;
	UE_API virtual IOnlineTitleFilePtr GetTitleFileInterface() const override;
	UE_API virtual IOnlineStoreV2Ptr GetStoreV2Interface() const override;
	UE_API virtual IOnlinePurchasePtr GetPurchaseInterface() const override;
	UE_API virtual IOnlineEventsPtr GetEventsInterface() const override;
	UE_API virtual IOnlineAchievementsPtr GetAchievementsInterface() const override;
	UE_API virtual IOnlineSharingPtr GetSharingInterface() const override;
	UE_API virtual IOnlineUserPtr GetUserInterface() const override;
	UE_API virtual IOnlineMessagePtr GetMessageInterface() const override;
	UE_API virtual IOnlinePresencePtr GetPresenceInterface() const override;
	UE_API virtual IOnlineChatPtr GetChatInterface() const override;
	UE_API virtual IOnlineStatsPtr GetStatsInterface() const override;
	UE_API virtual IOnlineGameActivityPtr GetGameActivityInterface() const override;
	UE_API virtual IOnlineGameItemStatsPtr GetGameItemStatsInterface() const override;
	UE_API virtual IOnlineGameMatchesPtr GetGameMatchesInterface() const override;
	UE_API virtual IOnlineTurnBasedPtr GetTurnBasedInterface() const override;
	UE_API virtual IOnlineTournamentPtr GetTournamentInterface() const override;
	UE_API virtual IOnlineContentAgeRestrictionPtr GetOnlineContentAgeRestrictionInterface() const override;
	UE_API virtual void PreUnload() override;
	UE_API virtual bool Shutdown() override;
	UE_API virtual bool IsServer() const override;
	virtual bool IsDedicated() const override{ return bForceDedicated || IsRunningDedicatedServer(); }
	virtual void SetForceDedicated(bool bForce) override { bForceDedicated = bForce; }
	UE_API virtual class UObject* GetNamedInterface(FName InterfaceName) override;
	UE_API virtual void SetNamedInterface(FName InterfaceName, class UObject* NewInterface) override;
	UE_API virtual bool IsLocalPlayer(const FUniqueNetId& UniqueId) const override;
	virtual void SetUsingMultiplayerFeatures(const FUniqueNetId& UniqueId, bool bUsingMP) override {};
	virtual EOnlineEnvironment::Type GetOnlineEnvironment() const override { return EOnlineEnvironment::Unknown; }
	virtual FString GetOnlineEnvironmentName() const override { return EOnlineEnvironment::ToString(GetOnlineEnvironment()); }
	UE_API virtual IMessageSanitizerPtr GetMessageSanitizer(int32 LocalUserNum, FString& OutAuthTypeToExclude) const override;
	UE_API virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
	virtual FName GetSubsystemName() const override { return SubsystemName; }
	virtual FName GetInstanceName() const override { return InstanceName; }
	UE_API virtual bool IsEnabled() const override;
	virtual void ReloadConfigs(const TSet<FString>& /*ConfigSections*/) override {};
	UE_API virtual FText GetSocialPlatformName() const override;

	// FTSTickerObjectBase
	UE_API virtual bool Tick(float DeltaTime) override;

	/* Performs cleanup tasks for editor teardown */
	UE_API void PIEEndCleanup();

	/**
	 * Modify a response string so that it can be logged cleanly
	 *
	 * @param ResponseStr - The JSONObject string we want to sanitize
	 * @param RedactFields - The fields we want to specifically omit (optional, only supports EJson::String), if nothing specified everything is redacted
	 * @return the modified version of the response string
	 */
	static UE_API FString FilterResponseStr(const FString& ResponseStr, const TArray<FString>& RedactFields = TArray<FString>());

	/**
	 * Queue a delegate to be executed on the next tick
	 */
	UE_API void ExecuteDelegateNextTick(const FNextTickDelegate& Callback);

	/**
	 * Templated helper for calling ExecuteDelegateNextTick with a lambda function
	 */
	template<typename LAMBDA_TYPE>
	inline void ExecuteNextTick(LAMBDA_TYPE&& Callback)
	{
		ExecuteDelegateNextTick(FNextTickDelegate::CreateLambda(Forward<LAMBDA_TYPE>(Callback)));
	}

	/**
	* Delegate for config file changes.
	*/
	UE_API virtual void OnConfigSectionsChanged(const FString& IniFilename, const TSet<FString>& SectionNames);

	/** Name given to default OSS instances (disambiguates for PIE) */
	static UE_API const FName DefaultInstanceName;
};

#undef UE_API
