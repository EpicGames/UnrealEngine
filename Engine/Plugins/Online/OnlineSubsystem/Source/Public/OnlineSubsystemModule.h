// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/TransactionallySafeCriticalSection.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

class IOnlineFactory;
class IOnlineSubsystem;
typedef TSharedPtr<class IOnlineSubsystem, ESPMode::ThreadSafe> IOnlineSubsystemPtr;

/**
 * Online subsystem module class
 * Wraps the loading of an online subsystem by name and allows new services to register themselves for use
 */
class FOnlineSubsystemModule : public IModuleInterface
{
private:

	/** Name of the default online service requested 
	 * Specified in DefaultEngine.ini 
	 *	[OnlineSubsystem] 
	 *	DefaultPlatformService 
	 */
	FName DefaultPlatformService;

	/**
	 * Name of the online service associated with the native platform
	 * Specified in Base<Platform>Engine.ini
	 *	[OnlineSubsystem]
	 *	NativePlatformService
	 */
	FName NativePlatformService;

	/**
	* Name of the online service associated with User Defined strings
	* Specified in Base<Platform>Engine.ini
	*	[OnlineSubsystem]
	*	ConfigDefinedPlatformServices
	*/
	TMap<FString, FName> ConfigDefinedSubsystems;

	/** Existing instances of any online subsystems created <PlatformName:InstanceName> */
	TMap<FName, IOnlineFactory*> OnlineFactories;

	/** Mapping of all currently loaded platform service subsystems to their name */
	TMap<FName, IOnlineSubsystemPtr> OnlineSubsystems;

	/**
	 * Lock for access to OnlineSubsystems array to allow for multiple threads to be able to create/get online subsystems simultaneously.
	 * The intent is OnlineSubsystems can have elements added on separate threads through normal gameplay, though this is expected to be very rare and only once per subsystem type.
	 * OnlineSubsystems is not expected to remove elements except in very controlled scenarios.  In that case, the lock is only limiting access to the OnlineSubsystems array, but the result may be invalidated.
	 */
	mutable FTransactionallySafeCriticalSection OnlineSubsystemsLock;

	/** Have we warned already for a given online subsystem creation failure */
	TSet<FName> OnlineSubsystemFailureNotes;

	/** Config driven override of module name for online subsystem */
	TMap<FString, FName> ModuleRedirects;

	/**
	 * Transform an online subsystem identifier into its Subsystem and Instance constituents
	 *
	 * accepts the following forms:
	 * <subsystem name>:<instance name> -> subsystem name / instance name
	 * :<instance name>					-> default subsystem / instance name
	 * <subsystem name>:				-> subsystem name / default instance
	 * <subsystem name>					-> subsystem name / default instance
	 * <nothing>						-> default subsystem / default instance
	 *
	 * @param FullName full name of the subsystem and instance that is being referenced
	 * @param SubsystemName parsed out value or default subsystem name
	 * @param InstanceName parsed out value or default instance name
	 *
	 * @return Properly formatted key name for accessing the online subsystem in the TMap
	 */
	FName ParseOnlineSubsystemName(const FName& FullName, FName& SubsystemName, FName& InstanceName) const;

	/**
	 * Read any config defined subsystems from the configuration file
	 */
	void ProcessConfigDefinedSubsystems();

	/**
	 * Read any config defined subsystem module overrides from the configuration file
	 */
	void ProcessConfigDefinedModuleRedirects();

	/**
	 * Attempt to load the default subsystem specified in the configuration file
	 */
	bool TryLoadSubsystemAndSetDefault(FName ModuleName);

	/**
	 * Attempt to load the default subsystem specified in the configuration file
	 */
	void LoadDefaultSubsystem();

	/**
	 *	Called before ShutdownOnlineSubsystem, before other modules have been unloaded
	 */
	virtual void PreUnloadOnlineSubsystem();

	/**
	 *	Shuts down all registered online subsystem platforms and unloads their modules
	 */
	virtual void ShutdownOnlineSubsystem();

public:

	FOnlineSubsystemModule() :
		DefaultPlatformService(NAME_None)
	{}
	virtual ~FOnlineSubsystemModule() = default;

	/** 
	 * Main entry point for accessing an online subsystem by name
	 * Will load the appropriate module if the subsystem isn't currently loaded
	 * It's possible that the subsystem doesn't exist and therefore can return NULL
	 *
	 * @param InSubsystemName - name of subsystem as referenced by consumers
	 *
	 * @return Requested online subsystem, or NULL if that subsystem was unable to load or doesn't exist
	 */
	virtual IOnlineSubsystem* GetOnlineSubsystem(const FName InSubsystemName = NAME_None);

	/** 
	 * Get the online subsystem native to the current hardware
	 *
	 * @param bAutoLoad - load the module if not already loaded
	 *
	 * @return pointer to the appropriate online subsystem
	 */
	virtual IOnlineSubsystem* GetNativeSubsystem(bool bAutoLoad);

	/**
	 * Get the online subsystem associated with the given config string
	 *
	 * @param ConfigString - Key to query for
	 * @param bAutoLoad - load the module if not already loaded
	 *
	 * @return pointer to the appropriate online subsystem
	 */
	virtual IOnlineSubsystem* GetSubsystemByConfig(const FString& ConfigString, bool bAutoLoad);

	/**
	 * Destroys an online subsystem created internally via access with GetOnlineSubsystem
	 * Typically destruction of the subsystem is handled at application exit, but
	 * there may be rare instances where the subsystem is destroyed by request
	 *
	 * @param InSubsystemName - name of subsystem as referenced by consumers
	 */
	virtual void DestroyOnlineSubsystem(const FName InSubsystemName);

	/**
	 * Does an instance of subsystem with the given name exist
	 *
	 * @return true if the instance exists, false otherwise
	 */
	ONLINESUBSYSTEM_API bool DoesInstanceExist(const FName InSubsystemName) const;

	/** 
	 * Determine if a subsystem is loaded by the OSS module
	 *
	 * @param SubsystemName - name of subsystem as referenced by consumers
	 * @return true if module for the subsystem is loaded
	 */
	virtual bool IsOnlineSubsystemLoaded(const FName InSubsystemName) const;

	/** 
	 * Register a new online subsystem interface with the base level factory provider
	 * @param FactoryName - name of subsystem as referenced by consumers
	 * @param Factory - instantiation of the online subsystem interface, this will take ownership
	 */
	virtual void RegisterPlatformService(const FName FactoryName, class IOnlineFactory* Factory);
	
	/** 
	 * Unregister an existing online subsystem interface from the base level factory provider
	 * @param FactoryName - name of subsystem as referenced by consumers
	 */
	virtual void UnregisterPlatformService(const FName FactoryName);

	/**
	 * Enumerate all loaded online subsystems
	 *
	 * @param EnumCb functor to call for each online subsystem
	 */
	typedef TFunction<void(IOnlineSubsystem*)> FEnumerateOnlineSubsystemCb;
	void ONLINESUBSYSTEM_API EnumerateOnlineSubsystems(FEnumerateOnlineSubsystemCb& EnumCb);

	/**
	 * Shutdown the current default subsystem (may be the fallback) and attempt to reload the one 
	 * specified in the configuration file
	 *
	 * **NOTE** This is intended for editor use only, attempting to use this at the wrong time can result
	 * in unexpected crashes/behavior
	 */
	void ONLINESUBSYSTEM_API ReloadDefaultSubsystem();

	// IModuleInterface

	/**
	 * Called right after the module DLL has been loaded and the module object has been created
	 * Overloaded to allow the default subsystem a chance to load
	 */
	virtual void StartupModule() override;

	/**
	 * Called before the module has been unloaded
	 * Overloaded to allow online subsystems to cancel any outstanding http requests
	 */
	virtual void PreUnloadCallback() override;

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 * Overloaded to shut down all loaded online subsystems
	 */
	virtual void ShutdownModule() override;

	/**
	 * Override this to set whether your module is allowed to be unloaded on the fly
	 *
	 * @return	Whether the module supports shutdown separate from the rest of the engine.
	 */
	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}

	/**
	 * Override this to set whether your module would like cleanup on application shutdown
	 *
	 * @return	Whether the module supports shutdown on application exit
	 */
	virtual bool SupportsAutomaticShutdown() override
	{
		return true;
	}

private:
	// Cached instance names for efficient lookup in ParseOnlineSubsystemName
	struct FInstanceNameEntry
	{
		FName SubsystemName;
		FName InstanceName;
		FName FullPath;
	};
	mutable TMap<FName, FInstanceNameEntry> InstanceNames;
};

/** Public references to the online subsystem module pointer should use this */
typedef TSharedPtr<FOnlineSubsystemModule, ESPMode::ThreadSafe> FOnlineSubsystemModulePtr;
