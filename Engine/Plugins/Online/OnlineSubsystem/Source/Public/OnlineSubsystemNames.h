// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"


#define OSS_PLATFORM_NAME_PS4		TEXT("PSN")
#define OSS_PLATFORM_NAME_XBOX		TEXT("XBL")
#define OSS_PLATFORM_NAME_WINDOWS	TEXT("WIN")
#define OSS_PLATFORM_NAME_MAC		TEXT("MAC")
#define OSS_PLATFORM_NAME_LINUX		TEXT("LNX")
#define OSS_PLATFORM_NAME_IOS		TEXT("IOS")
#define OSS_PLATFORM_NAME_ANDROID	TEXT("AND")
#define OSS_PLATFORM_NAME_SWITCH	TEXT("SWT")
#define OSS_PLATFORM_NAME_SWITCH2	TEXT("SWT2")
#define OSS_PLATFORM_NAME_OTHER		TEXT("OTHER")



#ifndef NULL_SUBSYSTEM
extern ONLINESUBSYSTEM_API const FName NULL_SUBSYSTEM;
#endif

#ifndef GOOGLEPLAY_SUBSYSTEM
extern ONLINESUBSYSTEM_API const FName GOOGLEPLAY_SUBSYSTEM;
#endif

#ifndef IOS_SUBSYSTEM
extern ONLINESUBSYSTEM_API const FName IOS_SUBSYSTEM;
#endif

#ifndef APPLE_SUBSYSTEM
extern ONLINESUBSYSTEM_API const FName APPLE_SUBSYSTEM;
#endif

#ifndef AMAZON_SUBSYSTEM
extern ONLINESUBSYSTEM_API const FName AMAZON_SUBSYSTEM;
#endif

#ifndef GOOGLE_SUBSYSTEM
extern ONLINESUBSYSTEM_API const FName GOOGLE_SUBSYSTEM;
#endif

#ifndef FACEBOOK_SUBSYSTEM
extern ONLINESUBSYSTEM_API const FName FACEBOOK_SUBSYSTEM;
#endif

#ifndef GAMECIRCLE_SUBSYSTEM
extern ONLINESUBSYSTEM_API const FName GAMECIRCLE_SUBSYSTEM;
#endif

#ifndef STEAM_SUBSYSTEM
extern ONLINESUBSYSTEM_API const FName STEAM_SUBSYSTEM;
#endif

#ifndef PS4_SUBSYSTEM
extern ONLINESUBSYSTEM_API const FName PS4_SUBSYSTEM;
#endif

#ifndef PS4SERVER_SUBSYSTEM
extern ONLINESUBSYSTEM_API const FName PS4SERVER_SUBSYSTEM;
#endif

#ifndef PS5_SUBSYSTEM
extern ONLINESUBSYSTEM_API const FName PS5_SUBSYSTEM;
#endif

#ifndef GDK_SUBSYSTEM
extern ONLINESUBSYSTEM_API const FName GDK_SUBSYSTEM;
#endif

#ifndef THUNDERHEAD_SUBSYSTEM
extern ONLINESUBSYSTEM_API const FName THUNDERHEAD_SUBSYSTEM;
#endif

#ifndef MCP_SUBSYSTEM
extern ONLINESUBSYSTEM_API const FName MCP_SUBSYSTEM;
#endif

#ifndef TENCENT_SUBSYSTEM
extern ONLINESUBSYSTEM_API const FName TENCENT_SUBSYSTEM;
#endif

#ifndef NINTENDO_SUBSYSTEM
extern ONLINESUBSYSTEM_API const FName NINTENDO_SUBSYSTEM;
#endif

#ifndef SAMSUNG_SUBSYSTEM
extern ONLINESUBSYSTEM_API const FName SAMSUNG_SUBSYSTEM;
#endif

#ifndef ONESTORE_SUBSYSTEM
extern ONLINESUBSYSTEM_API const FName ONESTORE_SUBSYSTEM;
#endif

#ifndef EOS_SUBSYSTEM
extern ONLINESUBSYSTEM_API const FName EOS_SUBSYSTEM;
#endif
