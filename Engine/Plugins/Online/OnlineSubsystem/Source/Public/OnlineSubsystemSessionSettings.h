// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// NOTE:  Session setting names have been moved to OnlineBase in OnlineSessionNames.h
