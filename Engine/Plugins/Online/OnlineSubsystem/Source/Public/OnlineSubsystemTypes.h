// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CoreOnline.h"
#include "OnlineSubsystemNames.h" // IWYU pragma: keep


#if UE_GAME && UE_BUILD_SHIPPING
extern ONLINESUBSYSTEM_API bool IsUniqueIdLocal(const FUniqueNetId& UniqueId);
#define OSS_UNIQUEID_REDACT(UniqueId, x) ( !IsUniqueIdLocal(UniqueId) ? ((x.Len() > 10) ? (x.Left(5) + TEXT("...") + x.Right(5)) : FString(TEXT("<Redacted>"))) : x)
#endif

#ifndef OSS_UNIQUEID_REDACT
#define OSS_UNIQUEID_REDACT(UniqueId, x) (x)
#endif

#if UE_BUILD_SHIPPING
#define OSS_REDACT(x) TEXT("<Redacted>")
#else
#define OSS_REDACT(x) (x)
#endif

#define DEDICATED_SERVER_USER_INDEX 0

/**
 * Environment for the current online platform
 */
namespace EOnlineEnvironment
{
	enum Type
	{
		/** Dev environment */
		Development,
		/** Cert environment */
		Certification,
		/** Prod environment */
		Production,
		/** Not determined yet */
		Unknown
	};

	/** @return the stringified version of the enum passed in */
	ONLINESUBSYSTEM_API const TCHAR* ToString(EOnlineEnvironment::Type EnvironmentType);
}

/** Possible login states */
namespace ELoginStatus
{
	enum Type
	{
		/** Player has not logged in or chosen a local profile */
		NotLoggedIn,
		/** Player is using a local profile but is not logged in */
		UsingLocalProfile,
		/** Player has been validated by the platform specific authentication service */
		LoggedIn
	};

	/** @return the stringified version of the enum passed in */
	ONLINESUBSYSTEM_API const TCHAR* ToString(ELoginStatus::Type EnumVal);
};

/** Possible connection states */
namespace EOnlineServerConnectionStatus
{
	enum Type : uint8
	{
		/** System normal (used for default state) */
		Normal = 0,
		/** Gracefully disconnected from the online servers */
		NotConnected,
		/** Connected to the online servers just fine */
		Connected,
		/** Connection was lost for some reason */
		ConnectionDropped,
		/** Can't connect because of missing network connection */
		NoNetworkConnection,
		/** Service is temporarily unavailable */
		ServiceUnavailable,
		/** An update is required before connecting is possible */
		UpdateRequired,
		/** Servers are too busy to handle the request right now */
		ServersTooBusy,
		/** Disconnected due to duplicate login */
		DuplicateLoginDetected,
		/** Can't connect because of an invalid/unknown user */
		InvalidUser,
		/** Not authorized */
		NotAuthorized,
		/** Session has been lost on the backend */
		InvalidSession
	};

	/** @return the stringified version of the enum passed in */
	ONLINESUBSYSTEM_API const TCHAR* ToString(EOnlineServerConnectionStatus::Type EnumVal);
};

/** Possible feature privilege access levels */
namespace EFeaturePrivilegeLevel
{
	enum Type
	{
		/** Not defined for the platform service */
		Undefined,
		/** Parental controls have disabled this feature */
		Disabled,
		/** Parental controls allow this feature only with people on their friends list */
		EnabledFriendsOnly,
		/** Parental controls allow this feature everywhere */
		Enabled
	};

	/** @return the stringified version of the enum passed in */
	ONLINESUBSYSTEM_API const TCHAR* ToString(EFeaturePrivilegeLevel::Type EnumVal);
}

/** The state of an async task (read friends, read content, write cloud file, etc) request */
namespace EOnlineAsyncTaskState
{
	enum Type : int
	{
		/** The task has not been started */
		NotStarted,
		/** The task is currently being processed */
		InProgress,
		/** The task has completed successfully */
		Done,
		/** The task failed to complete */
		Failed
	};  

	/** @return the stringified version of the enum passed in */
	ONLINESUBSYSTEM_API const TCHAR* ToString(EOnlineAsyncTaskState::Type EnumVal);
}

/** The possible friend states for a friend entry */
namespace EOnlineFriendState
{
	enum Type
	{
		/** Not currently online */
		Offline,
		/** Signed in and online */
		Online,
		/** Signed in, online, and idle */
		Away,
		/** Signed in, online, and asks to be left alone */
		Busy
	};

	/** @return the stringified version of the enum passed in */
	ONLINESUBSYSTEM_API const TCHAR* ToString(EOnlineFriendState::Type EnumVal);
}

/** Leaderboard entry sort types */
namespace ELeaderboardSort
{
	enum Type
	{
		/** Don't sort at all */
		None,
		/** Sort ascending */
		Ascending,
		/** Sort descending */
		Descending
	};

	/** @return the stringified version of the enum passed in */
	ONLINESUBSYSTEM_API const TCHAR* ToString(ELeaderboardSort::Type EnumVal);
}

/** Leaderboard display format */
namespace ELeaderboardFormat
{
	enum Type
	{
		/** A raw number */
		Number,
		/** Time, in seconds */
		Seconds,
		/** Time, in milliseconds */
		Milliseconds
	};

	/** @return the stringified version of the enum passed in */
	ONLINESUBSYSTEM_API const TCHAR* ToString(ELeaderboardFormat::Type EnumVal);
}

/** How to upload leaderboard score updates */
namespace ELeaderboardUpdateMethod
{
	enum Type
	{
		/** If current leaderboard score is better than the uploaded one, keep the current one */
		KeepBest,
		/** Leaderboard score is always replaced with uploaded value */
		Force
	};

	/** @return the stringified version of the enum passed in */
	ONLINESUBSYSTEM_API const TCHAR* ToString(ELeaderboardUpdateMethod::Type EnumVal);
}

/** Enum indicating the current state of the online session (in progress, ended, etc.) */
namespace EOnlineSessionState
{
	enum Type
	{
		/** An online session has not been created yet */
		NoSession,
		/** An online session is in the process of being created */
		Creating,
		/** Session has been created but the session hasn't started (pre match lobby) */
		Pending,
		/** Session has been asked to start (may take time due to communication with backend) */
		Starting,
		/** The current session has started. Sessions with join in progress disabled are no longer joinable */
		InProgress,
		/** The session is still valid, but the session is no longer being played (post match lobby) */
		Ending,
		/** The session is closed and any stats committed */
		Ended,
		/** The session is being destroyed */
		Destroying
	};

	/** @return the stringified version of the enum passed in */
	ONLINESUBSYSTEM_API const TCHAR* ToString(EOnlineSessionState::Type EnumVal);
}

/** The types of advertisement of settings to use */
namespace EOnlineDataAdvertisementType
{
	enum Type : uint8
	{
		/** Don't advertise via the online service or QoS data */
		DontAdvertise,
		/** Advertise via the server ping data only */
		ViaPingOnly,
		/** Advertise via the online service only */
		ViaOnlineService,
		/** Advertise via the online service and via the ping data */
		ViaOnlineServiceAndPing
	};

	/** @return the stringified version of the enum passed in */
	ONLINESUBSYSTEM_API const TCHAR* ToString(EOnlineDataAdvertisementType::Type EnumVal);
}

/** The types of comparison operations for a given search query */
namespace EOnlineComparisonOp
{
	enum Type
	{
		Equals,
		NotEquals,
		GreaterThan,
		GreaterThanEquals,
		LessThan,
		LessThanEquals,
		Near,
		In,
		NotIn
	};

	/** @return the stringified version of the enum passed in */
	ONLINESUBSYSTEM_API const TCHAR* ToString(EOnlineComparisonOp::Type EnumVal);
}

/** Return codes for the GetCached functions in the various subsystems. */
namespace EOnlineCachedResult
{
	enum Type
	{
		Success, /** The requested data was found and returned successfully. */
		NotFound /** The requested data was not found in the cache, and the out parameter was not modified. */
	};

	/**
	 * @param EnumVal the enum to convert to a string
	 * @return the stringified version of the enum passed in
	 */
	ONLINESUBSYSTEM_API const TCHAR* ToString(EOnlineCachedResult::Type EnumVal);
}

/** Permissions for who can send invites to a user. */
enum class EFriendInvitePolicy : uint8
{
	/** Anyone can send a friend invite. */
	Public,
	/** Only friends of friends can send a friend invite. */
	Friends_of_Friends,
	/** No one can send a friend invite. */
	Private,
	/** Invalid enum type, may be used as a number of enumerations. */
	InvalidOrMax
};

ONLINESUBSYSTEM_API const TCHAR* LexToString(EFriendInvitePolicy EnumVal);
ONLINESUBSYSTEM_API void LexFromString(EFriendInvitePolicy& Value, const TCHAR* String);

/*
 *	Base class for anything meant to be opaque so that the data can be passed around 
 *  without consideration for the data it contains.
 *	A human readable version of the data is available via the ToString() function
 *	Otherwise, nothing but platform code should try to operate directly on the data
 */
class IOnlinePlatformData
{
protected:

	/** Hidden on purpose */
	IOnlinePlatformData()
	{
	}

	/** Hidden on purpose */
	IOnlinePlatformData(const IOnlinePlatformData& Src)
	{
	}

	/** Hidden on purpose */
	IOnlinePlatformData& operator=(const IOnlinePlatformData& Src)
	{
		return *this;
	}

	virtual ONLINESUBSYSTEM_API bool Compare(const IOnlinePlatformData& Other) const;

public:

	virtual ~IOnlinePlatformData() = default;

	/**
	 *	Comparison operator
	 */
	bool operator==(const IOnlinePlatformData& Other) const
	{
		return Other.Compare(*this);
	}

	bool operator!=(const IOnlinePlatformData& Other) const
	{
		return !(IOnlinePlatformData::operator==(Other));
	}
	
	/** 
	 * Get the raw byte representation of this opaque data
	 * This data is platform dependent and shouldn't be manipulated directly
	 *
	 * @return byte array of size GetSize()
	 */
	virtual const uint8* GetBytes() const = 0;

	/** 
	 * Get the size of the opaque data
	 *
	 * @return size in bytes of the data representation
	 */
	virtual int32 GetSize() const = 0;

	/** 
	 * Check the validity of the opaque data
	 *
	 * @return true if this is well formed data, false otherwise
	 */
	virtual bool IsValid() const = 0;

	/** 
	 * Platform specific conversion to string representation of data
	 *
	 * @return data in string form 
	 */
	virtual FString ToString() const = 0;

	/** 
	 * Get a human readable representation of the opaque data
	 * Shouldn't be used for anything other than logging/debugging
	 *
	 * @return data in string form 
	 */
	virtual FString ToDebugString() const = 0;
};

/**
 * TArray helper for IndexOfByPredicate() function
 */
struct FUniqueNetIdMatcher
{
private:
	/** Target for comparison in the TArray */
	const FUniqueNetId& UniqueIdTarget;

public:
	FUniqueNetIdMatcher(const FUniqueNetId& InUniqueIdTarget) :
		UniqueIdTarget(InUniqueIdTarget)
	{
	}

	/**
	 * Match a given unique Id against the one stored in this struct
	 *
	 * @return true if they are an exact match, false otherwise
	 */
	bool operator()(const FUniqueNetId& Candidate) const
	{
		return UniqueIdTarget == Candidate;
	}
 
	/**
	 * Match a given unique Id against the one stored in this struct
	 *
	 * @return true if they are an exact match, false otherwise
	 */
	bool operator()(const FUniqueNetIdPtr& Candidate) const
	{
		return UniqueIdTarget == *Candidate;
	}

	/**
	 * Match a given unique Id against the one stored in this struct
	 *
	 * @return true if they are an exact match, false otherwise
	 */
	bool operator()(const FUniqueNetIdRef& Candidate) const
	{
		return UniqueIdTarget == *Candidate;
	}
};

// placeholder "type" until we can make FUniqueNetIdString sufficiently abstract
extern ONLINESUBSYSTEM_API FName NAME_Unset;

using FUniqueNetIdStringRef = TSharedRef<const class FUniqueNetIdString>;
using FUniqueNetIdStringPtr = TSharedPtr<const class FUniqueNetIdString>;

/**
 * Unique net id wrapper for a string
 */
class FUniqueNetIdString : public FUniqueNetId
{
public:
	/** Holds the net id for a player */
	FString UniqueNetIdStr;

	FName Type = NAME_Unset;
	
	static FUniqueNetIdStringRef Create(const FString& InUniqueNetId, const FName InType)
	{
		return MakeShareable(new FUniqueNetIdString(InUniqueNetId, InType));
	}

	static FUniqueNetIdStringRef Create(FString&& InUniqueNetId, const FName InType)
	{
		return MakeShareable(new FUniqueNetIdString(MoveTemp(InUniqueNetId), InType));
	}

	static FUniqueNetIdStringRef& EmptyId()
	{
		static FUniqueNetIdStringRef EmptyId(Create(FString(), NAME_Unset));
		return EmptyId;
	}

	virtual ~FUniqueNetIdString() = default;

	// IOnlinePlatformData

	virtual FName GetType() const override
	{
		return Type;
	}

	virtual const uint8* GetBytes() const override
	{
		return (const uint8*)UniqueNetIdStr.GetCharArray().GetData();
	}

	virtual int32 GetSize() const override
	{
		return sizeof(UniqueNetIdStr[0]) * UniqueNetIdStr.GetCharArray().Num();
	}

	virtual bool IsValid() const override
	{
		return !UniqueNetIdStr.IsEmpty();
	}

	virtual FString ToString() const override
	{
		return UniqueNetIdStr;
	}

	virtual FString ToDebugString() const override
	{
		if (IsValid())
		{
			return OSS_UNIQUEID_REDACT(*this, UniqueNetIdStr);
		}
		else
		{
			return TEXT("INVALID");
		}
	}

	virtual uint32 GetTypeHash() const override
	{
		return GetTypeHashHelper(UniqueNetIdStr);
	}

protected:
	FUniqueNetIdString() = default;

	/**
	 * Constructs this object with the specified net id
	 *
	 * @param InUniqueNetId the id to set ours to
	 */
	explicit FUniqueNetIdString(const FString& InUniqueNetId)
		: UniqueNetIdStr(InUniqueNetId)
		, Type(NAME_Unset)
	{
	}

	/**
	 * Constructs this object with the specified net id
	 *
	 * @param InUniqueNetId the id to set ours to
	 */
	explicit FUniqueNetIdString(FString&& InUniqueNetId)
		: UniqueNetIdStr(MoveTemp(InUniqueNetId))
		, Type(NAME_Unset)
	{
	}

	/**
	 * Constructs this object with the string value of the specified net id
	 *
	 * @param Src the id to copy
	 */
	explicit FUniqueNetIdString(const FUniqueNetId& Src)
		: UniqueNetIdStr(Src.ToString())
		, Type(Src.GetType())
	{
	}

	/**
	* don.eubanks - Including a constructor that allows for type passing to make transitioning easier, if we determine we want to abstract-ify this class, this constructor will be removed
	*/
	FUniqueNetIdString(const FString& InUniqueNetId, const FName InType)
		: UniqueNetIdStr(InUniqueNetId)
		, Type(InType)
	{
	}

	FUniqueNetIdString(FString&& InUniqueNetId, const FName InType)
		: UniqueNetIdStr(MoveTemp(InUniqueNetId))
		, Type(InType)
	{
	}
};


#define TEMP_UNIQUENETIDSTRING_SUBCLASS(SUBCLASSNAME, TYPE) \
using SUBCLASSNAME##Ptr = TSharedPtr<const class SUBCLASSNAME>; \
using SUBCLASSNAME##Ref = TSharedRef<const class SUBCLASSNAME>; \
class SUBCLASSNAME : public FUniqueNetIdString \
{ \
public: \
	template<typename... TArgs> \
	static SUBCLASSNAME##Ref Create(TArgs&&... Args) \
	{ \
		return MakeShareable(new SUBCLASSNAME(Forward<TArgs>(Args)...)); \
	} \
	static SUBCLASSNAME##Ref Cast(const FUniqueNetIdRef& InNetId) \
	{ \
		check(InNetId->GetType() == TYPE); \
		return StaticCastSharedRef<const SUBCLASSNAME>(InNetId); \
	} \
	static SUBCLASSNAME##Ptr Cast(const FUniqueNetIdPtr& InNetId) \
	{ \
		if(InNetId.IsValid()) \
		{ \
			check(InNetId->GetType() == TYPE); \
			return StaticCastSharedPtr<const SUBCLASSNAME>(InNetId); \
		} \
		return nullptr; \
	} \
	static const SUBCLASSNAME& Cast(const FUniqueNetId& InNetId) \
	{ \
		check(InNetId.GetType() == TYPE); \
		return static_cast<const SUBCLASSNAME&>(InNetId); \
	} \
	SUBCLASSNAME##Ref AsShared() const \
	{ \
		return StaticCastSharedRef<const SUBCLASSNAME>(FUniqueNetId::AsShared()); \
	} \
	friend uint32 GetTypeHash(const SUBCLASSNAME& A) \
	{ \
		return GetTypeHashHelper(A.UniqueNetIdStr); \
	} \
	static const SUBCLASSNAME##Ref& EmptyId() \
	{ \
		static const SUBCLASSNAME##Ref EmptyId(Create()); \
		return EmptyId; \
	} \
protected: \
	SUBCLASSNAME() \
		: FUniqueNetIdString() \
	{ \
		Type = TYPE; \
	} \
	explicit SUBCLASSNAME(const FString& InUniqueNetId) \
		: FUniqueNetIdString(InUniqueNetId, TYPE) \
	{ \
	} \
	explicit SUBCLASSNAME(FString&& InUniqueNetId) \
		: FUniqueNetIdString(MoveTemp(InUniqueNetId), TYPE) \
	{ \
	} \
	explicit SUBCLASSNAME(const FUniqueNetId& Src) \
		: FUniqueNetIdString(Src) \
	{ \
		check(GetType() == TYPE); \
	} \
};

/** 
 * Abstraction of a profile service shared file handle
 * The class is meant to be opaque (see IOnlinePlatformData)
 */
class FSharedContentHandle : public IOnlinePlatformData
{
protected:

	/** Hidden on purpose */
	FSharedContentHandle()
	{
	}

	/** Hidden on purpose */
	FSharedContentHandle(const FSharedContentHandle& Src)
	{
	}

	/** Hidden on purpose */
	FSharedContentHandle& operator=(const FSharedContentHandle& Src)
	{
		return *this;
	}

public:

	virtual ~FSharedContentHandle() = default;
};

/** 
 * Abstraction of a session's platform specific info
 * The class is meant to be opaque (see IOnlinePlatformData)
 */
class FOnlineSessionInfo : public IOnlinePlatformData
{
protected:

	/** Hidden on purpose */
	FOnlineSessionInfo()
	{
	}

	/** Hidden on purpose */
	FOnlineSessionInfo(const FOnlineSessionInfo& Src)
	{
	}

	/** Hidden on purpose */
	FOnlineSessionInfo& operator=(const FOnlineSessionInfo& Src)
	{
		return *this;
	}

public:

	virtual ~FOnlineSessionInfo() = default;

	/**
	 * Get the session id associated with this session
	 *
	 * @return session id for this session
	 */
	virtual const FUniqueNetId& GetSessionId() const = 0;
};

/**
 * Paging info needed for a request that can return paged results
 */
class FPagedQuery
{
public:
	FPagedQuery(int32 InStart = 0, int32 InCount = -1)
		: Start(InStart)
		, Count(InCount)
	{}

	/** @return true if valid range */
	bool IsValidRange() const
	{
		return Start >= 0 && Count >= 0;
	}

	bool operator==(const FPagedQuery& Other) const
	{
		return Other.Start == Start && Other.Count == Count;
	}

	/** first entry to fetch */
	int32 Start;
	/** total entries to fetch. -1 means ALL */
	int32 Count;
};

/**
 * Info for a response with paged results
 */
class FOnlinePagedResult
{
public:
	FOnlinePagedResult()
		: Start(0)
		, Count(0)
		, Total(0)
	{}
	virtual ~FOnlinePagedResult() = default;

	/** Starting entry */
	int32 Start;
	/** Number returned */
	int32 Count;
	/** Total available */
	int32 Total;
};

/** Locale and country code */
class FRegionInfo
{
public:

	FRegionInfo(const FString& InCountry = FString(), const FString& InLocale = FString())
		: Country(InCountry)
		, Locale(InLocale)
	{}

	/** country code used for configuring things like currency/pricing specific to a country. eg. US */
	FString Country;
	/** local code used to select the localization language. eg. en_US */
	FString Locale;
};

/** Holds metadata about a given downloadable file */
struct FCloudFileHeader
{
	/** Hash value, if applicable, of the given file contents */
	FString Hash;
	/** The hash algorithm used to sign this file */
	FName HashType;
	/** Filename as downloaded */
	FString DLName;
	/** Logical filename, maps to the downloaded filename */
	FString FileName;
	/** File size */
	int32 FileSize;
	/** The full URL to download the file if it is stored in a CDN or separate host site */
	FString URL;
	/** The chunk id this file represents */
	uint32 ChunkID;
	/** Pointers to externally-accessible representations of this file */
	TMap<FString, FString> ExternalStorageIds;

	/** Constructors */
	FCloudFileHeader() :
		FileSize(0),
		ChunkID(0)
	{}

	FCloudFileHeader(const FString& InFileName, const FString& InDLName, int32 InFileSize) :
		DLName(InDLName),
		FileName(InFileName),
		FileSize(InFileSize),
		ChunkID(0)
	{}

	bool operator==(const FCloudFileHeader& Other) const
	{
		return FileSize == Other.FileSize &&
			Hash == Other.Hash &&
			HashType == Other.HashType &&
			DLName == Other.DLName &&
			FileName == Other.FileName &&
			URL == Other.URL &&
			ChunkID == Other.ChunkID &&
			ExternalStorageIds.OrderIndependentCompareEqual(Other.ExternalStorageIds);
	}

	bool operator<(const FCloudFileHeader& Other) const
	{
		return FileName.Compare(Other.FileName, ESearchCase::IgnoreCase) < 0;
	}
};

/** Holds the data used in downloading a file asynchronously from the online service */
struct FCloudFile
{
	/** The name of the file as requested */
	FString FileName;
	/** The async state the file download is in */
	EOnlineAsyncTaskState::Type AsyncState;
	/** The buffer of data for the file */
	TArray<uint8> Data;

	/** Constructors */
	FCloudFile() :
		AsyncState(EOnlineAsyncTaskState::NotStarted)
	{
	}

	FCloudFile(const FString& InFileName) :
		FileName(InFileName),
		AsyncState(EOnlineAsyncTaskState::NotStarted)
	{
	}

	virtual ~FCloudFile() = default;
};

/**
 * User attribution constants for GetUserAttribute()
 */
#define USER_ATTR_REALNAME TEXT("realName")
#define USER_ATTR_DISPLAYNAME TEXT("displayName")
#define USER_ATTR_PREFERRED_DISPLAYNAME TEXT("prefDisplayName")
#define USER_ATTR_ID TEXT("id")
#define USER_ATTR_EMAIL TEXT("email")
#define USER_ATTR_ALIAS TEXT("alias")

/**
 * Base for all online user info
 */
class FOnlineUser
{
public:
	/**
	 * destructor
	 */
	virtual ~FOnlineUser() = default;

	/** 
	 * @return Id associated with the user account provided by the online service during registration 
	 */
	virtual FUniqueNetIdRef GetUserId() const = 0;
	/**
	 * @return the real name for the user if known
	 */
	virtual FString GetRealName() const = 0;
	/**
	 * @return the nickname of the user if known
	 */
	virtual FString GetDisplayName(const FString& Platform = FString()) const = 0;
	/** 
	 * @return Any additional user data associated with a registered user
	 */
	virtual bool GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const = 0;

	/**
	 * @return Whether a local attribute for a user was successfully set.
	 */
	virtual bool SetUserLocalAttribute(const FString& AttrName, const FString& InAttrValue) { return false; /* Not Implemented by default */};
};

/**
 * Auth attribution constants for GetAuthAttribute()
 */
#define AUTH_ATTR_REFRESH_TOKEN TEXT("refresh_token")
#define AUTH_ATTR_ID_TOKEN TEXT("id_token")
#define AUTH_ATTR_AUTHORIZATION_CODE TEXT("authorization_code")

/**
 * User account information returned via IOnlineIdentity interface
 */
class FUserOnlineAccount : public FOnlineUser
{
public:
	/**
	 * @return Access token which is provided to user once authenticated by the online service
	 */
	virtual FString GetAccessToken() const = 0;
	/**
	 * Tests if token has exceeded its time-to-live, when applicable.  Default is true, i.e. always expire.
	 * @return True, if access token has expired.
	 */
	virtual bool HasAccessTokenExpired(const FDateTime& Time) const { return true; }
	/** 
	 * @return Any additional auth data associated with a registered user
	 */
	virtual bool GetAuthAttribute(const FString& AttrName, FString& OutAttrValue) const = 0;
	/** 
	 * @return True, if the data has been changed
	 */
	virtual bool SetUserAttribute(const FString& AttrName, const FString& AttrValue) = 0;
};

/** 
 * Friend list invite states 
 */
namespace EInviteStatus
{
	enum Type
	{
		/** unknown state */
		Unknown,
		/** Friend has accepted the invite */
		Accepted,
		/** Friend has sent player an invite, but it has not been accepted/rejected */
		PendingInbound,
		/** Player has sent friend an invite, but it has not been accepted/rejected */
		PendingOutbound,
		/** Player has been blocked */
		Blocked,
		/** Suggested friend */
		Suggested
	};

	/** 
	 * @return the stringified version of the enum passed in 
	 */
	ONLINESUBSYSTEM_API const TCHAR* ToString(EInviteStatus::Type EnumVal);
};

/**
 * Friend user info returned via IOnlineFriends interface
 */
class FOnlineFriend : public FOnlineUser
{
public:

	/**
	 * @return the current invite status of a friend wrt to user that queried
	 */
	virtual EInviteStatus::Type GetInviteStatus() const = 0;
	
	/**
	 * @return presence info for an online friend
	 */
	virtual const class FOnlineUserPresence& GetPresence() const = 0;
};

/**
 * Recent player user info returned via IOnlineFriends interface
 */
class FOnlineRecentPlayer : public FOnlineUser
{
public:
	
	/**
	 * @return last time the player was seen by the current user
	 */
	virtual FDateTime GetLastSeen() const = 0;
};

/**
 * Blocked user info returned via IOnlineFriends interface
 */
class FOnlineBlockedPlayer : public FOnlineUser
{
};

/** Valid states for user facing permissions */
enum class EOnlineSharingPermissionState : uint8
{
	/** Permission has not been requested yet */
	Unknown = 0,
	/** Permission has been requested but declined by the user */
	Declined = 1,
	/** Permission has been granted by the user */
	Granted = 2,
};

/**
 * First 16 bits are reading permissions
 * Second 16 bits are writing/publishing permissions
 */
enum class EOnlineSharingCategory : uint32
{
	None = 0x00,
	// Read access to posts on the users feeds
	ReadPosts = 0x01,
	// Read access for a users friend information, and all data about those friends. e.g. Friends List and Individual Friends Birthday
	Friends = 0x02,
	// Read access to a user's email address
	Email = 0x04,
	// Read access to a users mailbox
	Mailbox = 0x08,
	// Read the current online status of a user
	OnlineStatus = 0x10,
	// Read a users profile information, e.g. Users Birthday
	ProfileInfo = 0x20,
	// Read information about the users locations and location history
	LocationInfo = 0x40,

	ReadPermissionMask = 0x0000FFFF,
	DefaultRead = ProfileInfo | LocationInfo,

	// Permission to post to a users news feed
	SubmitPosts = 0x010000,
	// Permission to manage a users friends list. Add/Remove contacts
	ManageFriends = 0x020000,
	// Manage a users account settings, such as pages they subscribe to, or which notifications they receive
	AccountAdmin = 0x040000,
	// Manage a users events. This features the capacity to create events as well as respond to events.
	Events = 0x080000,
	
	PublishPermissionMask = 0xFFFF0000,
	DefaultPublish = None
};

ENUM_CLASS_FLAGS(EOnlineSharingCategory);

ONLINESUBSYSTEM_API const TCHAR* ToString(EOnlineSharingCategory CategoryType);

/** Privacy permissions used for Online Status updates */
enum class EOnlineStatusUpdatePrivacy : uint8
{
	// Post will only be visible to the user alone
	OnlyMe,
	// Post will only be visible to the user and the users friends
	OnlyFriends,
	// Post will be visible to everyone
	Everyone,
};

ONLINESUBSYSTEM_API const TCHAR* ToString(EOnlineStatusUpdatePrivacy PrivacyType);

/**
 * unique identifier for notification transports
 */
typedef FString FNotificationTransportId;

/**
 * Id of a party instance
 */
class FOnlinePartyId : public IOnlinePlatformData, public TSharedFromThis<FOnlinePartyId>
{
protected:
	/** Hidden on purpose */
	FOnlinePartyId()
	{
	}

	/** Hidden on purpose */
	FOnlinePartyId(const FOnlinePartyId& Src)
	{
	}

	/** Hidden on purpose */
	FOnlinePartyId& operator=(const FOnlinePartyId& Src)
	{
		return *this;
	}

public:
	virtual ~FOnlinePartyId() = default;
};

ONLINESUBSYSTEM_API uint32 GetTypeHash(const FOnlinePartyId& Value);

template <typename ValueType>
struct TOnlinePartyIdMapKeyFuncs : public TDefaultMapKeyFuncs<TSharedRef<const FOnlinePartyId>, ValueType, false>
{
	static TSharedRef<const FOnlinePartyId>	GetSetKey(TPair<TSharedRef<const FOnlinePartyId>, ValueType> const& Element) { return Element.Key; }
	static uint32							GetKeyHash(TSharedRef<const FOnlinePartyId> const& Key) {	return GetTypeHash(*Key); }
	static bool								Matches(TSharedRef<const FOnlinePartyId> const& A, TSharedRef<const FOnlinePartyId> const& B) { return (A == B) || (*A == *B); }
};

template <typename ValueType>
using TOnlinePartyIdMap = TMap<TSharedRef<const FOnlinePartyId>, ValueType, FDefaultSetAllocator, TOnlinePartyIdMapKeyFuncs<ValueType>>;

/**
 * Id of a party's type
 */
class FOnlinePartyTypeId
{
public:
	typedef uint32 TInternalType;
	explicit FOnlinePartyTypeId(const TInternalType InValue = 0) : Value(InValue) {}

	bool operator==(const FOnlinePartyTypeId Rhs) const { return Value == Rhs.Value; }
	bool operator!=(const FOnlinePartyTypeId Rhs) const { return Value != Rhs.Value; }

	TInternalType GetValue() const { return Value; }

	friend bool IsValid(const FOnlinePartyTypeId Value);

protected:
	TInternalType Value;

	friend inline uint32 GetTypeHash(const FOnlinePartyTypeId Id)
	{
		return Id.GetValue();
	}
};

inline bool IsValid(const FOnlinePartyTypeId Id) { return Id.GetValue() != 0; }

class FOnlineFriendSettingsSourceData
{
public:

	virtual ~FOnlineFriendSettingsSourceData() = default;

	/** Constructors */
	FOnlineFriendSettingsSourceData() :
		bNeverShowAgain(false)
	{
	}

	explicit FOnlineFriendSettingsSourceData(
		bool bInNeverShowAgain)
		: bNeverShowAgain(bInNeverShowAgain)
	{
	}

	bool bNeverShowAgain;
};

/**
 * Parse an array of strings in the format (Key=Value) into an array of pairs of those keys and values
 *
 * @param InEntries array of strings in the format (Key=Value)
 * @param OutPairs the split key/value pairs
 */
ONLINESUBSYSTEM_API void ParseOnlineSubsystemConfigPairs(TArrayView<const FString> InEntries, TArray<TPair<FString, FString>>& OutPairs);
