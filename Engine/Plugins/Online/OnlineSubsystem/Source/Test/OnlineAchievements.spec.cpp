// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/OnlineAchievementsInterface.h"
#include "Online.h"
#include "Utils/OnlineErrors.data.h"
#include "Utils/OnlineTestCommon.h"

BEGIN_DEFINE_SPEC(FOnlineAchievementsSpec, "OnlineAchievementsInterface", EAutomationTestFlags::EngineFilter | EAutomationTestFlags_ApplicationContextMask)

IOnlineIdentityPtr OnlineIdentity;
IOnlineAchievementsPtr OnlineAchievements;

FOnlineAccountCredentials AccountCredentials;

FOnlineTestCommon CommonUtils;

// Delegate Handles
FDelegateHandle OnLoginCompleteDelegateHandle;
FDelegateHandle OnLogoutCompleteDelegateHandle;

END_DEFINE_SPEC(FOnlineAchievementsSpec)

void FOnlineAchievementsSpec::Define()
{
	TArray<FName> Subsystems = FOnlineTestCommon::GetEnabledTestSubsystems();

	for (int32 Index = 0; Index < Subsystems.Num(); Index++)
	{
		FName SubsystemType = Subsystems[Index];

		Describe(SubsystemType.ToString(), [this, SubsystemType]()
		{
			BeforeEach([this, SubsystemType]()
			{
				CommonUtils = FOnlineTestCommon();
				AccountCredentials = FOnlineTestCommon::GetSubsystemTestAccountCredentials(SubsystemType);

				OnlineIdentity = Online::GetIdentityInterface(SubsystemType);
				OnlineAchievements = Online::GetAchievementsInterface(SubsystemType);

				// If OnlineAchievements is not valid, the following test, including all other nested BeforeEaches, will not run
				if (!OnlineIdentity.IsValid())
				{
					UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("OSS Automation: Failed to load OnlineIdentity Interface for %s"), *SubsystemType.ToString());
				}

				if (!OnlineAchievements.IsValid())
				{
					UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("OSS Automation: Failed to load OnlineAchievements Interface for %s"), *SubsystemType.ToString());
				}

			});

			// TODO: Still need to validate some tests for functionality
			Describe("Online Achievements", [this, SubsystemType]()
			{
				Describe("WriteAchievements", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						CommonUtils.ResetTestAccountAchievements(OnlineIdentity, OnlineAchievements, TestDone);
					});

					LatentIt("When calling WriteAchievements with a valid PlayerId and WriteObject, this subsystem writes achievements to the server", EAsyncExecution::ThreadPool, [this](const FDoneDelegate TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->GetUniquePlayerId(0);

							if (TestAccountId.IsValid())
							{
								OnlineAchievements->QueryAchievements(*TestAccountId, FOnQueryAchievementsCompleteDelegate::CreateLambda([this, TestAccountId, TestDone](const FUniqueNetId& QueryAchievementsPlayerId, const bool bQueryAchievementsWasSuccessful)
								{
									TArray<FOnlineAchievement> PlayerAchievements;
									OnlineAchievements->GetCachedAchievements(QueryAchievementsPlayerId, PlayerAchievements);

									if (PlayerAchievements.Num() > 0)
									{
										FString TestAchievement = PlayerAchievements[0].Id;

										FOnlineAchievement SomeAchievement;
										OnlineAchievements->GetCachedAchievement(*TestAccountId, TestAchievement, SomeAchievement);

										TestEqual("Verify that SomeAchievement.Id is: TestAchievement", SomeAchievement.Id == TestAchievement, true);
										TestEqual("Verify that SomeAchievement.Progress is: 0", SomeAchievement.Progress == 0, true);

										FOnlineAchievementsWritePtr AchievementWriteObject = MakeShareable(new FOnlineAchievementsWrite());
										FOnlineAchievementsWriteRef AchievementWriter = AchievementWriteObject.ToSharedRef();
										AchievementWriteObject->SetFloatStat(TestAchievement, 1.0f);

										OnlineAchievements->WriteAchievements(*TestAccountId, AchievementWriter, FOnAchievementsWrittenDelegate::CreateLambda([this, &SomeAchievement, AchievementWriteObject, TestAccountId, TestAchievement, TestDone](const FUniqueNetId& WriteAchievementsPlayerId, bool bWriteAchievementsWasSuccessful)
										{
											TestEqual("Verify that bWriteAchievementsWasSuccessful returns as: True", bWriteAchievementsWasSuccessful, true);
											TestEqual("Verify that AchievementWriteObject->WriteState returns as: EOnlineAsyncTaskState::Type::Done", AchievementWriteObject->WriteState == EOnlineAsyncTaskState::Type::Done, true);

											OnlineAchievements->QueryAchievements(*TestAccountId, FOnQueryAchievementsCompleteDelegate::CreateLambda([this, &SomeAchievement, TestAccountId, TestAchievement, TestDone](const FUniqueNetId& SecondQueryAchievementsPlayerId, const bool bSecondQueryAchievementsWasSuccessful)
											{
												OnlineAchievements->GetCachedAchievement(*TestAccountId, TestAchievement, SomeAchievement);

												TestEqual("Verify that SomeAchievement.Id is: TestAchievement", SomeAchievement.Id == TestAchievement, true);
												TestEqual("Verify that SomeAchievement.Progress is: 100", SomeAchievement.Progress == 100, true);

												TestDone.Execute();
											}));
										}));
									}
									else
									{
										UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("OSS Automation: PlayerAchievements array is empty after a call to GetCachedAchievements. No Achievements found for this subsystem."));
										TestDone.Execute();
									}
								}));
							}
							else
							{
								UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					// TODO: Invalid WriteObject? Or Empty WriteObject? WriteAchievements doesn't care about an empty WriteObject
					LatentIt("When calling WriteAchievements with a valid PlayerId but an invalid WriteObject, this subsystem does not write achievements to the server", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->GetUniquePlayerId(0);

							if (TestAccountId.IsValid())
							{
								OnlineAchievements->QueryAchievements(*TestAccountId, FOnQueryAchievementsCompleteDelegate::CreateLambda([this, TestAccountId, TestDone](const FUniqueNetId& QueryAchievementsPlayerId, const bool bQueryAchievementsWasSuccessful)
								{
									TArray<FOnlineAchievement> PlayerAchievements;
									OnlineAchievements->GetCachedAchievements(QueryAchievementsPlayerId, PlayerAchievements);

									if (PlayerAchievements.Num() > 0)
									{
										FString TestAchievement = PlayerAchievements[0].Id;

										FOnlineAchievement SomeAchievement;
										OnlineAchievements->GetCachedAchievement(*TestAccountId, TestAchievement, SomeAchievement);

										TestEqual("Verify that SomeAchievement.Id is: TestAchievement", SomeAchievement.Id == TestAchievement, true);
										TestEqual("Verify that SomeAchievement.Progress is: 0", SomeAchievement.Progress == 0, true);

										FOnlineAchievementsWritePtr AchievementWriteObject = MakeShareable(new FOnlineAchievementsWrite());
										FOnlineAchievementsWriteRef AchievementWriter = AchievementWriteObject.ToSharedRef();

										OnlineAchievements->WriteAchievements(*TestAccountId, AchievementWriter, FOnAchievementsWrittenDelegate::CreateLambda([this, &SomeAchievement, AchievementWriteObject, TestAccountId, TestAchievement, TestDone](const FUniqueNetId& WriteAchievements, bool bWriteAchievementsWasSuccessful)
										{
											TestEqual("Verify that bWriteAchievementsWasSuccessful returns as: False", bWriteAchievementsWasSuccessful, false);
											TestEqual("Verify that AchievementWriteObject->WriteState returns as: EOnlineAsyncTaskState::Type::Failed", AchievementWriteObject->WriteState == EOnlineAsyncTaskState::Type::Failed, true);

											OnlineAchievements->QueryAchievements(*TestAccountId, FOnQueryAchievementsCompleteDelegate::CreateLambda([this, &SomeAchievement, TestAccountId, TestAchievement, TestDone](const FUniqueNetId& SecondQueryAchievementsPlayerId, const bool bSecondQueryAchievementsWasSuccessful)
											{
												OnlineAchievements->GetCachedAchievement(*TestAccountId, TestAchievement, SomeAchievement);

												TestEqual("Verify that SomeAchievement.Id is: TestAchievement", SomeAchievement.Id == TestAchievement, true);
												TestEqual("Verify that SomeAchievement.Progress is: 0", SomeAchievement.Progress == 0, true);

												TestDone.Execute();
											}));
										}));
									}
									else
									{
										UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("OSS Automation: PlayerAchievements array is empty after a call to GetCachedAchievements. No Achievements found for this subsystem."));
										TestDone.Execute();
									}
								}));
							}
							else
							{
								UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling WriteAchievements with a valid WriteObject but an invalid PlayerId, this subsystem does not write achievements to the server", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_NONLOCALPLAYER, EAutomationExpectedErrorFlags::Contains, 0);

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->GetUniquePlayerId(0);

							if (TestAccountId.IsValid())
							{
								OnlineAchievements->QueryAchievements(*TestAccountId, FOnQueryAchievementsCompleteDelegate::CreateLambda([this, TestAccountId, TestDone](const FUniqueNetId& QueryAchievementsPlayerId, const bool bQueryAchievementsWasSuccessful)
								{
									TArray<FOnlineAchievement> PlayerAchievements;
									OnlineAchievements->GetCachedAchievements(QueryAchievementsPlayerId, PlayerAchievements);

									if (PlayerAchievements.Num() > 0)
									{
										FString TestAchievement = PlayerAchievements[0].Id;

										FOnlineAchievement SomeAchievement;
										OnlineAchievements->GetCachedAchievement(*TestAccountId, TestAchievement, SomeAchievement);

										TestEqual("Verify that SomeAchievement.Id is: TestAchievement", SomeAchievement.Id == TestAchievement, true);
										TestEqual("Verify that SomeAchievement.Progress is: 0", SomeAchievement.Progress == 0, true);

										FOnlineAchievementsWritePtr AchievementWriteObject = MakeShareable(new FOnlineAchievementsWrite());
										FOnlineAchievementsWriteRef AchievementWriter = AchievementWriteObject.ToSharedRef();

										FUniqueNetIdPtr BadAccountId = OnlineIdentity->CreateUniquePlayerId(TEXT("0123456789"));

										OnlineAchievements->WriteAchievements(*BadAccountId, AchievementWriter, FOnAchievementsWrittenDelegate::CreateLambda([this, &SomeAchievement, AchievementWriteObject, TestAccountId, TestAchievement, TestDone](const FUniqueNetId& WriteAchievements, bool bWriteAchievementsWasSuccessful)
										{
											TestEqual("Verify that bWriteAchievementsWasSuccessful returns as: False", bWriteAchievementsWasSuccessful, false);
											TestEqual("Verify that AchievementWriteObject->WriteState returns as: EOnlineAsyncTaskState::Type::Failed", AchievementWriteObject->WriteState == EOnlineAsyncTaskState::Type::Failed, true);

											OnlineAchievements->QueryAchievements(*TestAccountId, FOnQueryAchievementsCompleteDelegate::CreateLambda([this, &SomeAchievement, TestAccountId, TestAchievement, TestDone](const FUniqueNetId& SecondQueryAchievementsPlayerId, const bool bSecondQueryAchievementsWasSuccessful)
											{
												OnlineAchievements->GetCachedAchievement(*TestAccountId, TestAchievement, SomeAchievement);

												TestEqual("Verify that SomeAchievement.Id is: TestAchievement", SomeAchievement.Id == TestAchievement, true);
												TestEqual("Verify that SomeAchievement.Progress is: 0", SomeAchievement.Progress == 0, true);

												TestDone.Execute();
											}));
										}));
									}
									else
									{
										UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("OSS Automation: PlayerAchievements array is empty after a call to GetCachedAchievements. No Achievements found for this subsystem."));
										TestDone.Execute();
									}
								}));
							}
							else
							{
								UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});
				});

				Describe("QueryAchievements", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						CommonUtils.AddAchievementToTestAccount(OnlineIdentity, OnlineAchievements, TestDone);
					});

					LatentIt("When calling QueryAchievements with a valid PlayerId, this subsystem caches that player's achievement information", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->GetUniquePlayerId(0);

							if (TestAccountId.IsValid())
							{
								OnlineAchievements->QueryAchievements(*TestAccountId, FOnQueryAchievementsCompleteDelegate::CreateLambda([this, TestAccountId, TestDone](const FUniqueNetId& QueryAchievementsPlayerId, const bool bQueryAchievementsWasSuccessful)
								{
									TestEqual("Verify that QueryAchievementsPlayerId is the same as TestAccountId", QueryAchievementsPlayerId == *TestAccountId, true);
									TestEqual("Verify that bQueryAchievementsWasSuccessful returns as: True", bQueryAchievementsWasSuccessful, true);

									TArray<FOnlineAchievement> PlayerAchievements;
									OnlineAchievements->GetCachedAchievements(QueryAchievementsPlayerId, PlayerAchievements);

									if (PlayerAchievements.Num() > 0)
									{
										TestEqual("Verify that PlayerAchievements[0].Progress is: 100", PlayerAchievements[0].Progress == 100, true);
										TestDone.Execute();
									}
									else
									{
										UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("OSS Automation: PlayerAchievements array is empty after a call to GetCachedAchievements. No Achievements found for this subsystem."));
										TestDone.Execute();
									}
								}));
							}
							else
							{
								UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					// TODO: Cached achievement progress returns on Steam with bad UserId in query? Might be residual since can't really logout. No way to clear cache?
					LatentIt("When calling QueryAchievements with an invalid PlayerId, this subsystem does not cache that player's achievement information", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FUniqueNetIdPtr BadAccountId = OnlineIdentity->CreateUniquePlayerId(TEXT("0123456789"));

							if (BadAccountId.IsValid())
							{
								OnlineAchievements->QueryAchievements(*BadAccountId, FOnQueryAchievementsCompleteDelegate::CreateLambda([this, BadAccountId, TestDone](const FUniqueNetId& QueryAchievementsPlayerId, const bool bQueryAchievementsWasSuccessful)
								{
									TestEqual("Verify that QueryAchievementsPlayerId is the same as TestAccountId", QueryAchievementsPlayerId == *BadAccountId, true);
									TestEqual("Verify that bQueryAchievementsWasSuccessful returns as: False", bQueryAchievementsWasSuccessful, false);

									TArray<FOnlineAchievement> PlayerAchievements;
									OnlineAchievements->GetCachedAchievements(QueryAchievementsPlayerId, PlayerAchievements);

									if (PlayerAchievements.Num() > 0)
									{
										TestEqual("Verify that PlayerAchievements[0].Progress is: 0", PlayerAchievements[0].Progress == 0, true);
										TestDone.Execute();
									}
									else
									{
										UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("OSS Automation: PlayerAchievements array is empty after a call to GetCachedAchievements. No Achievements found for this subsystem."));
										TestDone.Execute();
									}
								}));
							}
							else
							{
								UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("OSS Automation: IsValid() check on BadAccountId failed after a call to OnlineIdentity->CreateUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentAfterEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						CommonUtils.ResetTestAccountAchievements(OnlineIdentity, OnlineAchievements, TestDone);
					});
				});

				Describe("QueryAchievementDescriptions", [this, SubsystemType]()
				{
					LatentIt("When calling QueryAchievementDescriptions with a valid PlayerId, this subsystem caches those achievement descriptions", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->GetUniquePlayerId(0);

							if (TestAccountId.IsValid())
							{
								OnlineAchievements->QueryAchievements(*TestAccountId, FOnQueryAchievementsCompleteDelegate::CreateLambda([this, TestAccountId, TestDone](const FUniqueNetId& QueryAchievementsPlayerId, const bool bQueryAchievementsWasSuccessful)
								{
									TArray<FOnlineAchievement> PlayerAchievements;
									OnlineAchievements->GetCachedAchievements(QueryAchievementsPlayerId, PlayerAchievements);

									if (PlayerAchievements.Num() > 0)
									{
										OnlineAchievements->QueryAchievementDescriptions(*TestAccountId, FOnQueryAchievementsCompleteDelegate::CreateLambda([this, PlayerAchievements, TestAccountId, TestDone](const FUniqueNetId& QueryAchievementDescriptionsPlayerId, const bool bQueryAchievementDescriptionsWasSuccessful)
										{
											TestEqual("Verify that QueryAchievementDescriptionsPlayerId is the same as TestAccountId", QueryAchievementDescriptionsPlayerId == *TestAccountId, true);
											TestEqual("Verify that bQueryAchievementDescriptionsWasSuccessful returns as: True", bQueryAchievementDescriptionsWasSuccessful, true);

											FOnlineAchievementDesc AchievementDescription;

											OnlineAchievements->GetCachedAchievementDescription(PlayerAchievements[0].Id, AchievementDescription);

											UE_LOG_ONLINE_ACHIEVEMENTS(Display, TEXT("OSS Automation: Found Achievement Description: %s"), *AchievementDescription.ToDebugString());
											TestEqual("Verify that AchievementDescription.Title is populated", AchievementDescription.Title.IsEmpty(), false);
											TestEqual("Verify that AchievementDescription.LockedDesc is populated", AchievementDescription.LockedDesc.IsEmpty(), false);
											TestEqual("Verify that AchievementDescription.UnlockedDesc is populated", AchievementDescription.UnlockedDesc.IsEmpty(), false);

											TestDone.Execute();
										}));
									}
									else
									{
										UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("OSS Automation: PlayerAchievements array is empty after a call to GetCachedAchievements. No Achievements found for this subsystem."));
										TestDone.Execute();
									}
								}));
							}
							else
							{
								UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					// TODO: Achievement descriptions cached anyway with bad UserId
					LatentIt("When calling QueryAchievementDescriptions with an invalid PlayerId, this subsystem does not cache those achievement descriptions", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->GetUniquePlayerId(0);

							if (TestAccountId.IsValid())
							{
								OnlineAchievements->QueryAchievements(*TestAccountId, FOnQueryAchievementsCompleteDelegate::CreateLambda([this, TestAccountId, TestDone](const FUniqueNetId& QueryAchievementsPlayerId, const bool bQueryAchievementsWasSuccessful)
								{
									TArray<FOnlineAchievement> PlayerAchievements;
									OnlineAchievements->GetCachedAchievements(QueryAchievementsPlayerId, PlayerAchievements);

									if (PlayerAchievements.Num() > 0)
									{
										FUniqueNetIdPtr BadAccountId = OnlineIdentity->CreateUniquePlayerId(TEXT("0123456789"));

										OnlineAchievements->QueryAchievementDescriptions(*BadAccountId, FOnQueryAchievementsCompleteDelegate::CreateLambda([this, PlayerAchievements, BadAccountId, TestDone](const FUniqueNetId& QueryAchievementDescriptionsPlayerId, const bool bQueryAchievementDescriptionsWasSuccessful)
										{
											TestEqual("Verify that QueryAchievementDescriptionsPlayerId is the same as TestAccountId", QueryAchievementDescriptionsPlayerId == *BadAccountId, true);
											TestEqual("Verify that bQueryAchievementDescriptionsWasSuccessful returns as: True", bQueryAchievementDescriptionsWasSuccessful, true);

											FOnlineAchievementDesc AchievementDescription;

											OnlineAchievements->GetCachedAchievementDescription(PlayerAchievements[0].Id, AchievementDescription);

											UE_LOG_ONLINE_ACHIEVEMENTS(Display, TEXT("OSS Automation: Found Achievement Description: %s"), *AchievementDescription.ToDebugString());
											TestEqual("Verify that AchievementDescription.Title is not populated", AchievementDescription.Title.IsEmpty(), true);
											TestEqual("Verify that AchievementDescription.LockedDesc is not populated", AchievementDescription.LockedDesc.IsEmpty(), true);
											TestEqual("Verify that AchievementDescription.UnlockedDesc is not populated", AchievementDescription.UnlockedDesc.IsEmpty(), true);

											TestDone.Execute();
										}));
									}
									else
									{
										UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("OSS Automation: PlayerAchievements array is empty after a call to GetCachedAchievements. No Achievements found for this subsystem."));
										TestDone.Execute();
									}
								}));
							}
							else
							{
								UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->CreateUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});
				});

				Describe("GetCachedAchievement", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						CommonUtils.AddAchievementToTestAccount(OnlineIdentity, OnlineAchievements, TestDone);
					});

					LatentIt("When calling GetCachedAchievement with a valid PlayerId and AchievementId, this subsystem returns the cached Achievement", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->GetUniquePlayerId(0);

							if (TestAccountId.IsValid())
							{
								OnlineAchievements->QueryAchievements(*TestAccountId, FOnQueryAchievementsCompleteDelegate::CreateLambda([this, TestAccountId, TestDone](const FUniqueNetId& QueryAchievementsPlayerId, const bool bQueryAchievementsWasSuccessful)
								{
									TArray<FOnlineAchievement> PlayerAchievements;
									OnlineAchievements->GetCachedAchievements(QueryAchievementsPlayerId, PlayerAchievements);

									if (PlayerAchievements.Num() > 0)
									{
										FString TestAchievement = PlayerAchievements[0].Id;

										FOnlineAchievement SomeAchievement;
										OnlineAchievements->GetCachedAchievement(*TestAccountId, TestAchievement, SomeAchievement);

										TestEqual("Verify that SomeAchievement.Id is: TestAchievement", SomeAchievement.Id == TestAchievement, true);
										TestEqual("Verify that SomeAchievement.Progress is: 100", SomeAchievement.Progress == 100, true);

										TestDone.Execute();
									}
									else
									{
										UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("OSS Automation: PlayerAchievements array is empty after a call to GetCachedAchievements. No Achievements found for this subsystem."));
										TestDone.Execute();
									}
								}));
							}
							else
							{
								UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling GetCachedAchievement with a valid PlayerId but an invalid AchievementId, this subsystem does not return any Achievement", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_INVALID_ACHIEVEMENT, EAutomationExpectedErrorFlags::Contains, 0);

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->GetUniquePlayerId(0);

							if (TestAccountId.IsValid())
							{
								OnlineAchievements->QueryAchievements(*TestAccountId, FOnQueryAchievementsCompleteDelegate::CreateLambda([this, TestAccountId, TestDone](const FUniqueNetId& QueryAchievementsPlayerId, const bool bQueryAchievementsWasSuccessful)
								{
									FString FakeAchievement = TEXT("fake_achievement");

									FOnlineAchievement SomeAchievement;
									OnlineAchievements->GetCachedAchievement(*TestAccountId, FakeAchievement, SomeAchievement);

									TestEqual("Verify that SomeAchievement.Id is empty", SomeAchievement.Id.IsEmpty(), true);
									TestEqual("Verify that SomeAchievement.Progress is: 0", SomeAchievement.Progress == 0, true);

									TestDone.Execute();
								}));
							}
							else
							{
								UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling GetCachedAchievement with a valid AchievementId but an invalid PlayerId, this subsystem does not return the cached Achievement", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_MISSINGUSER, EAutomationExpectedErrorFlags::Contains, 0);

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->GetUniquePlayerId(0);

							if (TestAccountId.IsValid())
							{
								OnlineAchievements->QueryAchievements(*TestAccountId, FOnQueryAchievementsCompleteDelegate::CreateLambda([this, TestAccountId, TestDone](const FUniqueNetId& QueryAchievementsPlayerId, const bool bQueryAchievementsWasSuccessful)
								{
									TArray<FOnlineAchievement> PlayerAchievements;
									OnlineAchievements->GetCachedAchievements(QueryAchievementsPlayerId, PlayerAchievements);

									if (PlayerAchievements.Num() > 0)
									{
										FUniqueNetIdPtr BadAccountId = OnlineIdentity->CreateUniquePlayerId(TEXT("0123456789"));

										FString TestAchievement = PlayerAchievements[0].Id;

										FOnlineAchievement SomeAchievement;
										OnlineAchievements->GetCachedAchievement(*BadAccountId, TestAchievement, SomeAchievement);

										TestEqual("Verify that SomeAchievement.Id is: TestAchievement", SomeAchievement.Id.IsEmpty(), true);
										TestEqual("Verify that SomeAchievement.Progress is: 0", SomeAchievement.Progress == 0, true);

										TestDone.Execute();
									}
									else
									{
										UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("OSS Automation: PlayerAchievements array is empty after a call to GetCachedAchievements. No Achievements found for this subsystem."));
										TestDone.Execute();
									}
								}));
							}
							else
							{
								UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentAfterEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						CommonUtils.ResetTestAccountAchievements(OnlineIdentity, OnlineAchievements, TestDone);
					});
				});

				Describe("GetCachedAchievements", [this, SubsystemType]()
				{
					LatentIt("When calling GetCachedAchievements with a valid PlayerId, this subsystem returns all cached Achievements", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->GetUniquePlayerId(0);

							if (TestAccountId.IsValid())
							{
								OnlineAchievements->QueryAchievements(*TestAccountId, FOnQueryAchievementsCompleteDelegate::CreateLambda([this, TestAccountId, TestDone](const FUniqueNetId& QueryAchievementsPlayerId, const bool bQueryAchievementsWasSuccessful)
								{
									TArray<FOnlineAchievement> PlayerAchievements;
									OnlineAchievements->GetCachedAchievements(QueryAchievementsPlayerId, PlayerAchievements);

									TestEqual("Verify that PlayerAchievements is populated", PlayerAchievements.Num() > 0, true);

									TestDone.Execute();
								}));
							}
							else
							{
								UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling GetCachedAchievements with an invalid PlayerId, this subsystem does not return any Achievements", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_MISSINGUSER, EAutomationExpectedErrorFlags::Contains, 0);

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->GetUniquePlayerId(0);

							if (TestAccountId.IsValid())
							{
								OnlineAchievements->QueryAchievements(*TestAccountId, FOnQueryAchievementsCompleteDelegate::CreateLambda([this, TestAccountId, TestDone](const FUniqueNetId& QueryAchievementsPlayerId, const bool bQueryAchievementsWasSuccessful)
								{
									FUniqueNetIdPtr BadAccountId = OnlineIdentity->CreateUniquePlayerId(TEXT("0123456789"));

									TArray<FOnlineAchievement> PlayerAchievements;
									OnlineAchievements->GetCachedAchievements(*BadAccountId, PlayerAchievements);

									TestEqual("Verify that PlayerAchievements is not populated", PlayerAchievements.Num() == 0, true);

									TestDone.Execute();
								}));
							}
							else
							{
								UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});
				});

				Describe("GetCachedAchievementDescription", [this, SubsystemType]()
				{
					LatentIt("When calling GetCachedAchievementDescription with a valid AchievementId, this subsystem returns the cached Achievement's description", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->GetUniquePlayerId(0);

							if (TestAccountId.IsValid())
							{
								OnlineAchievements->QueryAchievements(*TestAccountId, FOnQueryAchievementsCompleteDelegate::CreateLambda([this, TestAccountId, TestDone](const FUniqueNetId& QueryAchievementsPlayerId, const bool bQueryAchievementsWasSuccessful)
								{
									TArray<FOnlineAchievement> PlayerAchievements;
									OnlineAchievements->GetCachedAchievements(QueryAchievementsPlayerId, PlayerAchievements);

									if (PlayerAchievements.Num() > 0)
									{
										OnlineAchievements->QueryAchievementDescriptions(*TestAccountId, FOnQueryAchievementsCompleteDelegate::CreateLambda([this, PlayerAchievements, TestAccountId, TestDone](const FUniqueNetId& QueryAchievementDescriptionsPlayerId, const bool bQueryAchievementDescriptionsWasSuccessful)
										{
											FOnlineAchievementDesc AchievementDescription;

											OnlineAchievements->GetCachedAchievementDescription(PlayerAchievements[0].Id, AchievementDescription);

											UE_LOG_ONLINE_ACHIEVEMENTS(Display, TEXT("OSS Automation: Found Achievement Description: %s"), *AchievementDescription.ToDebugString());
											TestEqual("Verify that AchievementDescription.Title is populated", AchievementDescription.Title.IsEmpty(), false);
											TestEqual("Verify that AchievementDescription.LockedDesc is populated", AchievementDescription.LockedDesc.IsEmpty(), false);
											TestEqual("Verify that AchievementDescription.UnlockedDesc is populated", AchievementDescription.UnlockedDesc.IsEmpty(), false);

											TestDone.Execute();
										}));
									}
									else
									{
										UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("OSS Automation: PlayerAchievements array is empty after a call to GetCachedAchievements. No Achievements found for this subsystem."));
										TestDone.Execute();
									}
								}));
							}
							else
							{
								UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling GetCachedAchievementDescription with an invalid AchievementId, this subsystem does not return any cached Achievement's description", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_INVALID_ACHIEVEMENT, EAutomationExpectedErrorFlags::Contains, 0);

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->GetUniquePlayerId(0);

							if (TestAccountId.IsValid())
							{
								OnlineAchievements->QueryAchievements(*TestAccountId, FOnQueryAchievementsCompleteDelegate::CreateLambda([this, TestAccountId, TestDone](const FUniqueNetId& QueryAchievementsPlayerId, const bool bQueryAchievementsWasSuccessful)
								{
									TArray<FOnlineAchievement> PlayerAchievements;
									OnlineAchievements->GetCachedAchievements(QueryAchievementsPlayerId, PlayerAchievements);

									if (PlayerAchievements.Num() > 0)
									{
										OnlineAchievements->QueryAchievementDescriptions(*TestAccountId, FOnQueryAchievementsCompleteDelegate::CreateLambda([this, PlayerAchievements, TestAccountId, TestDone](const FUniqueNetId& QueryAchievementDescriptionsPlayerId, const bool bQueryAchievementDescriptionsWasSuccessful)
										{
											FString FakeAchievementId = TEXT("fake_achievement");

											FOnlineAchievementDesc AchievementDescription;

											OnlineAchievements->GetCachedAchievementDescription(FakeAchievementId, AchievementDescription);

											UE_LOG_ONLINE_ACHIEVEMENTS(Display, TEXT("OSS Automation: Found Achievement Description: %s"), *AchievementDescription.ToDebugString());
											TestEqual("Verify that AchievementDescription.Title is not populated", AchievementDescription.Title.IsEmpty(), true);
											TestEqual("Verify that AchievementDescription.LockedDesc is not populated", AchievementDescription.LockedDesc.IsEmpty(), true);
											TestEqual("Verify that AchievementDescription.UnlockedDesc is not populated", AchievementDescription.UnlockedDesc.IsEmpty(), true);

											TestDone.Execute();
										}));
									}
									else
									{
										UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("OSS Automation: PlayerAchievements array is empty after a call to GetCachedAchievements. No Achievements found for this subsystem."));
										TestDone.Execute();
									}
								}));
							}
							else
							{
								UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});
				});

				Describe("ResetAchievements", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						CommonUtils.AddAchievementToTestAccount(OnlineIdentity, OnlineAchievements, TestDone);
					});

					LatentIt("When calling ResetAchievements with a valid PlayerId, this subsystem resets that player's achievements", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->GetUniquePlayerId(0);

							if (TestAccountId.IsValid())
							{
								OnlineAchievements->QueryAchievements(*TestAccountId, FOnQueryAchievementsCompleteDelegate::CreateLambda([this, TestAccountId, TestDone](const FUniqueNetId& QueryAchievementsPlayerId, const bool bQueryAchievementsWasSuccessful)
								{
									TestEqual("Verify that QueryAchievementsPlayerId is the same as TestAccountId", QueryAchievementsPlayerId == *TestAccountId, true);
									TestEqual("Verify that bQueryAchievementsWasSuccessful returns as: True", bQueryAchievementsWasSuccessful, true);

									TArray<FOnlineAchievement> PlayerAchievements;
									OnlineAchievements->GetCachedAchievements(QueryAchievementsPlayerId, PlayerAchievements);

									if (PlayerAchievements.Num() > 0)
									{
										TestEqual("Verify that PlayerAchievements[0].Progress is: 100", PlayerAchievements[0].Progress == 100, true);

#if !UE_BUILD_SHIPPING
										OnlineAchievements->ResetAchievements(*TestAccountId);
#endif // !UE_BUILD_SHIPPING

										OnlineAchievements->QueryAchievements(*TestAccountId, FOnQueryAchievementsCompleteDelegate::CreateLambda([this, &PlayerAchievements, TestAccountId, TestDone](const FUniqueNetId& SecondQueryAchievementsPlayerId, const bool bSecondQueryAchievementsWasSuccessful)
										{
											TestEqual("Verify that QueryAchievementsPlayerId is the same as TestAccountId", SecondQueryAchievementsPlayerId == *TestAccountId, true);
											TestEqual("Verify that bQueryAchievementsWasSuccessful returns as: True", bSecondQueryAchievementsWasSuccessful, true);

											OnlineAchievements->GetCachedAchievements(SecondQueryAchievementsPlayerId, PlayerAchievements);

											if (PlayerAchievements.Num() > 0)
											{
												TestEqual("Verify that PlayerAchievements[0].Progress is: 0", PlayerAchievements[0].Progress == 0, true);

												TestDone.Execute();
											}
										}));
									}
									else
									{
										UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("OSS Automation: PlayerAchievements array is empty after a call to GetCachedAchievements. No Achievements found for this subsystem."));
										TestDone.Execute();
									}
								}));
							}
							else
							{
								UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling ResetAchievements with an invalid PlayerId, this subsystem does not reset any player's achievements", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_MISSINGUSER, EAutomationExpectedErrorFlags::Contains, 0);

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->GetUniquePlayerId(0);

							if (TestAccountId.IsValid())
							{
								OnlineAchievements->QueryAchievements(*TestAccountId, FOnQueryAchievementsCompleteDelegate::CreateLambda([this, TestAccountId, TestDone](const FUniqueNetId& QueryAchievementsPlayerId, const bool bQueryAchievementsWasSuccessful)
								{
									TestEqual("Verify that QueryAchievementsPlayerId is the same as TestAccountId", QueryAchievementsPlayerId == *TestAccountId, true);
									TestEqual("Verify that bQueryAchievementsWasSuccessful returns as: True", bQueryAchievementsWasSuccessful, true);

									TArray<FOnlineAchievement> PlayerAchievements;
									OnlineAchievements->GetCachedAchievements(QueryAchievementsPlayerId, PlayerAchievements);

									if (PlayerAchievements.Num() > 0)
									{
										TestEqual("Verify that PlayerAchievements[0].Progress is: 100", PlayerAchievements[0].Progress == 100, true);

										FUniqueNetIdPtr BadAccountId = OnlineIdentity->CreateUniquePlayerId(TEXT("0123456789"));
#if !UE_BUILD_SHIPPING
										OnlineAchievements->ResetAchievements(*BadAccountId);
#endif // !UE_BUILD_SHIPPING

										OnlineAchievements->QueryAchievements(*TestAccountId, FOnQueryAchievementsCompleteDelegate::CreateLambda([this, &PlayerAchievements, TestAccountId, TestDone](const FUniqueNetId& SecondQueryAchievementsPlayerId, const bool bSecondQueryAchievementsWasSuccessful)
										{
											TestEqual("Verify that QueryAchievementsPlayerId is the same as TestAccountId", SecondQueryAchievementsPlayerId == *TestAccountId, true);
											TestEqual("Verify that bQueryAchievementsWasSuccessful returns as: True", bSecondQueryAchievementsWasSuccessful, true);

											OnlineAchievements->GetCachedAchievements(SecondQueryAchievementsPlayerId, PlayerAchievements);

											if (PlayerAchievements.Num() > 0)
											{
												TestEqual("Verify that PlayerAchievements[0].Progress is: 100", PlayerAchievements[0].Progress == 100, true);

												TestDone.Execute();
											}
										}));
									}
									else
									{
										UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("OSS Automation: PlayerAchievements array is empty after a call to GetCachedAchievements. No Achievements found for this subsystem."));
										TestDone.Execute();
									}
								}));
							}
							else
							{
								UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentAfterEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						CommonUtils.ResetTestAccountAchievements(OnlineIdentity, OnlineAchievements, TestDone);
					});
				});
			});

			AfterEach(EAsyncExecution::ThreadPool, [this]()
			{
				// Clean up Identity
				if (OnlineIdentity.IsValid())
				{
					if (OnlineIdentity->GetLoginStatus(0) == ELoginStatus::LoggedIn)
					{
						OnlineIdentity->Logout(0);
					}

					OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
					OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
					OnlineIdentity = nullptr;
				}

				// Clean up Achievements
				if (OnlineAchievements.IsValid())
				{
					OnlineAchievements = nullptr;
				}
			});
		});
	}
}
