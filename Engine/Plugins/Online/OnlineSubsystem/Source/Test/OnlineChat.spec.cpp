// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/OnlineChatInterface.h"
#include "Online.h"
#include "Utils/OnlineTestCommon.h"

BEGIN_DEFINE_SPEC(FOnlineChatSpec, "OnlineChatInterface", EAutomationTestFlags::EngineFilter | EAutomationTestFlags_ApplicationContextMask)

IOnlineSubsystem* OnlineSubsystem;

IOnlineIdentityPtr OnlineIdentity;
IOnlineFriendsPtr OnlineFriends;
IOnlineChatPtr OnlineChat;

FOnlineAccountCredentials AccountCredentials;
FOnlineAccountCredentials FriendAccountCredentials;

FOnlineTestCommon CommonUtils;

// Delegate Handles
FDelegateHandle OnLogoutCompleteDelegateHandle;
FDelegateHandle	OnLoginCompleteDelegateHandle;

END_DEFINE_SPEC(FOnlineChatSpec)

void FOnlineChatSpec::Define()
{
	TArray<FName> Subsystems = FOnlineTestCommon::GetEnabledTestSubsystems();

	for (int32 Index = 0; Index < Subsystems.Num(); Index++)
	{
		FName SubsystemType = Subsystems[Index];

		Describe(SubsystemType.ToString(), [this, SubsystemType]()
		{
			BeforeEach([this, SubsystemType]()
			{
				CommonUtils = FOnlineTestCommon();
				AccountCredentials = FOnlineTestCommon::GetSubsystemTestAccountCredentials(SubsystemType);
				FriendAccountCredentials = FOnlineTestCommon::GetSubsystemFriendAccountCredentials(SubsystemType);

				OnlineIdentity = Online::GetIdentityInterface(SubsystemType);
				OnlineFriends = Online::GetFriendsInterface(SubsystemType);
				OnlineChat = Online::GetChatInterface(SubsystemType);

				// If OnlineIdentity, OnlineFriends, or OnlineChat is not valid, the following test, including all other nested BeforeEaches, will not run
				if (!OnlineIdentity.IsValid())
				{
					UE_LOG_ONLINE(Error, TEXT("OSS Automation: Failed to load OnlineIdentity Interface for %s"), *SubsystemType.ToString());
				}

				if (!OnlineFriends.IsValid())
				{
					UE_LOG_ONLINE(Error, TEXT("OSS Automation: Failed to load OnlineFriends Interface for %s"), *SubsystemType.ToString());
				}

				if (!OnlineChat.IsValid())
				{
					UE_LOG_ONLINE(Error, TEXT("OSS Automation: Failed to load OnlineChat Interface for %s"), *SubsystemType.ToString());
				}

			});

			// TODO: No Tests have been validated yet for functionality
			Describe("Online Chat", [this, SubsystemType]()
			{
				xDescribe("SendPrivateChat", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						//CommonUtils.AddFriendToTestAccount(OnlineIdentity, OnlineFriends, SubsystemType, TestDone);
						TestDone.Execute();
					});

					LatentIt("Private Chat", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNumTestAccount, bool bLoginWasSuccessfulTestAccount, const FUniqueNetId& LoginUserIdTestAccount, const FString& LoginErrorTestAccount)
						{
							FString TestAccountIdString = CommonUtils.GetSubsystemTestAccountUniqueId(SubsystemType);
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->CreateUniquePlayerId(TestAccountIdString);

							FUniqueNetIdPtr ReceivingAccountId = OnlineIdentity->CreateUniquePlayerId(TEXT("7c2bdf22c0264d7193a88002c0ea95bf"));

							OnlineChat->SendPrivateChat(*TestAccountId, *ReceivingAccountId, TEXT("Test"));

							TestDone.Execute();
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});
				});
			});

			AfterEach(EAsyncExecution::ThreadPool, [this]()
			{
				// Clean up Identity
				if (OnlineIdentity.IsValid())
				{
					if (OnlineIdentity->GetLoginStatus(0) == ELoginStatus::LoggedIn)
					{
						OnlineIdentity->Logout(0);
					}

					OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
					OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
					OnlineIdentity = nullptr;
				}

				// Clean up Friends
				if (OnlineFriends.IsValid())
				{
					OnlineFriends = nullptr;
				}

				// Clean up Chat
				if (OnlineChat.IsValid())
				{
					OnlineChat = nullptr;
				}
			});
		});
	}
}
