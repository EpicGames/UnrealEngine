// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online.h"
#include "Utils/OnlineTestCommon.h"

BEGIN_DEFINE_SPEC(FOnlineEntitlementsSpec, "OnlineEntitlementsInterface", EAutomationTestFlags::EngineFilter | EAutomationTestFlags_ApplicationContextMask)

IOnlineSubsystem* OnlineSubsystem;

IOnlineIdentityPtr OnlineIdentity;
IOnlineEntitlementsPtr OnlineEntitlements;

FOnlineAccountCredentials AccountCredentials;

FOnlineTestCommon CommonUtils;

// Delegate Handles
FDelegateHandle OnLogoutCompleteDelegateHandle;
FDelegateHandle	OnLoginCompleteDelegateHandle;
FDelegateHandle OnQueryEntitlementsCompleteDelegateHandle;

END_DEFINE_SPEC(FOnlineEntitlementsSpec)

void FOnlineEntitlementsSpec::Define()
{
	TArray<FName> Subsystems = FOnlineTestCommon::GetEnabledTestSubsystems();

	for (int32 Index = 0; Index < Subsystems.Num(); Index++)
	{
		FName SubsystemType = Subsystems[Index];

		Describe(SubsystemType.ToString(), [this, SubsystemType]()
		{
			BeforeEach([this, SubsystemType]()
			{
				CommonUtils = FOnlineTestCommon();
				AccountCredentials = FOnlineTestCommon::GetSubsystemTestAccountCredentials(SubsystemType);

				OnlineIdentity = Online::GetIdentityInterface(SubsystemType);
				OnlineEntitlements = Online::GetEntitlementsInterface(SubsystemType);

				// If OnlineIdentity or OnlineEntitlements is not valid, the following test, including all other nested BeforeEaches, will not run
				if (!OnlineIdentity.IsValid())
				{
					UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: Failed to load OnlineIdentity Interface for %s"), *SubsystemType.ToString());
				}

				if (!OnlineEntitlements.IsValid())
				{
					UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: Failed to load OnlineEntitlements Interface for %s"), *SubsystemType.ToString());
				}
			});

			// TODO: No tests have been validated for functionality yet
			Describe("Online Entitlements", [this, SubsystemType]()
			{
				Describe("GetEntitlement", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						TestDone.Execute();
					});

					LatentIt("When calling GetEntitlement with a valid UserId and EntitlementId that is cached locally, this subsystem returns that entitlement", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: Test not yet implemented"));

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->GetUniquePlayerId(0);

							if (TestAccountId.IsValid())
							{
								OnQueryEntitlementsCompleteDelegateHandle = OnlineEntitlements->AddOnQueryEntitlementsCompleteDelegate_Handle(FOnQueryEntitlementsCompleteDelegate::CreateLambda([this, TestAccountId, TestDone](bool bQueryEntitlementsWasSuccessful, const FUniqueNetId& QueryEntitlementsUserId, const FString& QueryEntitlementsNamespace, const FString& QueryEntitlementsError)
								{
									TestEqual("Verify that bQueryEntitlementsWasSuccessful returns as: True", bQueryEntitlementsWasSuccessful, true);

									TArray<TSharedRef<FOnlineEntitlement>> PlayerEntitlements;
									OnlineEntitlements->GetAllEntitlements(*TestAccountId, TEXT(""), PlayerEntitlements);

									if (PlayerEntitlements.Num() > 0)
									{
										FUniqueEntitlementId EntitlementId = PlayerEntitlements[0].Get().Id;
										TSharedPtr<FOnlineEntitlement> Entitlement;

										Entitlement = OnlineEntitlements->GetEntitlement(*TestAccountId, EntitlementId);

										TestEqual("Verify that the returned Entitlement->Id is the same as EntitlementId", Entitlement->Id == EntitlementId, true);

										TestDone.Execute();
									}
									else
									{
										UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: PlayerEntitlements array is empty after a call to GetAllEntitlements. No Entitlements found for this user."));
										TestDone.Execute();
									}
								}));

								OnlineEntitlements->QueryEntitlements(*TestAccountId, TEXT(""));
							}
							else
							{
								UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					// TODO: Need a way to get a non-subsystem specific valid entitlement without caching it, or a way to delete the cache
					LatentIt("When calling GetEntitlement with a valid UserId and EntitlementId that is not cached locally, this subsystem does not return that entitlement", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: Test not yet implemented"));

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->GetUniquePlayerId(0);

							if (TestAccountId.IsValid())
							{
								FUniqueEntitlementId EntitlementId = TEXT("0");
								TSharedPtr<FOnlineEntitlement> Entitlement;

								Entitlement = OnlineEntitlements->GetEntitlement(*TestAccountId, EntitlementId);

								TestEqual("Verify that the returned Entitlement is not valid", Entitlement.IsValid(), false);

								UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: Test not yet implemented"));

								TestDone.Execute();
							}
							else
							{
								UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling GetEntitlement with a valid EntitlementId that is cached locally but an invalid UserId, this subsystem does not return that entitlement", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: Test not yet implemented"));

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->GetUniquePlayerId(0);

							if (TestAccountId.IsValid())
							{
								OnQueryEntitlementsCompleteDelegateHandle = OnlineEntitlements->AddOnQueryEntitlementsCompleteDelegate_Handle(FOnQueryEntitlementsCompleteDelegate::CreateLambda([this, TestAccountId, TestDone](bool bQueryEntitlementsWasSuccessful, const FUniqueNetId& QueryEntitlementsUserId, const FString& QueryEntitlementsNamespace, const FString& QueryEntitlementsError)
								{
									TestEqual("Verify that bQueryEntitlementsWasSuccessful returns as: True", bQueryEntitlementsWasSuccessful, true);

									TArray<TSharedRef<FOnlineEntitlement>> PlayerEntitlements;
									OnlineEntitlements->GetAllEntitlements(*TestAccountId, TEXT(""), PlayerEntitlements);

									if (PlayerEntitlements.Num() > 0)
									{
										FUniqueEntitlementId EntitlementId = PlayerEntitlements[0].Get().Id;
										TSharedPtr<FOnlineEntitlement> Entitlement;

										FUniqueNetIdPtr BadAccountId = OnlineIdentity->CreateUniquePlayerId(TEXT("0123456789"));

										if (BadAccountId.IsValid())
										{
											Entitlement = OnlineEntitlements->GetEntitlement(*BadAccountId, EntitlementId);

											TestEqual("Verify that the returned Entitlement is not valid", Entitlement.IsValid(), false);

											TestDone.Execute();
										}
										else
										{
											UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: IsValid() check on BadAccountId failed after a call to OnlineIdentity->CreateUniquePlayerId()"));
											TestDone.Execute();
										}
									}
									else
									{
										UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: PlayerEntitlements array is empty after a call to GetAllEntitlements. No Entitlements found for this user."));
										TestDone.Execute();
									}
								}));

								OnlineEntitlements->QueryEntitlements(*TestAccountId, TEXT("TestNamespace"));
							}
							else
							{
								UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling GetEntitlement with a valid UserId but an invalid EntitlementId, this subsystem does not return any entitlement", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: Test not yet implemented"));

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->GetUniquePlayerId(0);

							if (TestAccountId.IsValid())
							{
								OnQueryEntitlementsCompleteDelegateHandle = OnlineEntitlements->AddOnQueryEntitlementsCompleteDelegate_Handle(FOnQueryEntitlementsCompleteDelegate::CreateLambda([this, TestAccountId, TestDone](bool bQueryEntitlementsWasSuccessful, const FUniqueNetId& QueryEntitlementsUserId, const FString& QueryEntitlementsNamespace, const FString& QueryEntitlementsError)
								{
									TestEqual("Verify that bQueryEntitlementsWasSuccessful returns as: True", bQueryEntitlementsWasSuccessful, true);

									FUniqueEntitlementId EntitlementId = TEXT("-1");
									TSharedPtr<FOnlineEntitlement> Entitlement;

									Entitlement = OnlineEntitlements->GetEntitlement(*TestAccountId, EntitlementId);

									TestEqual("Verify that the returned Entitlement is not valid", Entitlement.IsValid(), false);

									TestDone.Execute();
								}));

								OnlineEntitlements->QueryEntitlements(*TestAccountId, TEXT("TestNamespace"));
							}
							else
							{
								UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});
				});

				xDescribe("GetItemEntitlement", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						TestDone.Execute();
					});

					// TODO: How to get a non-subsystem specific ItemId?
					LatentIt("When calling GetItemEntitlement with a valid UserId and ItemId that is cached locally, this subsystem returns that entitlement", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: Test not yet implemented"));

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->GetUniquePlayerId(0);

							OnQueryEntitlementsCompleteDelegateHandle = OnlineEntitlements->AddOnQueryEntitlementsCompleteDelegate_Handle(FOnQueryEntitlementsCompleteDelegate::CreateLambda([this, TestAccountId, TestDone](bool bQueryEntitlementsWasSuccessful, const FUniqueNetId& QueryEntitlementsUserId, const FString& QueryEntitlementsNamespace, const FString& QueryEntitlementsError)
							{
								TestEqual("Verify that bQueryEntitlementsWasSuccessful returns as: True", bQueryEntitlementsWasSuccessful, true);

								FString ItemId = TEXT("0");
								TSharedPtr<FOnlineEntitlement> Entitlement;

								Entitlement = OnlineEntitlements->GetItemEntitlement(*TestAccountId, ItemId);

								TestEqual("Verify that the returned Entitlement->Id is the same as ItemId", Entitlement->Id == ItemId, true);

								TestDone.Execute();
							}));

							OnlineEntitlements->QueryEntitlements(*TestAccountId, TEXT("TestNamespace"));
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					// TODO: How to get a non-subsystem specific ItemId?
					LatentIt("When calling GetItemEntitlement with a valid UserId and ItemId that is not cached locally, this subsystem does not return that entitlement", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: Test not yet implemented"));

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->GetUniquePlayerId(0);

							if (TestAccountId.IsValid())
							{
								FString ItemId = TEXT("0");
								TSharedPtr<FOnlineEntitlement> Entitlement;

								Entitlement = OnlineEntitlements->GetItemEntitlement(*TestAccountId, ItemId);

								TestEqual("Verify that the returned Entitlement is not valid", Entitlement.IsValid(), true);

								TestDone.Execute();
							}
							else
							{
								UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					// TODO: How to get a non-subsystem specific ItemId?
					LatentIt("When calling GetItemEntitlement with a valid ItemId that is cached locally but an invalid UserId, this subsystem does not return that entitlement", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: Test not yet implemented"));

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->GetUniquePlayerId(0);

							if (TestAccountId.IsValid())
							{
								OnQueryEntitlementsCompleteDelegateHandle = OnlineEntitlements->AddOnQueryEntitlementsCompleteDelegate_Handle(FOnQueryEntitlementsCompleteDelegate::CreateLambda([this, TestAccountId, TestDone](bool bQueryEntitlementsWasSuccessful, const FUniqueNetId& QueryEntitlementsUserId, const FString& QueryEntitlementsNamespace, const FString& QueryEntitlementsError)
								{
									TestEqual("Verify that bQueryEntitlementsWasSuccessful returns as: True", bQueryEntitlementsWasSuccessful, true);

									FString ItemId = TEXT("0");
									TSharedPtr<FOnlineEntitlement> Entitlement;

									FUniqueNetIdPtr BadAccountId = OnlineIdentity->CreateUniquePlayerId(TEXT("0123456789"));

									if (BadAccountId.IsValid())
									{
										Entitlement = OnlineEntitlements->GetItemEntitlement(*BadAccountId, ItemId);

										TestEqual("Verify that the returned Entitlement is not valid", Entitlement.IsValid(), true);

										TestDone.Execute();
									}
									else
									{
										UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: IsValid() check on BadAccountId failed after a call to OnlineIdentity->CreateUniquePlayerId()"));
										TestDone.Execute();
									}
								}));

								OnlineEntitlements->QueryEntitlements(*TestAccountId, TEXT("TestNamespace"));
							}
							else
							{
								UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					// TODO: How to get a non-subsystem specific ItemId?
					LatentIt("When calling GetItemEntitlement with a valid UserId but an invalid ItemId, this subsystem does not return any entitlement", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: Test not yet implemented"));

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->GetUniquePlayerId(0);

							if (TestAccountId.IsValid())
							{
								OnQueryEntitlementsCompleteDelegateHandle = OnlineEntitlements->AddOnQueryEntitlementsCompleteDelegate_Handle(FOnQueryEntitlementsCompleteDelegate::CreateLambda([this, TestAccountId, TestDone](bool bQueryEntitlementsWasSuccessful, const FUniqueNetId& QueryEntitlementsUserId, const FString& QueryEntitlementsNamespace, const FString& QueryEntitlementsError)
								{
									TestEqual("Verify that bQueryEntitlementsWasSuccessful returns as: True", bQueryEntitlementsWasSuccessful, true);

									FString ItemId = TEXT("a");
									TSharedPtr<FOnlineEntitlement> Entitlement;

									Entitlement = OnlineEntitlements->GetItemEntitlement(*TestAccountId, ItemId);

									TestEqual("Verify that the returned Entitlement->Id.IsEmpty() is: True", Entitlement->Id.IsEmpty(), true);

									TestDone.Execute();
								}));

								OnlineEntitlements->QueryEntitlements(*TestAccountId, TEXT("TestNamespace"));
							}
							else
							{
								UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});
				});

				xDescribe("GetAllEntitlements", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						TestDone.Execute();
					});

					LatentIt("When calling GetAllEntitlements with a valid UserId, Namespace, and cached entitlements, this subsystem returns those entitlements", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: Test not yet implemented"));

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->GetUniquePlayerId(0);

							if (TestAccountId.IsValid())
							{
								OnQueryEntitlementsCompleteDelegateHandle = OnlineEntitlements->AddOnQueryEntitlementsCompleteDelegate_Handle(FOnQueryEntitlementsCompleteDelegate::CreateLambda([this, TestAccountId, TestDone](bool bQueryEntitlementsWasSuccessful, const FUniqueNetId& QueryEntitlementsUserId, const FString& QueryEntitlementsNamespace, const FString& QueryEntitlementsError)
								{
									TestEqual("Verify that bQueryEntitlementsWasSuccessful returns as: True", bQueryEntitlementsWasSuccessful, true);

									FString Namespace = TEXT("TestNamespace");
									TArray<TSharedRef<FOnlineEntitlement>> Entitlements;

									OnlineEntitlements->GetAllEntitlements(*TestAccountId, Namespace, Entitlements);

									TestEqual("Verify that the returned Entitlements array is populated", Entitlements.Num() > 0, true);

									TestDone.Execute();
								}));

								OnlineEntitlements->QueryEntitlements(*TestAccountId, TEXT("TestNamespace"));
							}
							else
							{
								UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling GetAllEntitlements with a valid UserId and Namespace but no cached entitlements, this subsystem does not return any entitlements", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: Test not yet implemented"));

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->GetUniquePlayerId(0);

							if (TestAccountId.IsValid())
							{
								FString Namespace = TEXT("TestNamespace");
								TArray<TSharedRef<FOnlineEntitlement>> Entitlements;

								OnlineEntitlements->GetAllEntitlements(*TestAccountId, Namespace, Entitlements);

								TestEqual("Verify that the returned Entitlements array is not populated", Entitlements.Num() == 0, true);

								TestDone.Execute();
							}
							else
							{
								UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling GetAllEntitlements with a valid Namespace and cached entitlements but an invalid UserId, this subsystem does not return those entitlements", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: Test not yet implemented"));

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->GetUniquePlayerId(0);

							if (TestAccountId.IsValid())
							{
								OnQueryEntitlementsCompleteDelegateHandle = OnlineEntitlements->AddOnQueryEntitlementsCompleteDelegate_Handle(FOnQueryEntitlementsCompleteDelegate::CreateLambda([this, TestAccountId, TestDone](bool bQueryEntitlementsWasSuccessful, const FUniqueNetId& QueryEntitlementsUserId, const FString& QueryEntitlementsNamespace, const FString& QueryEntitlementsError)
								{
									TestEqual("Verify that bQueryEntitlementsWasSuccessful returns as: True", bQueryEntitlementsWasSuccessful, true);

									FString Namespace = TEXT("TestNamespace");
									TArray<TSharedRef<FOnlineEntitlement>> Entitlements;

									FUniqueNetIdPtr BadAccountId = OnlineIdentity->CreateUniquePlayerId(TEXT("0123456789"));

									if (BadAccountId.IsValid())
									{
										OnlineEntitlements->GetAllEntitlements(*BadAccountId, Namespace, Entitlements);

										TestEqual("Verify that the returned Entitlements array is not populated", Entitlements.Num() == 0, true);

										TestDone.Execute();
									}
									else
									{
										UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: IsValid() check on BadAccountId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
										TestDone.Execute();
									}
								}));

								OnlineEntitlements->QueryEntitlements(*TestAccountId, TEXT("TestNamespace"));
							}
							else
							{
								UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling GetAllEntitlements with a valid UserId and cached entitlements but an invalid Namespace, this subsystem does not return those entitlements", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: Test not yet implemented"));

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->GetUniquePlayerId(0);

							if (TestAccountId.IsValid())
							{
								OnQueryEntitlementsCompleteDelegateHandle = OnlineEntitlements->AddOnQueryEntitlementsCompleteDelegate_Handle(FOnQueryEntitlementsCompleteDelegate::CreateLambda([this, TestAccountId, TestDone](bool bQueryEntitlementsWasSuccessful, const FUniqueNetId& QueryEntitlementsUserId, const FString& QueryEntitlementsNamespace, const FString& QueryEntitlementsError)
								{
									TestEqual("Verify that bQueryEntitlementsWasSuccessful returns as: True", bQueryEntitlementsWasSuccessful, true);

									FString Namespace = TEXT("InvalidNamespace");
									TArray<TSharedRef<FOnlineEntitlement>> Entitlements;

									OnlineEntitlements->GetAllEntitlements(*TestAccountId, Namespace, Entitlements);

									TestEqual("Verify that the returned Entitlements array is not populated", Entitlements.Num() == 0, true);

									TestDone.Execute();
								}));

								OnlineEntitlements->QueryEntitlements(*TestAccountId, TEXT("TestNamespace"));
							}
							else
							{
								UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});
				});

				xDescribe("QueryEntitlements", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						TestDone.Execute();
					});

					LatentIt("When calling QueryEntitlements with a valid UserId and Namespace, this subsystem caches that user's entitlements locally", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: Test not yet implemented"));

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->GetUniquePlayerId(0);

							if (TestAccountId.IsValid())
							{
								OnQueryEntitlementsCompleteDelegateHandle = OnlineEntitlements->AddOnQueryEntitlementsCompleteDelegate_Handle(FOnQueryEntitlementsCompleteDelegate::CreateLambda([this, TestAccountId, TestDone](bool bQueryEntitlementsWasSuccessful, const FUniqueNetId& QueryEntitlementsUserId, const FString& QueryEntitlementsNamespace, const FString& QueryEntitlementsError)
								{
									TestEqual("Verify that bQueryEntitlementsWasSuccessful returns as: True", bQueryEntitlementsWasSuccessful, true);
									TestEqual("Verify that QueryEntitlementsUserId is equal to TestAccountId", QueryEntitlementsUserId == *TestAccountId, true);
									TestEqual("Verify that QueryEntitlementsNamespace is: TestNamespace", QueryEntitlementsNamespace == TEXT("TestNamespace"), true);
									TestEqual("Verify that QueryEntitlementsError is empty", QueryEntitlementsError.IsEmpty(), true);

									FUniqueEntitlementId EntitlementId = TEXT("0");
									TSharedPtr<FOnlineEntitlement> Entitlement;

									Entitlement = OnlineEntitlements->GetEntitlement(*TestAccountId, EntitlementId);

									TestEqual("Verify that the returned Entitlement.IsValid() is: True", Entitlement.IsValid(), true);

									TestDone.Execute();
								}));

								OnlineEntitlements->QueryEntitlements(*TestAccountId, TEXT("TestNamespace"));
							}
							else
							{
								UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling QueryEntitlements with a valid Namespace but an invalid UserId, this subsystem does not cache any user's entitlements locally", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: Test not yet implemented"));

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->GetUniquePlayerId(0);

							if (TestAccountId.IsValid())
							{
								FUniqueNetIdPtr BadAccountId = OnlineIdentity->CreateUniquePlayerId(TEXT("0123456789"));

								if (BadAccountId.IsValid())
								{
									OnQueryEntitlementsCompleteDelegateHandle = OnlineEntitlements->AddOnQueryEntitlementsCompleteDelegate_Handle(FOnQueryEntitlementsCompleteDelegate::CreateLambda([this, BadAccountId, TestAccountId, TestDone](bool bQueryEntitlementsWasSuccessful, const FUniqueNetId& QueryEntitlementsUserId, const FString& QueryEntitlementsNamespace, const FString& QueryEntitlementsError)
									{
										TestEqual("Verify that bQueryEntitlementsWasSuccessful returns as: False", bQueryEntitlementsWasSuccessful, false);
										TestEqual("Verify that QueryEntitlementsUserId is equal to BadAccountId", QueryEntitlementsUserId == *BadAccountId, true);
										TestEqual("Verify that QueryEntitlementsNamespace is: TestNamespace", QueryEntitlementsNamespace == TEXT("TestNamespace"), true);
										TestEqual("Verify that QueryEntitlementsError is empty", QueryEntitlementsError.IsEmpty(), true);

										FUniqueEntitlementId EntitlementId = TEXT("0");
										TSharedPtr<FOnlineEntitlement> Entitlement;

										Entitlement = OnlineEntitlements->GetEntitlement(*TestAccountId, EntitlementId);

										TestEqual("Verify that the returned Entitlement.IsValid() is: False", Entitlement.IsValid(), false);

										TestDone.Execute();
									}));

									OnlineEntitlements->QueryEntitlements(*BadAccountId, TEXT("TestNamespace"));
								}
								else
								{
									UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: IsValid() check on BadAccountId failed after a call to OnlineIdentity->CreateUniquePlayerId()"));
									TestDone.Execute();
								}
							}
							else
							{
								UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling QueryEntitlements with a valid UserId but an invalid Namespace, this subsystem does not cache that user's entitlements locally", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: Test not yet implemented"));

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->GetUniquePlayerId(0);

							if (TestAccountId.IsValid())
							{
								OnQueryEntitlementsCompleteDelegateHandle = OnlineEntitlements->AddOnQueryEntitlementsCompleteDelegate_Handle(FOnQueryEntitlementsCompleteDelegate::CreateLambda([this, TestAccountId, TestDone](bool bQueryEntitlementsWasSuccessful, const FUniqueNetId& QueryEntitlementsUserId, const FString& QueryEntitlementsNamespace, const FString& QueryEntitlementsError)
								{
									TestEqual("Verify that bQueryEntitlementsWasSuccessful returns as: False", bQueryEntitlementsWasSuccessful, false);
									TestEqual("Verify that QueryEntitlementsUserId is equal to BadAccountId", QueryEntitlementsUserId == *TestAccountId, true);
									TestEqual("Verify that QueryEntitlementsNamespace is: InvalidNamespace", QueryEntitlementsNamespace == TEXT("InvalidNamespace"), true);
									TestEqual("Verify that QueryEntitlementsError is empty", QueryEntitlementsError.IsEmpty(), true);

									FUniqueEntitlementId EntitlementId = TEXT("0");
									TSharedPtr<FOnlineEntitlement> Entitlement;

									Entitlement = OnlineEntitlements->GetEntitlement(*TestAccountId, EntitlementId);

									TestEqual("Verify that the returned Entitlement.IsValid is: False", Entitlement.IsValid(), false);

									TestDone.Execute();
								}));

								OnlineEntitlements->QueryEntitlements(*TestAccountId, TEXT("InvalidNamespace"));
							}
							else
							{
								UE_LOG_ONLINE_ENTITLEMENT(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});
				});
			});

			AfterEach(EAsyncExecution::ThreadPool, [this]()
			{
				// Clean up Identity
				if (OnlineIdentity.IsValid())
				{
					OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
					OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
					OnlineIdentity = nullptr;
				}

				// Clean up Entitlements
				if (OnlineEntitlements.IsValid())
				{
					OnlineEntitlements->ClearOnQueryEntitlementsCompleteDelegate_Handle(OnQueryEntitlementsCompleteDelegateHandle);
					OnlineEntitlements = nullptr;
				}
			});
		});
	}
}
