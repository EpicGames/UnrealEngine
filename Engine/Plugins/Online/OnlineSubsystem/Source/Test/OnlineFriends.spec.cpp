// Copyright Epic Games, Inc. All Rights Reserved.


#include "Online.h"
#include "Utils/OnlineErrors.data.h"
#include "Misc/CommandLine.h"
#include "Utils/OnlineTestCommon.h"

BEGIN_DEFINE_SPEC(FOnlineFriendsSpec, "OnlineFriendsInterface", EAutomationTestFlags::EngineFilter | EAutomationTestFlags_ApplicationContextMask)

FOnlineTestCommon CommonUtils;

IOnlineIdentityPtr OnlineIdentity;
IOnlineFriendsPtr OnlineFriends;
FOnlineAccountCredentials AccountCredentials;
FOnlineAccountCredentials FriendAccountCredentials;

// Delegate Handles
FDelegateHandle OnLogoutCompleteDelegateHandle;
FDelegateHandle	OnLoginCompleteDelegateHandle;

FDelegateHandle OnReadFriendsListCompleteDelegateHandle;
FDelegateHandle OnInviteAcceptedDelegateHandle;
FDelegateHandle OnRejectInviteCompleteDelegateHandle;
FDelegateHandle OnDeleteFriendCompleteDelegateHandle;
FDelegateHandle OnBlockedPlayerCompleteDelegateHandle;
FDelegateHandle OnUnblockedPlayerCompleteDelegateHandle;
FDelegateHandle OnQueryBlockedPlayersCompleteDelegateHandle;

END_DEFINE_SPEC(FOnlineFriendsSpec)

void FOnlineFriendsSpec::Define()
{
	TArray<FName> Subsystems = FOnlineTestCommon::GetEnabledTestSubsystems();

	for (int32 Index = 0; Index < Subsystems.Num(); Index++)
	{
		FName SubsystemType = Subsystems[Index];
		
		Describe(SubsystemType.ToString(), [this, SubsystemType]()
		{
			BeforeEach([this, SubsystemType]()
			{
				CommonUtils = FOnlineTestCommon();
				AccountCredentials = FOnlineTestCommon::GetSubsystemTestAccountCredentials(SubsystemType);
				FriendAccountCredentials = FOnlineTestCommon::GetSubsystemFriendAccountCredentials(SubsystemType);

				OnlineIdentity = Online::GetIdentityInterface(SubsystemType);
				OnlineFriends = Online::GetFriendsInterface(SubsystemType);

				// If OnlineFriends or OnlineIdentity is not valid, the following test, including all other nested BeforeEaches, will not run
				if (!OnlineFriends.IsValid())
				{
					UE_LOG_ONLINE_FRIEND(Error, TEXT("Failed to get online friends interface for %s"), *SubsystemType.ToString());
				}
				if (!OnlineIdentity.IsValid())
				{
					UE_LOG_ONLINE_FRIEND(Error, TEXT("Failed to get online identity interface for %s"), *SubsystemType.ToString());
				}
			});
			
			Describe("Online Friends", [this, SubsystemType]()
			{
				Describe("ReadFriendsList", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						CommonUtils.AddFriendToTestAccount(OnlineIdentity, OnlineFriends, SubsystemType, TestDone);
					});

					LatentIt("When calling ReadFriendsList with a valid local user and the Default list name, the user will receive their friends list", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnlineFriends->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateLambda([this, TestDone](int32 ReadFriendsListLocalUserNum, bool bReadFriendsListWasSuccessful, const FString& ReadFriendsListListName, const FString& ReadFriendsListErrorStr)
							{
								TestEqual("Verify that bReadFriendsListWasSuccessful returns as: True", bReadFriendsListWasSuccessful, true);
								TestEqual("Verify that ReadFriendsListErrorStr is empty", ReadFriendsListErrorStr.Len() == 0, true);

								TArray<TSharedRef<FOnlineFriend>> FriendsList;

								OnlineFriends->GetFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FriendsList);

								TestEqual("Verify that FriendsList is populated", FriendsList.Num() > 0, true);

								TestDone.Execute();
							}));
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					// TODO: Figure out how to realistically get results from these lists
					LatentIt("When calling ReadFriendsList with a valid local user and the OnlinePlayers list name, the user will receive their friends list", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineFriends->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::OnlinePlayers), FOnReadFriendsListComplete::CreateLambda([this, TestDone](int32 ReadFriendsListLocalUserNum, bool bReadFriendsListWasSuccessful, const FString& ReadFriendsListListName, const FString& ReadFriendsListErrorStr)
						{
							TestEqual("Verify that bReadFriendsListWasSuccessful returns as: True", bReadFriendsListWasSuccessful, true);
							TestEqual("Verify that ReadFriendsListErrorStr is empty", ReadFriendsListErrorStr.Len() == 0, true);

							TArray<TSharedRef<FOnlineFriend>> FriendsList;

							OnlineFriends->GetFriendsList(0, EFriendsLists::ToString(EFriendsLists::OnlinePlayers), FriendsList);

							TestEqual("Verify that FriendsList is populated", FriendsList.Num() > 0, true);

							UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: Test implementation not yet complete. Need to verify that different friends lists produce different results"));
							TestDone.Execute();
						}));
					});

					// TODO: How to get a friend in game/session for these results?
					LatentIt("When calling ReadFriendsList with a valid local user and the InGamePlayers list name, the user will receive their friends list", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineFriends->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::InGamePlayers), FOnReadFriendsListComplete::CreateLambda([this, TestDone](int32 ReadFriendsListLocalUserNum, bool bReadFriendsListWasSuccessful, const FString& ReadFriendsListListName, const FString& ReadFriendsListErrorStr)
						{
							TestEqual("Verify that bReadFriendsListWasSuccessful returns as: True", bReadFriendsListWasSuccessful, true);
							TestEqual("Verify that ReadFriendsListErrorStr is empty", ReadFriendsListErrorStr.Len() == 0, true);

							TArray<TSharedRef<FOnlineFriend>> FriendsList;

							OnlineFriends->GetFriendsList(0, EFriendsLists::ToString(EFriendsLists::InGamePlayers), FriendsList);

							TestEqual("Verify that FriendsList is populated", FriendsList.Num() > 0, true);

							UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: Test implementation not yet complete. Need to verify that different friends lists produce different results"));
							TestDone.Execute();
						}));
					});

					// TODO: How to get a friend in game/session for these results?
					LatentIt("When calling ReadFriendsList with a valid local user and the InGameAndSessionPlayers list name, the user will receive their friends list", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineFriends->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::InGameAndSessionPlayers), FOnReadFriendsListComplete::CreateLambda([this, TestDone](int32 ReadFriendsListLocalUserNum, bool bReadFriendsListWasSuccessful, const FString& ReadFriendsListListName, const FString& ReadFriendsListErrorStr)
						{
							TestEqual("Verify that bReadFriendsListWasSuccessful returns as: True", bReadFriendsListWasSuccessful, true);
							TestEqual("Verify that ReadFriendsListErrorStr is empty", ReadFriendsListErrorStr.Len() == 0, true);

							TArray<TSharedRef<FOnlineFriend>> FriendsList;

							OnlineFriends->GetFriendsList(0, EFriendsLists::ToString(EFriendsLists::InGameAndSessionPlayers), FriendsList);

							TestEqual("Verify that FriendsList is populated", FriendsList.Num() > 0, true);

							UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: Test implementation not yet complete. Need to verify that different friends lists produce different results"));
							TestDone.Execute();
						}));
					});

					LatentIt("When calling ReadFriendsList with a list name but an invalid local user (-1), the user does not receive that friends list", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							AddExpectedError(ONLINE_EXPECTEDERROR_INVALID_LOCALUSER, EAutomationExpectedErrorFlags::Contains, 0);

							OnlineFriends->ReadFriendsList(-1, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateLambda([this, TestDone](int32 ReadFriendsListLocalUserNum, bool bReadFriendsListWasSuccessful, const FString& ReadFriendsListListName, const FString& ReadFriendsListErrorStr)
							{
								TestEqual("Verify that bReadFriendsListWasSuccessful returns as: False", bReadFriendsListWasSuccessful, false);
								TestEqual("Verify that ReadFriendsListErrorStr return the expected error code: ONLINE_EXPECTEDERROR_INVALID_LOCALUSER", ReadFriendsListErrorStr.Contains(ONLINE_EXPECTEDERROR_INVALID_LOCALUSER), true);

								TArray<TSharedRef<FOnlineFriend>> FriendsList;

								OnlineFriends->GetFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FriendsList);

								TestEqual("Verify that FriendsList is not populated", FriendsList.Num() == 0, true);

								TestDone.Execute();
							}));
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					// TODO: Shouldn't an invalid list name produce an error?
					LatentIt("When calling ReadFriendsList with a valid local user but an invalid list name, the user does not receive a friends list", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_INVALID_LOCALUSER, EAutomationExpectedErrorFlags::Contains, 0);

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnlineFriends->ReadFriendsList(0, TEXT("fakelistname"), FOnReadFriendsListComplete::CreateLambda([this, TestDone](int32 ReadFriendsListLocalUserNum, bool bReadFriendsListWasSuccessful, const FString& ReadFriendsListListName, const FString& ReadFriendsListErrorStr)
							{
								TestEqual("Verify that bReadFriendsListWasSuccessful returns as: False", bReadFriendsListWasSuccessful, false);
								TestEqual("Verify that ReadFriendsListErrorStr return the expected error code: ONLINE_EXPECTEDERROR_INVALID_LOCALUSER", ReadFriendsListErrorStr.Contains(ONLINE_EXPECTEDERROR_INVALID_LOCALUSER), true);

								TArray<TSharedRef<FOnlineFriend>> FriendsList;

								OnlineFriends->GetFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FriendsList);

								TestEqual("Verify that FriendsList is not populated", FriendsList.Num() == 0, true);

								UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: Test implementation not yet complete. Need to verify that different friends lists produce different results"));
								TestDone.Execute();
							}));
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentAfterEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
						OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoggedOutLocalUserNum, bool bLogoutWasSuccessful)
						{
							CommonUtils.RemoveFriendFromTestAccount(OnlineIdentity, OnlineFriends, SubsystemType, TestDone);
							OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
						}));

						OnlineIdentity->Logout(0);
					});
				});

				Describe("DeleteFriendsList", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						CommonUtils.AddFriendToTestAccount(OnlineIdentity, OnlineFriends, SubsystemType, TestDone);
					});

					LatentIt("When calling DeleteFriendsList with a valid local user and the Default list name, this subsystem deletes that user's friends list", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnlineFriends->DeleteFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FOnDeleteFriendsListComplete::CreateLambda([this, SubsystemType, TestDone](int32 DeleteFriendsListLocalUserNum, bool bDeleteFriendsListWasSuccessful, const FString& DeleteFriendsListListName, const FString& DeleteFriendsListErrorStr)
							{
								TestEqual("Verify that bDeleteFriendsListWasSuccessful returns as: True", bDeleteFriendsListWasSuccessful, true);
								TestEqual("Verify that DeleteFriendsListErrorStr is empty", DeleteFriendsListErrorStr.Len() == 0, true);

								OnlineFriends->ReadFriendsList(0, DeleteFriendsListListName, FOnReadFriendsListComplete::CreateLambda([this, SubsystemType, TestDone](int32 ReadFriendsListLocalUserNum, bool bReadFriendsListWasSuccessful, const FString& ReadFriendsListListName, const FString& ReadFriendsListErrorStr)
								{
									TArray<TSharedRef<FOnlineFriend>> FriendsList;
									OnlineFriends->GetFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FriendsList);

									TestEqual("Verify that FriendsList is not populated", FriendsList.Num() == 0, true);

									TestDone.Execute();
								}));
							}));
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					// TODO: ListName does nothing?
					LatentIt("When calling DeleteFriendsList with a valid local user and the OnlinePlayers list name, this subsystem deletes that user's friends list", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_INVALID_LOCALUSER, EAutomationExpectedErrorFlags::Contains, 0);

						OnlineFriends->DeleteFriendsList(0, EFriendsLists::ToString(EFriendsLists::OnlinePlayers), FOnDeleteFriendsListComplete::CreateLambda([this, TestDone](int32 DeleteFriendsListLocalUserNum, bool bDeleteFriendsListWasSuccessful, const FString& DeleteFriendsListListName, const FString& DeleteFriendsListErrorStr)
						{
							TestEqual("Verify that bDeleteFriendsListWasSuccessful returns as: True", bDeleteFriendsListWasSuccessful, true);
							TestEqual("Verify that DeleteFriendsListErrorStr is empty", DeleteFriendsListErrorStr.Len() == 0, true);

							TArray<TSharedRef<FOnlineFriend>> FriendsList;
							OnlineFriends->GetFriendsList(0, EFriendsLists::ToString(EFriendsLists::OnlinePlayers), FriendsList);

							TestEqual("Verify that FriendsList is not populated", FriendsList.Num() == 0, true);

							UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: Test implementation not yet complete. Need to verify that different friends lists produce different results"));
							TestDone.Execute();
						}));
					});

					// TODO: ListName does nothing?
					LatentIt("When calling DeleteFriendsList with a valid local user and the InGamePlayers list name, this subsystem deletes that user's friends list", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_INVALID_LOCALUSER, EAutomationExpectedErrorFlags::Contains, 0);

						OnlineFriends->DeleteFriendsList(0, EFriendsLists::ToString(EFriendsLists::InGamePlayers), FOnDeleteFriendsListComplete::CreateLambda([this, TestDone](int32 DeleteFriendsListLocalUserNum, bool bDeleteFriendsListWasSuccessful, const FString& DeleteFriendsListListName, const FString& DeleteFriendsListErrorStr)
						{
							TestEqual("Verify that bDeleteFriendsListWasSuccessful returns as: True", bDeleteFriendsListWasSuccessful, true);
							TestEqual("Verify that DeleteFriendsListErrorStr is empty", DeleteFriendsListErrorStr.Len() == 0, true);

							TArray<TSharedRef<FOnlineFriend>> FriendsList;
							OnlineFriends->GetFriendsList(0, EFriendsLists::ToString(EFriendsLists::InGamePlayers), FriendsList);

							TestEqual("Verify that FriendsList is not populated", FriendsList.Num() == 0, true);

							UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: Test implementation not yet complete. Need to verify that different friends lists produce different results"));
							TestDone.Execute();
						}));
					});

					// TODO: ListName does nothing?
					LatentIt("When calling DeleteFriendsList with a valid local user and the InGameAndSessionPlayers list name, this subsystem deletes that user's friends list", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_INVALID_LOCALUSER, EAutomationExpectedErrorFlags::Contains, 0);

						OnlineFriends->DeleteFriendsList(0, EFriendsLists::ToString(EFriendsLists::InGameAndSessionPlayers), FOnDeleteFriendsListComplete::CreateLambda([this, TestDone](int32 DeleteFriendsListLocalUserNum, bool bDeleteFriendsListWasSuccessful, const FString& DeleteFriendsListListName, const FString& DeleteFriendsListErrorStr)
						{
							TestEqual("Verify that bDeleteFriendsListWasSuccessful returns as: True", bDeleteFriendsListWasSuccessful, true);
							TestEqual("Verify that DeleteFriendsListErrorStr is empty", DeleteFriendsListErrorStr.Len() == 0, true);

							TArray<TSharedRef<FOnlineFriend>> FriendsList;
							OnlineFriends->GetFriendsList(0, EFriendsLists::ToString(EFriendsLists::InGameAndSessionPlayers), FriendsList);

							TestEqual("Verify that FriendsList is not populated", FriendsList.Num() == 0, true);

							UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: Test implementation not yet complete. Need to verify that different friends lists produce different results"));
							TestDone.Execute();
						}));
					});

					LatentIt("When calling DeleteFriendsList with a valid list name but an invalid local user (-1), this subsystem does not attempt a delete friends list request", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_INVALID_LOCALUSER, EAutomationExpectedErrorFlags::Contains, 0);

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnlineFriends->DeleteFriendsList(-1, EFriendsLists::ToString(EFriendsLists::Default), FOnDeleteFriendsListComplete::CreateLambda([this, SubsystemType, TestDone](int32 DeleteFriendsListLocalUserNum, bool bDeleteFriendsListWasSuccessful, const FString& DeleteFriendsListListName, const FString& DeleteFriendsListErrorStr)
							{
								TestEqual("Verify that DeleteFriendsListLocalUserNum is: -1", DeleteFriendsListLocalUserNum == -1, true);
								TestEqual("Verify that bDeleteFriendsListWasSuccessful returns as: False", bDeleteFriendsListWasSuccessful, false);
								TestEqual("Verify that DeleteFriendsListListName is: Default", DeleteFriendsListListName == EFriendsLists::ToString(EFriendsLists::Default), true);
								TestEqual("Verify that DeleteFriendsListErrorStr returns the expected error code: ONLINE_EXPECTEDERROR_INVALID_LOCALUSER", DeleteFriendsListErrorStr.Contains(ONLINE_EXPECTEDERROR_INVALID_LOCALUSER), true);

								OnlineFriends->ReadFriendsList(0, DeleteFriendsListListName, FOnReadFriendsListComplete::CreateLambda([this, SubsystemType, TestDone](int32 ReadFriendsListLocalUserNum, bool bReadFriendsListWasSuccessful, const FString& ReadFriendsListListName, const FString& ReadFriendsListErrorStr)
								{
									TArray<TSharedRef<FOnlineFriend>> FriendsList;
									OnlineFriends->GetFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FriendsList);

									TestEqual("Verify that FriendsList is populated", FriendsList.Num() > 0, true);

									TestDone.Execute();
								}));
							}));
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					It("When calling DeleteFriendsList with a valid local user but an invalid list name, this subsystem does not attempt a delete friends list request", [this, SubsystemType]()
					{
						UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: Test implementation not yet complete. Need to verify that different friends lists produce different results"));
					});

					LatentAfterEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
						OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoggedOutLocalUserNum, bool bLogoutWasSuccessful)
						{
							CommonUtils.RemoveFriendFromTestAccount(OnlineIdentity, OnlineFriends, SubsystemType, TestDone);
							OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
						}));

						OnlineIdentity->Logout(0);
					});
				});

				Describe("SendInvite", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: True", bLoginWasSuccessful, true);

							TestDone.Execute();
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling SendInvite with a valid local user, a valid friend ID, and a list name, this subsystem sends a friend invite to that friend ID", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						FString FriendIdString = FOnlineTestCommon::GetSubsystemFriendAccountUniqueId(SubsystemType);
						FUniqueNetIdPtr FriendIdToUse = OnlineIdentity->CreateUniquePlayerId(FriendIdString);

						OnlineFriends->SendInvite(0, *FriendIdToUse, EFriendsLists::ToString(EFriendsLists::Default), FOnSendInviteComplete::CreateLambda([this, FriendIdToUse, SubsystemType, TestDone](int32 SendInviteLocalUserNum, bool bSendInviteWasSuccessful, const FUniqueNetId& SendInviteFriendId, const FString& SendInviteListName, const FString& SendInviteErrorStr)
						{
							TestEqual("Verify that SendInviteLocalUserNum is: 0", SendInviteLocalUserNum == 0, true);
							TestEqual("Verify that bSendInviteWasSuccessful returns as: True", bSendInviteWasSuccessful, true);
							TestEqual("Verify that SendInviteFriendId is the Id that was originally used", SendInviteFriendId.ToString() == FriendIdToUse->ToString(), true);
							TestEqual("Verify that SendInviteListName is: Default", SendInviteListName == EFriendsLists::ToString(EFriendsLists::Default), true);
							TestEqual("Verify that SendInviteErrorStr is not populated", SendInviteErrorStr.Len() == 0, true);

							OnlineFriends->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateLambda([this, FriendIdToUse, SubsystemType, TestDone](int32 ReadFriendsListLocalUserNum, bool bReadFriendsListWasSuccessful, const FString& ReadFriendsListListName, const FString& ReadFriendsListErrorStr)
							{
								TestEqual("Verify that ReadFriendsListLocalUserNum is: 0", ReadFriendsListLocalUserNum == 0, true);
								TestEqual("Verify that bReadFriendsListWasSuccessful returns as: True", bReadFriendsListWasSuccessful, true);
								TestEqual("Verify that ReadFriendsListListName is: Default", ReadFriendsListListName == EFriendsLists::ToString(EFriendsLists::Default), true);
								TestEqual("Verify that ReadFriendsListErrorStr is not populated", ReadFriendsListErrorStr.Len() == 0, true);

								TArray<TSharedRef<FOnlineFriend>> FriendsList;
								OnlineFriends->GetFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FriendsList);

								bool bFoundFriend = false;

								for (TSharedRef<FOnlineFriend> Friend : FriendsList)
								{
									if (Friend->GetUserId()->ToString() == FriendIdToUse->ToString())
									{
										bFoundFriend = true;
										break;
									}
								}

								TestEqual("Verify that bFoundFriend is: True", bFoundFriend, true);

								TestDone.Execute();
							}));
						}));
					});

					LatentIt("When calling SendInvite with a valid local user and a list name but a valid friend ID that is already on the friends list, this subsystem states that both users are already friends", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						FString FriendAccountIdString = FOnlineTestCommon::GetSubsystemFriendAccountUniqueId(SubsystemType);
						FUniqueNetIdPtr FriendAccountId = OnlineIdentity->CreateUniquePlayerId(FriendAccountIdString);

						FString TestAccountIdString = FOnlineTestCommon::GetSubsystemTestAccountUniqueId(SubsystemType);
						FUniqueNetIdPtr TestAccountId = OnlineIdentity->CreateUniquePlayerId(TestAccountIdString);

						AddExpectedError(ONLINE_EXPECTEDERROR_ALREADYFRIENDS, EAutomationExpectedErrorFlags::Contains, 0);

						OnlineFriends->SendInvite(0, *FriendAccountId, EFriendsLists::ToString(EFriendsLists::Default), FOnSendInviteComplete::CreateLambda([this, FriendAccountId, TestAccountId, SubsystemType, TestDone](int32 SendInviteLocalUserNum, bool bSendInviteWasSuccessful, const FUniqueNetId& SendInviteFriendId, const FString& SendInviteListName, const FString& SendInviteErrorStr)
						{
							TestEqual("Verify that bSendInviteWasSuccessful returns as: True", bSendInviteWasSuccessful, true);

							OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
							OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, FriendAccountId, TestAccountId, SubsystemType, TestDone](int32 InitialLoggedOutLocalUserNum, bool bInitialLogoutWasSuccessful)
							{
								OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
								OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, FriendAccountId, TestAccountId, SubsystemType, TestDone](int32 InitialLoginLocalPlayerNum, bool bInitialLoginWasSuccessful, const FUniqueNetId& InitialLoginUserId, const FString& InitialLoginError)
								{
									OnlineFriends->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateLambda([this, FriendAccountId, TestAccountId, SubsystemType, TestDone](int32 ReadFriendsListLocalUserNum, bool bReadFriendsListWasSuccessful, const FString& ReadFriendsListListName, const FString& ReadFriendsListErrorStr)
									{
										OnlineFriends->AcceptInvite(0, *TestAccountId, EFriendsLists::ToString(EFriendsLists::Default), FOnAcceptInviteComplete::CreateLambda([this, FriendAccountId, TestAccountId, SubsystemType, TestDone](int32 AcceptInviteLocalPlayerNum, bool bAcceptInviteWasSuccessful, const FUniqueNetId& AcceptInviteFriendId, const FString& AcceptInviteListName, const FString& AcceptInviteErrorStr)
										{
											OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
											OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, FriendAccountId, TestAccountId, SubsystemType, TestDone](int32 PostLoggedOutLocalUserNum, bool bPostLogoutWasSuccessful)
											{
												OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
												OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, FriendAccountId, TestAccountId, SubsystemType, TestDone](int32 PostLoginLocalPlayerNum, bool bPostLoginWasSuccessful, const FUniqueNetId& PostLoginUserId, const FString& PostLoginError)
												{
													OnlineFriends->SendInvite(0, *FriendAccountId, EFriendsLists::ToString(EFriendsLists::Default), FOnSendInviteComplete::CreateLambda([this, FriendAccountId, TestAccountId, SubsystemType, TestDone](int32 bTestSendInviteLocalUserNum, bool bTestSendInviteWasSuccessful, const FUniqueNetId& TestSendInviteFriendId, const FString& TestSendInviteListName, const FString& TestSendInviteErrorStr)
													{
														TestEqual("Verify that SendInviteLocalUserNum is: 0", bTestSendInviteLocalUserNum == 0, true);
														TestEqual("Verify that bSendInviteWasSuccessful returns as: False", bTestSendInviteWasSuccessful, false);
														TestEqual("Verify that SendInviteFriendId is the Id that was originally used", TestSendInviteFriendId.ToString() == FriendAccountId->ToString(), true);
														TestEqual("Verify that SendInviteListName is: Default", TestSendInviteListName == EFriendsLists::ToString(EFriendsLists::Default), true);
														TestEqual("Verify that SendInviteErrorStr returns the expected error code: ONLINE_EXPECTEDERROR_ALREADYFRIENDS", TestSendInviteErrorStr.Contains(ONLINE_EXPECTEDERROR_ALREADYFRIENDS), true);

														TestDone.Execute();
													}));
												}));

												OnlineIdentity->Login(0, AccountCredentials);
											}));

											OnlineIdentity->Logout(0);
										}));
									}));
								}));

								OnlineIdentity->Login(0, FriendAccountCredentials);
							}));

							OnlineIdentity->Logout(0);
						}));
					});

					LatentIt("When calling SendInvite with a valid friend ID and a list name but an invalid local user(-1), this subsystem does not send a friend invite to that friend ID", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_INVALID_LOCALUSER, EAutomationExpectedErrorFlags::Contains, 0);

						FString FriendAccountIdString = FOnlineTestCommon::GetSubsystemFriendAccountUniqueId(SubsystemType);
						FUniqueNetIdPtr FriendAccountId = OnlineIdentity->CreateUniquePlayerId(FriendAccountIdString);

						OnlineFriends->SendInvite(-1, *FriendAccountId, EFriendsLists::ToString(EFriendsLists::Default), FOnSendInviteComplete::CreateLambda([this, FriendAccountId, SubsystemType, TestDone](int32 SendInviteLocalUserNum, bool bSendInviteWasSuccessful, const FUniqueNetId& SendInviteFriendId, const FString& SendInviteListName, const FString& SendInviteErrorStr)
						{
							TestEqual("Verify that SendInviteLocalUserNum is: -1", SendInviteLocalUserNum == -1, true);
							TestEqual("Verify that bSendInviteWasSuccessful returns as: False", bSendInviteWasSuccessful, false);
							TestEqual("Verify that SendInviteFriendId is the Id that was originally used", SendInviteFriendId.ToString() == FriendAccountId->ToString(), true);
							TestEqual("Verify that SendInviteListName is: Default", SendInviteListName == EFriendsLists::ToString(EFriendsLists::Default), true);
							TestEqual("Verify that SendInviteErrorStr returns the expected error code: ONLINE_EXPECTEDERROR_INVALID_LOCALUSER", SendInviteErrorStr.Contains(ONLINE_EXPECTEDERROR_INVALID_LOCALUSER), true);

							TestDone.Execute();
						}));
					});

					LatentIt("When calling SendInvite with a valid local user and a list name but an invalid friend ID, this subsystem does not send a friend invite to that friend ID", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						FString FriendAccountIdString = TEXT(" ");
						FUniqueNetIdPtr FriendAccountId = OnlineIdentity->CreateUniquePlayerId(FriendAccountIdString);

						AddExpectedError(ONLINE_EXPECTEDERROR_ACCOUNT_DOESNOTEXIST, EAutomationExpectedErrorFlags::Contains, 0);

						OnlineFriends->SendInvite(0, *FriendAccountId, EFriendsLists::ToString(EFriendsLists::Default), FOnSendInviteComplete::CreateLambda([this, FriendAccountIdString, TestDone](int32 SendInviteLocalUserNum, bool bSendInviteWasSuccessful, const FUniqueNetId& SendInviteFriendId, const FString& SendInviteListName, const FString& SendInviteErrorStr)
						{
							TestEqual("Verify that SendInviteLocalUserNum is 0", SendInviteLocalUserNum == 0, true);
							TestEqual("Verify that bSendInviteWasSuccessful return as: False", bSendInviteWasSuccessful, false);
							TestEqual("Verify that SendInviteFriendId is the Id that was originally used", SendInviteFriendId.ToString() == FriendAccountIdString, true);
							TestEqual("Verify that SendInviteListName is: Default", SendInviteListName == EFriendsLists::ToString(EFriendsLists::Default), true);
							TestEqual("Verify that SendInviteErrorStr returns the expected error code: ONLINE_EXPECTEDERROR_ACCOUNT_DOESNOTEXIST", SendInviteErrorStr.Contains(ONLINE_EXPECTEDERROR_ACCOUNT_DOESNOTEXIST), true);

							TestDone.Execute();
						}));
					});

					// TODO: Shouldn't there be an error here when the list name is bad?
					LatentIt("When calling SendInvite with a valid local user and a valid friend ID but an invalid list name, this subsystem does not send a friend invite to that friend ID", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_INVALID_LOCALUSER, EAutomationExpectedErrorFlags::Contains, 0);

						FString FriendAccountIdString = FOnlineTestCommon::GetSubsystemFriendAccountUniqueId(SubsystemType);
						FUniqueNetIdPtr FriendAccountId = OnlineIdentity->CreateUniquePlayerId(FriendAccountIdString);

						OnlineFriends->SendInvite(0, *FriendAccountId, TEXT("InvalidListName"), FOnSendInviteComplete::CreateLambda([this, FriendAccountId, SubsystemType, TestDone](int32 SendInviteLocalUserNum, bool bSendInviteWasSuccessful, const FUniqueNetId& SendInviteFriendId, const FString& SendInviteListName, const FString& SendInviteErrorStr)
						{
							TestEqual("Verify that SendInviteLocalUserNum is: 0", SendInviteLocalUserNum == 0, true);
							TestEqual("Verify that bSendInviteWasSuccessful returns as: False", bSendInviteWasSuccessful, false);
							TestEqual("Verify that SendInviteFriendId is the Id that was originally used", SendInviteFriendId.ToString() == FriendAccountId->ToString(), true);
							TestEqual("Verify that SendInviteListName is: InvalidListName", SendInviteListName == TEXT("InvalidListName"), true);
							TestEqual("Verify that SendInviteErrorStr returns the expected error code: ONLINE_EXPECTEDERROR_INVALID_LOCALUSER", SendInviteErrorStr.Contains(ONLINE_EXPECTEDERROR_INVALID_LOCALUSER), true);

							UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: Test implementation not yet complete. Need to verify that different friends lists produce different results"));
							TestDone.Execute();
						}));
					});

					LatentAfterEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoggedOutLocalUserNum, bool bLogoutWasSuccessful)
						{
							CommonUtils.RejectInviteOnFriendAccount(OnlineIdentity, OnlineFriends, SubsystemType, TestDone);
							OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
						}));

						OnlineIdentity->Logout(0);
					});
				});

				Describe("AcceptInvite", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						CommonUtils.SendInviteToFriendAccount(OnlineIdentity, OnlineFriends, SubsystemType, TestDone);
					});

					LatentIt("When calling AcceptInvite with a valid local user, a valid friend ID, and a list name, this subsystem accepts a friend invite from that friend ID", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnlineFriends->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateLambda([this, SubsystemType, TestDone](int32 ReadFriendsListLocalPlayerNum, bool bReadFriendsListWasSuccessful, const FString& ReadFriendsListListName, const FString& ReadFriendsListErrorStr)
							{
								TestEqual("Verify that bReadFriendsListWasSuccessful returns as: True", bReadFriendsListWasSuccessful, true);

								FString TestAccountIdString = FOnlineTestCommon::GetSubsystemTestAccountUniqueId(SubsystemType);
								FUniqueNetIdPtr TestAccountId = OnlineIdentity->CreateUniquePlayerId(TestAccountIdString);

								OnlineFriends->AcceptInvite(0, *TestAccountId, EFriendsLists::ToString(EFriendsLists::Default), FOnAcceptInviteComplete::CreateLambda([this, SubsystemType, TestAccountIdString, TestDone](int32 AcceptInviteLocalPlayerNum, bool bAcceptInviteWasSuccessful, const FUniqueNetId& AcceptInviteFriendId, const FString& AcceptInviteListName, const FString& AcceptInviteErrorStr)
								{
									TestEqual("Verify that AcceptInviteLocalPlayerNum is: 0", AcceptInviteLocalPlayerNum == 0, true);
									TestEqual("Verify that bAcceptInviteWasSuccessful returns as: True", bAcceptInviteWasSuccessful, true);
									TestEqual("Verify that AcceptInviteFriendId is the Id that was originally used", AcceptInviteFriendId.ToString() == TestAccountIdString, true);
									TestEqual("Verify that AcceptInviteListName is: Default", AcceptInviteListName == EFriendsLists::ToString(EFriendsLists::Default), true);
									TestEqual("Verify that AcceptInviteErrorStr is unpopulated", AcceptInviteErrorStr.Len() == 0, true);

									OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
									OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoggedOutLocalUserNum, bool bLogoutWasSuccessful)
									{
										OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
										CommonUtils.RemoveFriendFromTestAccount(OnlineIdentity, OnlineFriends, SubsystemType, TestDone);
										OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
									}));

									OnlineIdentity->Logout(0);
								}));
							}));
						}));

						OnlineIdentity->Login(0, FriendAccountCredentials);
					});

					LatentIt("When calling AcceptInvite with a valid friend ID and a list name but an invalid local user(-1), this subsystem does not accept a friend invite from that friend ID", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_INVALID_LOCALUSER, EAutomationExpectedErrorFlags::Contains, 0);

						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: True", bLoginWasSuccessful, true);

							OnlineFriends->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateLambda([this, SubsystemType, TestDone](int32 ReadFriendsListLocalPlayerNum, bool bReadFriendsListWasSuccessful, const FString& ReadFriendsListListName, const FString& ReadFriendsListErrorStr)
							{
								TestEqual("Verify that bReadFriendsListWasSuccessful returns as: True", bReadFriendsListWasSuccessful, true);

								FString TestAccountIdString = FOnlineTestCommon::GetSubsystemTestAccountUniqueId(SubsystemType);
								FUniqueNetIdPtr TestAccountId = OnlineIdentity->CreateUniquePlayerId(TestAccountIdString);

								OnlineFriends->AcceptInvite(-1, *TestAccountId, EFriendsLists::ToString(EFriendsLists::Default), FOnAcceptInviteComplete::CreateLambda([this, SubsystemType, TestAccountIdString, TestDone](int32 AcceptInviteLocalPlayerNum, bool bAcceptInviteWasSuccessful, const FUniqueNetId& AcceptInviteFriendId, const FString& AcceptInviteListName, const FString& AcceptInviteErrorStr)
								{
									TestEqual("Verify that AcceptInviteLocalPlayerNum is: -1", AcceptInviteLocalPlayerNum == -1, true);
									TestEqual("Verify that bAcceptInviteWasSuccessful returns as: False", bAcceptInviteWasSuccessful, false);
									TestEqual("Verify that AcceptInviteFriendId is the Id that was originally used", AcceptInviteFriendId.ToString() == TestAccountIdString, true);
									TestEqual("Verify that AcceptInviteListName is: Default", AcceptInviteListName == EFriendsLists::ToString(EFriendsLists::Default), true);
									TestEqual("Verify that AcceptInviteErrorStr returns the expected error code:  ONLINE_EXPECTEDERROR_INVALID_LOCALUSER", AcceptInviteErrorStr.Contains(ONLINE_EXPECTEDERROR_INVALID_LOCALUSER), true);

									OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
									OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoggedOutLocalUserNum, bool bLogoutWasSuccessful)
									{
										OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
										CommonUtils.RejectInviteOnFriendAccount(OnlineIdentity, OnlineFriends, SubsystemType, TestDone);
										OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
									}));

									OnlineIdentity->Logout(0);
								}));
							}));
						}));

						OnlineIdentity->Login(0, FriendAccountCredentials);
					});

					LatentIt("When calling AcceptInvite with a valid local user and a list name but an invalid friend ID, this subsystem does not accept a friend invite from that friend ID", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						FString TestAccountIdString = TEXT(" ");

						AddExpectedError(ONLINE_EXPECTEDERROR_NOCACHEDFRIEND, EAutomationExpectedErrorFlags::Contains, 0);

						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestAccountIdString, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: True", bLoginWasSuccessful, true);

							OnlineFriends->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateLambda([this, TestAccountIdString, SubsystemType, TestDone](int32 ReadFriendsListLocalPlayerNum, bool bReadFriendsListWasSuccessful, const FString& ReadFriendsListListName, const FString& ReadFriendsListErrorStr)
							{
								TestEqual("Verify that bReadFriendsListWasSuccessful returns as: True", bReadFriendsListWasSuccessful, true);

								FUniqueNetIdPtr TestAccountId = OnlineIdentity->CreateUniquePlayerId(TestAccountIdString);

								OnlineFriends->AcceptInvite(0, *TestAccountId, EFriendsLists::ToString(EFriendsLists::Default), FOnAcceptInviteComplete::CreateLambda([this, SubsystemType, TestAccountIdString, TestDone](int32 AcceptInviteLocalPlayerNum, bool bAcceptInviteWasSuccessful, const FUniqueNetId& AcceptInviteFriendId, const FString& AcceptInviteListName, const FString& AcceptInviteErrorStr)
								{
									TestEqual("Verify that AcceptInviteLocalPlayerNum is: 0", AcceptInviteLocalPlayerNum == 0, true);
									TestEqual("Verify that bAcceptInviteWasSuccessful returns as: False", bAcceptInviteWasSuccessful, false);
									TestEqual("Verify that AcceptInviteFriendId is the Id that was originally used", AcceptInviteFriendId.ToString() == TestAccountIdString, true);
									TestEqual("Verify that AcceptInviteListName is: Default", AcceptInviteListName == EFriendsLists::ToString(EFriendsLists::Default), true);
									TestEqual("Verify that AcceptInviteErrorStr returns the expected error code: ONLINE_EXPECTEDERROR_NOCACHEDFRIEND", AcceptInviteErrorStr.Contains(ONLINE_EXPECTEDERROR_NOCACHEDFRIEND), true);

									OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
									OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoggedOutLocalUserNum, bool bLogoutWasSuccessful)
									{
										OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
										CommonUtils.RejectInviteOnFriendAccount(OnlineIdentity, OnlineFriends, SubsystemType, TestDone);
										OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
									}));

									OnlineIdentity->Logout(0);
								}));
							}));
						}));

						OnlineIdentity->Login(0, FriendAccountCredentials);
					});

					// TODO: Invalid list name should throw an error?
					LatentIt("When calling AcceptInvite with a valid local user and a valid friend ID but an invalid list name, this subsystem does not accept a friend invite from that friend ID", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnlineFriends->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateLambda([this, SubsystemType, TestDone](int32 ReadFriendsListLocalPlayerNum, bool bReadFriendsListWasSuccessful, const FString& ReadFriendsListListName, const FString& ReadFriendsListErrorStr)
							{
								TestEqual("Verify that bReadFriendsListWasSuccessful returns as: True", bReadFriendsListWasSuccessful, true);

								FString TestAccountIdString = FOnlineTestCommon::GetSubsystemTestAccountUniqueId(SubsystemType);
								FUniqueNetIdPtr TestAccountId = OnlineIdentity->CreateUniquePlayerId(TestAccountIdString);

								OnlineFriends->AcceptInvite(0, *TestAccountId, TEXT("InvalidListName"), FOnAcceptInviteComplete::CreateLambda([this, SubsystemType, TestAccountIdString, TestDone](int32 AcceptInviteLocalPlayerNum, bool bAcceptInviteWasSuccessful, const FUniqueNetId& AcceptInviteFriendId, const FString& AcceptInviteListName, const FString& AcceptInviteErrorStr)
								{
									TestEqual("Verify that AcceptInviteLocalPlayerNum is: 0", AcceptInviteLocalPlayerNum == 0, true);
									TestEqual("Verify that bAcceptInviteWasSuccessful returns as: True", bAcceptInviteWasSuccessful, true);
									TestEqual("Verify that AcceptInviteFriendId is the Id that was originally used", AcceptInviteFriendId.ToString() == TestAccountIdString, true);
									TestEqual("Verify that AcceptInviteListName is: InvalidListName", AcceptInviteListName == TEXT("InvalidListName"), true);
									TestEqual("Verify that AcceptInviteErrorStr is unpopulated", AcceptInviteErrorStr.Len() == 0, true);

									OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
									OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoggedOutLocalUserNum, bool bLogoutWasSuccessful)
									{
										OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
										CommonUtils.RemoveFriendFromTestAccount(OnlineIdentity, OnlineFriends, SubsystemType, TestDone);
										OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
									}));

									OnlineIdentity->Logout(0);
								}));
							}));
						}));

						OnlineIdentity->Login(0, FriendAccountCredentials);
					});
				});

				Describe("RejectInvite", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						CommonUtils.SendInviteToFriendAccount(OnlineIdentity, OnlineFriends, SubsystemType, TestDone);
					});

					LatentIt("When calling RejectInvite with a valid local user, a valid friend ID, and a list name, this subsystem rejects a friend invite from that friend ID", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnlineFriends->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateLambda([this, SubsystemType, TestDone](int32 ReadFriendsListLocalPlayerNum, bool bReadFriendsListWasSuccessful, const FString& ReadFriendsListListName, const FString& ReadFriendsListErrorStr)
							{
								TestEqual("Verify that bReadFriendsListWasSuccessful returns as: True", bReadFriendsListWasSuccessful, true);

								FString TestAccountIdString = FOnlineTestCommon::GetSubsystemTestAccountUniqueId(SubsystemType);
								FUniqueNetIdPtr TestAccountId = OnlineIdentity->CreateUniquePlayerId(TestAccountIdString);

								OnRejectInviteCompleteDelegateHandle = OnlineFriends->AddOnRejectInviteCompleteDelegate_Handle(0, FOnRejectInviteCompleteDelegate::CreateLambda([this, SubsystemType, TestAccountIdString, TestDone](int32 RejectInviteLocalPlayerNum, bool bRejectInviteWasSuccessful, const FUniqueNetId& RejectInviteFriendId, const FString& RejectInviteListName, const FString& RejectInviteErrorStr)
								{
									TestEqual("Verify that RejectInviteLocalPlayerNum is: 0", RejectInviteLocalPlayerNum == 0, true);
									TestEqual("Verify that bRejectInviteWasSuccessful returns as: True", bRejectInviteWasSuccessful, true);
									TestEqual("Verify that RejectInviteFriendId is the Id that was originally used", RejectInviteFriendId.ToString() == TestAccountIdString, true);
									TestEqual("Verify that RejectInviteListName is: Default", RejectInviteListName == EFriendsLists::ToString(EFriendsLists::Default), true);
									TestEqual("Verify that RejectInviteErrorStr is unpopulated", RejectInviteErrorStr.Len() == 0, true);

									TestDone.Execute();
								}));

								OnlineFriends->RejectInvite(0, *TestAccountId, EFriendsLists::ToString(EFriendsLists::Default));
							}));
						}));

						OnlineIdentity->Login(0, FriendAccountCredentials);
					});

					// OGS-1023: Macro that builds TriggerDELEGATE does not accept a negative LocalUserNum to trigger the delegate
					LatentIt("When calling RejectInvite with a valid friend ID and list name but an invalid local user (-1), this subsystem does not reject a friend invite from that friend ID", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_INVALID_LOCALUSER, EAutomationExpectedErrorFlags::Contains, 0);

						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnlineFriends->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateLambda([this, SubsystemType, TestDone](int32 ReadFriendsListLocalPlayerNum, bool bReadFriendsListWasSuccessful, const FString& ReadFriendsListListName, const FString& ReadFriendsListErrorStr)
							{
								TestEqual("Verify that bReadFriendsListWasSuccessful returns as: True", bReadFriendsListWasSuccessful, true);

								FString TestAccountIdString = FOnlineTestCommon::GetSubsystemTestAccountUniqueId(SubsystemType);
								FUniqueNetIdPtr TestAccountId = OnlineIdentity->CreateUniquePlayerId(TestAccountIdString);

								OnRejectInviteCompleteDelegateHandle = OnlineFriends->AddOnRejectInviteCompleteDelegate_Handle(0, FOnRejectInviteCompleteDelegate::CreateLambda([this, SubsystemType, TestAccountIdString, TestDone](int32 RejectInviteLocalPlayerNum, bool bRejectInviteWasSuccessful, const FUniqueNetId& RejectInviteFriendId, const FString& RejectInviteListName, const FString& RejectInviteErrorStr)
								{
									TestEqual("Verify that RejectInviteLocalPlayerNum is: -1", RejectInviteLocalPlayerNum == -1, true);
									TestEqual("Verify that bRejectInviteWasSuccessful returns as: False", bRejectInviteWasSuccessful, false);
									TestEqual("Verify that RejectInviteFriendId is the Id that was originally used", RejectInviteFriendId.ToString() == TestAccountIdString, true);
									TestEqual("Verify that RejectInviteListName is: Default", RejectInviteListName == EFriendsLists::ToString(EFriendsLists::Default), true);
									TestEqual("Verify that RejectInviteErrorStr returns the expected error code: ONLINE_EXPECTEDERROR_INVALID_LOCALUSER", RejectInviteErrorStr.Contains(ONLINE_EXPECTEDERROR_INVALID_LOCALUSER), true);

									UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: Test implementation not yet complete. Bug: OGS-1023 - Macro that builds TriggerDELEGATE does not accept a negative LocalUserNum to trigger the delegate"));
									TestDone.Execute();
								}));

								OnlineFriends->RejectInvite(-1, *TestAccountId, EFriendsLists::ToString(EFriendsLists::Default));
							}));
						}));

						OnlineIdentity->Login(0, FriendAccountCredentials);
					});

					LatentIt("When calling RejectInvite with a valid local user and list name but an invalid friend ID, this subsystem does not reject a friend invite from that friend ID", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						FString TestAccountIdString = TEXT(" ");

						AddExpectedError(ONLINE_EXPECTEDERROR_NOCACHEDFRIEND, EAutomationExpectedErrorFlags::Contains, 0);

						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestAccountIdString, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnlineFriends->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateLambda([this, TestAccountIdString, SubsystemType, TestDone](int32 ReadFriendsListLocalPlayerNum, bool bReadFriendsListWasSuccessful, const FString& ReadFriendsListListName, const FString& ReadFriendsListErrorStr)
							{
								TestEqual("Verify that bReadFriendsListWasSuccessful returns as: True", bReadFriendsListWasSuccessful, true);

								FUniqueNetIdPtr TestAccountId = OnlineIdentity->CreateUniquePlayerId(TestAccountIdString);

								OnRejectInviteCompleteDelegateHandle = OnlineFriends->AddOnRejectInviteCompleteDelegate_Handle(0, FOnRejectInviteCompleteDelegate::CreateLambda([this, SubsystemType, TestAccountIdString, TestDone](int32 RejectInviteLocalPlayerNum, bool bRejectInviteWasSuccessful, const FUniqueNetId& RejectInviteFriendId, const FString& RejectInviteListName, const FString& RejectInviteErrorStr)
								{
									TestEqual("Verify that RejectInviteLocalPlayerNum is: 0", RejectInviteLocalPlayerNum == 0, true);
									TestEqual("Verify that bRejectInviteWasSuccessful returns as: False", bRejectInviteWasSuccessful, false);
									TestEqual("Verify that RejectInviteFriendId is the Id that was originally used", RejectInviteFriendId.ToString() == TestAccountIdString, true);
									TestEqual("Verify that RejectInviteListName is: Default", RejectInviteListName == EFriendsLists::ToString(EFriendsLists::Default), true);
									TestEqual("Verify that RejectInviteErrorStr returns the expected error code: ONLINE_EXPECTEDERROR_NOCACHEDFRIEND", RejectInviteErrorStr.Contains(ONLINE_EXPECTEDERROR_NOCACHEDFRIEND), true);

									TestDone.Execute();
								}));

								OnlineFriends->RejectInvite(0, *TestAccountId, EFriendsLists::ToString(EFriendsLists::Default));
							}));
						}));

						OnlineIdentity->Login(0, FriendAccountCredentials);
					});

					// TODO: Should an invalid list name throw an error?
					LatentIt("When calling RejectInvite with a valid local user and friend ID but an invalid list name, this subsystem does not reject a friend invite from that friend ID", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnlineFriends->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateLambda([this, SubsystemType, TestDone](int32 ReadFriendsListLocalPlayerNum, bool bReadFriendsListWasSuccessful, const FString& ReadFriendsListListName, const FString& ReadFriendsListErrorStr)
							{
								TestEqual("Verify that bReadFriendsListWasSuccessful returns as: True", bReadFriendsListWasSuccessful, true);

								FString TestAccountIdString = FOnlineTestCommon::GetSubsystemTestAccountUniqueId(SubsystemType);
								FUniqueNetIdPtr TestAccountId = OnlineIdentity->CreateUniquePlayerId(TestAccountIdString);

								OnRejectInviteCompleteDelegateHandle = OnlineFriends->AddOnRejectInviteCompleteDelegate_Handle(0, FOnRejectInviteCompleteDelegate::CreateLambda([this, SubsystemType, TestAccountIdString, TestDone](int32 RejectInviteLocalPlayerNum, bool bRejectInviteWasSuccessful, const FUniqueNetId& RejectInviteFriendId, const FString& RejectInviteListName, const FString& RejectInviteErrorStr)
								{
									TestEqual("Verify that RejectInviteLocalPlayerNum is: 0", RejectInviteLocalPlayerNum == 0, true);
									TestEqual("Verify that bRejectInviteWasSuccessful returns as: False", bRejectInviteWasSuccessful, false);
									TestEqual("Verify that RejectInviteFriendId is the Id that was originally used", RejectInviteFriendId.ToString() == TestAccountIdString, true);
									TestEqual("Verify that RejectInviteListName is: InvalidListName", RejectInviteListName == TEXT("InvalidListName"), true);
									TestEqual("Verify that RejectInviteErrorStr returns the expected error code: ", RejectInviteErrorStr.Contains(TEXT("something")), true);

									UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: Test implementation not yet complete. Need to verify that different friends lists produce different results"));
									TestDone.Execute();
								}));

								OnlineFriends->RejectInvite(0, *TestAccountId, TEXT("InvalidListName"));
							}));
						}));

						OnlineIdentity->Login(0, FriendAccountCredentials);
					});

					LatentAfterEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoggedOutLocalUserNum, bool bLogoutWasSuccessful)
						{
							CommonUtils.RejectInviteOnFriendAccount(OnlineIdentity, OnlineFriends, SubsystemType, TestDone);
							OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
						}));

						OnlineIdentity->Logout(0);
					});
				});

				Describe("DeleteFriend", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						CommonUtils.AddFriendToTestAccount(OnlineIdentity, OnlineFriends, SubsystemType, TestDone);
					});

					LatentIt("When calling DeleteFriend with a valid local user, a valid friend ID, and a list name, this subsystem deletes that friend ID", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnlineFriends->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateLambda([this, SubsystemType, TestDone](int32 ReadFriendsListLocalPlayerNumBeforeDelete, bool bReadFriendsListWasSuccessfulBeforeDelete, const FString& ReadFriendsListListNameBeforeDelete, const FString& ReadFriendsListErrorStrBeforeDelete)
							{
								FString FriendAccountIdString = FOnlineTestCommon::GetSubsystemFriendAccountUniqueId(SubsystemType);
								FUniqueNetIdPtr FriendAccountId = OnlineIdentity->CreateUniquePlayerId(FriendAccountIdString);

								OnDeleteFriendCompleteDelegateHandle = OnlineFriends->AddOnDeleteFriendCompleteDelegate_Handle(0, FOnDeleteFriendCompleteDelegate::CreateLambda([this, FriendAccountIdString, TestDone](int32 DeleteFriendLocalPlayerNum, bool bDeleteFriendWasSuccessful, const FUniqueNetId& DeleteFriendFriendId, const FString& DeleteFriendListName, const FString& DeleteFriendErrorStr)
								{
									TestEqual("Verify that DeleteFriendLocalPlayerNum is: 0", DeleteFriendLocalPlayerNum == 0, true);
									TestEqual("Verify that bDeleteFriendWasSuccessful returns as: True", bDeleteFriendWasSuccessful, true);
									TestEqual("Verify that DeleteFriendFriendId is the Id that was originally used", DeleteFriendFriendId.ToString() == FriendAccountIdString, true);
									TestEqual("Verify that DeleteFriendListName is: Default", DeleteFriendListName == EFriendsLists::ToString(EFriendsLists::Default), true);
									TestEqual("Verify that DeleteFriendErrorStr is unpopulated", DeleteFriendErrorStr.Len() == 0, true);

									OnlineFriends->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateLambda([this, TestDone](int32 ReadFriendsListLocalPlayerNumAfterDelete, bool bReadFriendsListWasSuccessfulAfterDelete, const FString& ReadFriendsListListNameAfterDelete, const FString& ReadFriendsListErrorStrAfterDelete)
									{
										TArray<TSharedRef<FOnlineFriend>> FriendsList;
										OnlineFriends->GetFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FriendsList);

										TestEqual("Verify that FriendsList is unpopulated", FriendsList.Num() == 0, true);

										TestDone.Execute();
									}));
								}));

								OnlineFriends->DeleteFriend(0, *FriendAccountId, EFriendsLists::ToString(EFriendsLists::Default));
							}));
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					// OGS-1023: Macro that builds TriggerDELEGATE does not accept a negative LocalUserNum to trigger the delegate
					LatentIt("When calling DeleteFriend with a valid friend ID and list name but an invalid local user (-1), this subsystem does not delete that friend ID", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_INVALID_LOCALUSER, EAutomationExpectedErrorFlags::Contains, 0);

						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnlineFriends->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateLambda([this, SubsystemType, TestDone](int32 ReadFriendsListLocalPlayerNum, bool bReadFriendsListWasSuccessful, const FString& ReadFriendsListListName, const FString& ReadFriendsListErrorStr)
							{
								FString FriendAccountIdString = FOnlineTestCommon::GetSubsystemFriendAccountUniqueId(SubsystemType);
								FUniqueNetIdPtr FriendAccountId = OnlineIdentity->CreateUniquePlayerId(FriendAccountIdString);

								OnDeleteFriendCompleteDelegateHandle = OnlineFriends->AddOnDeleteFriendCompleteDelegate_Handle(0, FOnDeleteFriendCompleteDelegate::CreateLambda([this, FriendAccountIdString, TestDone](int32 DeleteFriendLocalPlayerNum, bool bDeleteFriendWasSuccessful, const FUniqueNetId& DeleteFriendFriendId, const FString& DeleteFriendListName, const FString& DeleteFriendErrorStr)
								{
									TestEqual("Verify that DeleteFriendLocalPlayerNum is: -1", DeleteFriendLocalPlayerNum == -1, true);
									TestEqual("Verify that bDeleteFriendWasSuccessful returns as: False", bDeleteFriendWasSuccessful, false);
									TestEqual("Verify that DeleteFriendFriendId is the Id that was originally used", DeleteFriendFriendId.ToString() == FriendAccountIdString, true);
									TestEqual("Verify that DeleteFriendListName is: Default", DeleteFriendListName == EFriendsLists::ToString(EFriendsLists::Default), true);
									TestEqual("Verify that DeleteFriendErrorStr returns the expected error code: ONLINE_EXPECTEDERROR_INVALID_LOCALUSER", DeleteFriendErrorStr.Contains(ONLINE_EXPECTEDERROR_INVALID_LOCALUSER), true);

									OnlineFriends->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateLambda([this, TestDone](int32 , bool , const FString& , const FString& )
									{
										TArray<TSharedRef<FOnlineFriend>> FriendsList;
										OnlineFriends->GetFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FriendsList);

										TestEqual("Verify that FriendsList is populated", FriendsList.Num() > 0, true);

										UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: Test implementation not yet complete. Bug: OGS-1023 - Macro that builds TriggerDELEGATE does not accept a negative LocalUserNum to trigger the delegate"));
										TestDone.Execute();
									}));
								}));

								OnlineFriends->DeleteFriend(-1, *FriendAccountId, EFriendsLists::ToString(EFriendsLists::Default));
							}));
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling DeleteFriend with a valid local user and list name but an invalid friend ID, this subsystem does not delete that friend ID", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						FString FriendAccountIdString = TEXT(" ");

						AddExpectedError(ONLINE_EXPECTEDERROR_NOCACHEDFRIEND, EAutomationExpectedErrorFlags::Contains, 0);

						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, FriendAccountIdString, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnlineFriends->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateLambda([this, SubsystemType, FriendAccountIdString, TestDone](int32 ReadFriendsListLocalPlayerNum, bool bReadFriendsListWasSuccessful, const FString& ReadFriendsListListName, const FString& ReadFriendsListErrorStr)
							{
								FUniqueNetIdPtr FriendAccountId = OnlineIdentity->CreateUniquePlayerId(FriendAccountIdString);

								OnDeleteFriendCompleteDelegateHandle = OnlineFriends->AddOnDeleteFriendCompleteDelegate_Handle(0, FOnDeleteFriendCompleteDelegate::CreateLambda([this, FriendAccountIdString, TestDone](int32 DeleteFriendLocalPlayerNum, bool bDeleteFriendWasSuccessful, const FUniqueNetId& DeleteFriendFriendId, const FString& DeleteFriendListName, const FString& DeleteFriendErrorStr)
								{
									TestEqual("Verify that DeleteFriendLocalPlayerNum is: 0", DeleteFriendLocalPlayerNum == 0, true);
									TestEqual("Verify that bDeleteFriendWasSuccessful returns as: False", bDeleteFriendWasSuccessful, false);
									TestEqual("Verify that DeleteFriendFriendId is the Id that was originally used", DeleteFriendFriendId.ToString() == FriendAccountIdString, true);
									TestEqual("Verify that DeleteFriendListName is: Default", DeleteFriendListName == EFriendsLists::ToString(EFriendsLists::Default), true);
									TestEqual("Verify that DeleteFriendErrorStr returns the expected error code: ONLINE_EXPECTEDERROR_NOCACHEDFRIEND", DeleteFriendErrorStr.Contains(ONLINE_EXPECTEDERROR_NOCACHEDFRIEND), true);

									OnlineFriends->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateLambda([this, TestDone](int32 ReadLocalPlayerNum, bool bReadWasSuccessful, const FString& ReadListName, const FString& ReadErrorStr)
									{
										TArray<TSharedRef<FOnlineFriend>> FriendsList;
										OnlineFriends->GetFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FriendsList);

										TestEqual("Verify that FriendsList is populated", FriendsList.Num() > 0, true);

										TestDone.Execute();
									}));
								}));

								OnlineFriends->DeleteFriend(0, *FriendAccountId, EFriendsLists::ToString(EFriendsLists::Default));
							}));
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					It("When calling DeleteFriend with a valid local user and friend ID but an invalid list name, this subsystem does not delete that friend ID", [this, SubsystemType]()
					{
						UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: Test implementation not yet complete. Need to verify that different friends lists produce different results"));
					});

					LatentAfterEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineFriends->ClearOnDeleteFriendCompleteDelegate_Handle(0, OnDeleteFriendCompleteDelegateHandle);

						OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoggedOutLocalUserNum, bool bLogoutWasSuccessful)
						{
							CommonUtils.RemoveFriendFromTestAccount(OnlineIdentity, OnlineFriends, SubsystemType, TestDone);
							OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
						}));

						OnlineIdentity->Logout(0);
					});
				});

				Describe("GetFriendsList", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						CommonUtils.AddFriendToTestAccount(OnlineIdentity, OnlineFriends, SubsystemType, TestDone);
					});

					LatentIt("When calling GetFriendsList with a valid local user and list name after polling for friends list data, this subsystem will return that data", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnlineFriends->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateLambda([this, TestDone](int32 ReadFriendsListLocalPlayerNum, bool bReadFriendsListWasSuccessful, const FString& ReadFriendsListListName, const FString& ReadFriendsListErrorStr)
							{
								TestEqual("Verify that bReadFriendsListWasSuccessful returns as: True", bReadFriendsListWasSuccessful, true);

								TArray<TSharedRef<FOnlineFriend>> FriendsList;
								bool bGetFriendsListWasSuccessful = OnlineFriends->GetFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FriendsList);

								TestEqual("Verify that bGetFriendsListWasSuccessful returns as: True", bGetFriendsListWasSuccessful, true);
								TestEqual("Verify that FriendsList is populated", FriendsList.Num() > 0, true);

								TestDone.Execute();
							}));
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling GetFriendsList with a valid local user and list name without polling for friends list data, this subsystem will not return that data", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TArray<TSharedRef<FOnlineFriend>> FriendsList;
							bool bGetFriendsListWasSuccessful = OnlineFriends->GetFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FriendsList);

							TestEqual("Verify that bGetFriendsListWasSuccessful returns as: False", bGetFriendsListWasSuccessful, false);
							TestEqual("Verify that FriendsList is unpopulated", FriendsList.Num() == 0, true);

							TestDone.Execute();
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling GetFriendsList with a list name but an invalid local user after polling for friends list data, this subsystem will not return that data", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_INVALID_USERID, EAutomationExpectedErrorFlags::Contains, 0);

						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnlineFriends->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateLambda([this, TestDone](int32 ReadFriendsListLocalPlayerNum, bool bReadFriendsListWasSuccessful, const FString& ReadFriendsListListName, const FString& ReadFriendsListErrorStr)
							{
								TestEqual("Verify that ReadFriendsList bWasSuccessful is true", bReadFriendsListWasSuccessful, true);

								TArray<TSharedRef<FOnlineFriend>> FriendsList;
								bool bGetFriendsListWasSuccessful = OnlineFriends->GetFriendsList(-1, EFriendsLists::ToString(EFriendsLists::Default), FriendsList);

								TestEqual("Verify that bGetFriendsListWasSuccessful returns as: False", bGetFriendsListWasSuccessful, false);
								TestEqual("Verify that FriendsList is unpopulated", FriendsList.Num() == 0, true);

								TestDone.Execute();
							}));
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					It("When calling GetFriendsList with a valid local user but an invalid list name after polling for friends list data, this subsystem will not return that data", [this, SubsystemType]()
					{
						UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: Test implementation not yet complete. Need to verify that different friends lists produce different results"));
					});

					LatentAfterEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineFriends->ClearOnDeleteFriendCompleteDelegate_Handle(0, OnDeleteFriendCompleteDelegateHandle);

						OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoggedOutLocalUserNum, bool bLogoutWasSuccessful)
						{
							CommonUtils.RemoveFriendFromTestAccount(OnlineIdentity, OnlineFriends, SubsystemType, TestDone);
							OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
						}));

						OnlineIdentity->Logout(0);
					});
				});

				Describe("GetFriend", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						CommonUtils.AddFriendToTestAccount(OnlineIdentity, OnlineFriends, SubsystemType, TestDone);
					});

					// TODO: Give this tests more checks against the FriendEntry
					LatentIt("When calling GetFriend with a valid local user, list name, and friend ID after polling for friend data, this subsystem will return that online friend", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnlineFriends->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateLambda([this, SubsystemType, TestDone](int32 ReadFriendsListLocalPlayerNum, bool bReadFriendsListWasSuccessful, const FString& ReadFriendsListListName, const FString& ReadFriendsListErrorStr)
							{
								TestEqual("Verify that bReadFriendsListWasSuccessful returns as: True", bReadFriendsListWasSuccessful, true);

								FString FriendAccountIdString = FOnlineTestCommon::GetSubsystemFriendAccountUniqueId(SubsystemType);
								FUniqueNetIdPtr FriendAccountId = OnlineIdentity->CreateUniquePlayerId(FriendAccountIdString);

								TSharedPtr<FOnlineFriend> FriendEntry = OnlineFriends->GetFriend(0, *FriendAccountId, EFriendsLists::ToString(EFriendsLists::Default));

								if (FriendEntry.IsValid())
								{
									TestEqual("Verify that the returned FriendEntry's ID is the correct ID", *FriendEntry->GetUserId() == *FriendAccountId, true);
								}
								else
								{
									UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: IsValid() check on FriendEntry failed after a call to OnlineFriends->GetFriend()"));
								}
								UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: Test implementation not yet complete. Need more checks against FriendEntry"));
								TestDone.Execute();
							}));
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling GetFriend with a valid local user, list name, and friend ID without polling for friend data, this subsystem will not return that online friend", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: True", bLoginWasSuccessful, true);

							FString FriendAccountIdString = FOnlineTestCommon::GetSubsystemFriendAccountUniqueId(SubsystemType);
							FUniqueNetIdPtr FriendAccountId = OnlineIdentity->CreateUniquePlayerId(FriendAccountIdString);

							TSharedPtr<FOnlineFriend> FriendEntry = OnlineFriends->GetFriend(0, *FriendAccountId, EFriendsLists::ToString(EFriendsLists::Default));

							TestEqual("Verify that the returned FriendEntry pointer is invalid", FriendEntry.IsValid(), false);

							TestDone.Execute();
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling GetFriend with a valid local user and list name but an invalid friend ID after polling for friend data, this subsystem will not return that online friend", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: True", bLoginWasSuccessful, true);

							FString FriendAccountIdString = TEXT(" ");
							FUniqueNetIdPtr FriendAccountId = OnlineIdentity->CreateUniquePlayerId(FriendAccountIdString);

							TSharedPtr<FOnlineFriend> FriendEntry = OnlineFriends->GetFriend(0, *FriendAccountId, EFriendsLists::ToString(EFriendsLists::Default));

							TestEqual("Verify that the returned FriendEntry pointer is invalid", FriendEntry.IsValid(), false);

							TestDone.Execute();
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling GetFriend with a valid list name and friend ID but an invalid local user (-1) after polling for friend data, this subsystem will not return that online friend", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: True", bLoginWasSuccessful, true);

							FString FriendAccountIdString = TEXT("0123456789");
							FUniqueNetIdPtr FriendAccountId = OnlineIdentity->CreateUniquePlayerId(FriendAccountIdString);

							TSharedPtr<FOnlineFriend> FriendEntry = OnlineFriends->GetFriend(-1, *FriendAccountId, EFriendsLists::ToString(EFriendsLists::Default));

							TestEqual("Verify that the returned FriendEntry pointer is invalid", FriendEntry.IsValid(), false);

							TestDone.Execute();
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					It("When calling GetFriend with a valid local user and friend ID but an invalid list name after polling for friend data, this subsystem will not return that online friend", [this, SubsystemType]()
					{
						UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: Test implementation not yet complete. Need to verify that different friends lists produce different results"));
					});

					LatentAfterEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineFriends->ClearOnDeleteFriendCompleteDelegate_Handle(0, OnDeleteFriendCompleteDelegateHandle);

						OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoggedOutLocalUserNum, bool bLogoutWasSuccessful)
						{
							CommonUtils.RemoveFriendFromTestAccount(OnlineIdentity, OnlineFriends, SubsystemType, TestDone);
							OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
						}));

						OnlineIdentity->Logout(0);
					});
				});

				Describe("IsFriend", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						CommonUtils.AddFriendToTestAccount(OnlineIdentity, OnlineFriends, SubsystemType, TestDone);
					});

					LatentIt("When calling IsFriend with a valid local user, list name, and friend ID who is currently on the user's friends list after polling for friend data, this subsystem will return true", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnlineFriends->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateLambda([this, SubsystemType, TestDone](int32 ReadFriendsListLocalPlayerNum, bool bReadFriendsListWasSuccessful, const FString& ReadFriendsListListName, const FString& ReadFriendsListErrorStr)
							{
								TestEqual("Verify that bReadFriendsListWasSuccessful returns as: True", bReadFriendsListWasSuccessful, true);

								FString FriendAccountIdString = FOnlineTestCommon::GetSubsystemFriendAccountUniqueId(SubsystemType);
								FUniqueNetIdPtr FriendAccountId = OnlineIdentity->CreateUniquePlayerId(FriendAccountIdString);

								bool bIsFriend = OnlineFriends->IsFriend(0, *FriendAccountId, EFriendsLists::ToString(EFriendsLists::Default));

								TestEqual("Verify that bIsFriend is: True", bIsFriend, true);

								TestDone.Execute();
							}));
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling IsFriend with a valid local user, list name, and friend ID who is currently on the user's friends list without polling for friend data, this subsystem will return false", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FString FriendAccountIdString = FOnlineTestCommon::GetSubsystemFriendAccountUniqueId(SubsystemType);
							FUniqueNetIdPtr FriendAccountId = OnlineIdentity->CreateUniquePlayerId(FriendAccountIdString);

							bool bIsFriend = OnlineFriends->IsFriend(0, *FriendAccountId, EFriendsLists::ToString(EFriendsLists::Default));

							TestEqual("Verify that bIsFriend is: False", bIsFriend, false);

							TestDone.Execute();
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling IsFriend with a valid local user, list name, and friend ID who is not on the user's friends list after polling for friend data, this subsystem will return false", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnlineFriends->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateLambda([this, SubsystemType, TestDone](int32 ReadFriendsListLocalPlayerNum, bool bReadFriendsListWasSuccessful, const FString& ReadFriendsListListName, const FString& ReadFriendsListErrorStr)
							{
								TArray<TSharedRef<FOnlineFriend>> FriendsList;
								OnlineFriends->GetFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FriendsList);
								if (FriendsList.Num() > 0)
								{
									FString FriendAccountIdString = FOnlineTestCommon::GetSubsystemFriendAccountUniqueId(SubsystemType);
									FUniqueNetIdPtr FriendAccountId = OnlineIdentity->CreateUniquePlayerId(FriendAccountIdString);

									OnlineFriends->ClearOnDeleteFriendCompleteDelegate_Handle(0, OnDeleteFriendCompleteDelegateHandle);
									OnDeleteFriendCompleteDelegateHandle = OnlineFriends->AddOnDeleteFriendCompleteDelegate_Handle(0, FOnDeleteFriendCompleteDelegate::CreateLambda([this, FriendAccountId, SubsystemType, TestDone](int32 DeleteFriendLocalUserNum, bool bDeleteFriendWasSuccessful, const FUniqueNetId& DeleteFriendFriendId, const FString& DeleteFriendListName, const FString& DeleteFriendErrorStr)
									{
										OnlineFriends->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateLambda([this, FriendAccountId, SubsystemType, TestDone](int32 , bool , const FString& , const FString& )
										{
											bool bIsFriend = OnlineFriends->IsFriend(0, *FriendAccountId, EFriendsLists::ToString(EFriendsLists::Default));

											TestEqual("Verify that bIsFriend is: False", bIsFriend, false);

											TestDone.Execute();
										}));
									}));

									OnlineFriends->DeleteFriend(0, *FriendAccountId, EFriendsLists::ToString(EFriendsLists::Default));
								}
								else
								{
									UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: FriendsList was empty after calling GetFriendsList(). Expected 1 friend entry"));
									TestDone.Execute();
								}
							}));
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling IsFriend with a valid local user and list name but an invalid friend ID after polling for friend data, this subsystem will return false", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnlineFriends->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateLambda([this, SubsystemType, TestDone](int32 ReadFriendsListLocalPlayerNum, bool bReadFriendsListWasSuccessful, const FString& ReadFriendsListListName, const FString& ReadFriendsListErrorStr)
							{
								FString FriendAccountIdString = TEXT(" ");
								FUniqueNetIdPtr FriendAccountId = OnlineIdentity->CreateUniquePlayerId(FriendAccountIdString);

								bool bIsFriend = OnlineFriends->IsFriend(0, *FriendAccountId, EFriendsLists::ToString(EFriendsLists::Default));

								TestEqual("Verify that bIsFriend is: False", bIsFriend, false);

								TestDone.Execute();
							}));
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling IsFriend with a valid list name and friend ID who is on the user's friends list but an invalid local user (-1) after polling for friend data, this subsystem will return false", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnlineFriends->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateLambda([this, SubsystemType, TestDone](int32 ReadFriendsListLocalPlayerNum, bool bReadFriendsListWasSuccessful, const FString& ReadFriendsListListName, const FString& ReadFriendsListErrorStr)
							{
								FString FriendAccountIdString = FOnlineTestCommon::GetSubsystemFriendAccountUniqueId(SubsystemType);
								FUniqueNetIdPtr FriendAccountId = OnlineIdentity->CreateUniquePlayerId(FriendAccountIdString);

								bool bIsFriend = OnlineFriends->IsFriend(-1, *FriendAccountId, EFriendsLists::ToString(EFriendsLists::Default));

								TestEqual("Verify that bIsFriend is: False", bIsFriend, false);

								TestDone.Execute();
							}));
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					It("When calling IsFriend with a valid local user and friend ID who is on the user's friends list but an invalid list name after polling for friend data, this subsystem will return false", [this, SubsystemType]()
					{
						UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: Test implementation not yet complete. Need to verify that different friends lists produce different results"));
					});

					LatentAfterEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineFriends->ClearOnDeleteFriendCompleteDelegate_Handle(0, OnDeleteFriendCompleteDelegateHandle);

						OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoggedOutLocalUserNum, bool bLogoutWasSuccessful)
						{
							CommonUtils.RemoveFriendFromTestAccount(OnlineIdentity, OnlineFriends, SubsystemType, TestDone);
							OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
						}));

						OnlineIdentity->Logout(0);
					});
				});

				// TODO: Figure out how to add players to the test account's recent players list
				Describe("QueryRecentPlayers", [this, SubsystemType]()
				{
					BeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType]()
					{
					});

					It("When calling QueryRecentPlayers with a namespace and a valid FUniqueNetId who recently played with others, this subsystem will return that user's recent players", [this, SubsystemType]()
					{
						UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: Test implementation not yet complete. Need to figure out how to set-up an account to have recent players to query"));
					});

					It("When calling QueryRecentPlayers with a namespace but a valid FUniqueNetId who has not played with others, this subsystem will return that user's recent players", [this, SubsystemType]()
					{
						UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: Test implementation not yet complete. Need to figure out how to set-up an account to have recent players to query"));
					});

					It("When calling QueryRecentPlayers a valid FUniqueNetId who recently played with others but an invalid namespace, this subsystem will return that user's recent players", [this, SubsystemType]()
					{
						UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: Test implementation not yet complete. Need to figure out how to set-up an account to have recent players to query"));
					});

					It("When calling QueryRecentPlayers with a namespace but an invalid FUniqueNetId, this subsystem will return that user's recent players", [this, SubsystemType]()
					{
						UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: Test implementation not yet complete. Need to figure out how to set-up an account to have recent players to query"));
					});

					AfterEach(EAsyncExecution::ThreadPool, [this, SubsystemType]()
					{
					});
				});
				Describe("GetRecentPlayers", [this, SubsystemType]()
				{
					BeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType]()
					{
					});

					It("When calling GetRecentPlayers with a namespace and a valid FUniqueNetId who recently played with others after polling for recent players data, this subsystem will return that user's recent players", [this, SubsystemType]()
					{
						UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: Test implementation not yet complete. Need to figure out how to set-up an account to have recent players to query"));
					});

					It("When calling GetRecentPlayers with a namespace and a valid FUniqueNetId who recently played with others without polling for recent players data, this subsystem will not return that user's recent players", [this, SubsystemType]()
					{
						UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: Test implementation not yet complete. Need to figure out how to set-up an account to have recent players to query"));
					});

					It("When calling GetRecentPlayers with a namespace and a valid FUniqueNetId who has not played with others after polling for recent players data, this subsystem will return not that user's recent players", [this, SubsystemType]()
					{
						UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: Test implementation not yet complete. Need to figure out how to set-up an account to have recent players to query"));
					});

					It("When calling GetRecentPlayers with a valid FUniqueNetId who recently played with others but an invalid namespace after polling for recent players data, this subsystem will not return that user's recent players", [this, SubsystemType]()
					{
						UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: Test implementation not yet complete. Need to figure out how to set-up an account to have recent players to query"));
					});

					It("When calling GetRecentPlayers with a namespace but an invalid FUniqueNetId after polling for recent players data, this subsystem will not return that user's recent players", [this, SubsystemType]()
					{
						UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: Test implementation not yet complete. Need to figure out how to set-up an account to have recent players to query"));
					});

					AfterEach(EAsyncExecution::ThreadPool, [this, SubsystemType]()
					{
					});
				});

				Describe("BlockPlayer", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						CommonUtils.AddFriendToTestAccount(OnlineIdentity, OnlineFriends, SubsystemType, TestDone);
					});

					// TODO: BlockedPlayers list is a thing, but there is no BlockedPlayers list in EFriendsLists
					LatentIt("When calling BlockPlayer with a valid local user and player id, this subsystem blocks that player", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FString FriendAccountIdString = FOnlineTestCommon::GetSubsystemFriendAccountUniqueId(SubsystemType);
							FUniqueNetIdPtr FriendAccountId = OnlineIdentity->CreateUniquePlayerId(FriendAccountIdString);

							if (FriendAccountId.IsValid())
							{
								OnBlockedPlayerCompleteDelegateHandle = OnlineFriends->AddOnBlockedPlayerCompleteDelegate_Handle(0, FOnBlockedPlayerCompleteDelegate::CreateLambda([this, FriendAccountIdString, TestDone](int32 BlockedPlayerLocalUserNum, bool bBlockedPlayerWasSuccessful, const FUniqueNetId& BlockedPlayerUniqueID, const FString& BlockedPlayerListName, const FString& BlockedPlayerErrorStr)
								{
									TestEqual("Verify that BlockedPlayerLocalUserNum is: 0", BlockedPlayerLocalUserNum == 0, true);
									TestEqual("Verify that bBlockedPlayerWasSuccessful returns as: True", bBlockedPlayerWasSuccessful, true);
									TestEqual("Verify that BlockedPlayerUniqueID is the Id that was originally used", BlockedPlayerUniqueID.ToString() == FriendAccountIdString, true);
									TestEqual("Verify that BlockedPlayerListName is: BlockedPlayers", BlockedPlayerListName == TEXT("BlockedPlayers"), true);
									TestEqual("Verify that BlockedPlayerErrorStr is unpopulated", BlockedPlayerErrorStr.Len() == 0, true);

									FUniqueNetIdPtr TestAccountId = OnlineIdentity->GetUniquePlayerId(0);
									TArray<TSharedRef<FOnlineBlockedPlayer>> BlockedPlayersArray;

									if (TestAccountId.IsValid())
									{
										OnlineFriends->GetBlockedPlayers(*TestAccountId, BlockedPlayersArray);

										bool bFoundBlockedPlayer = false;

										for (TSharedRef<FOnlineBlockedPlayer> BlockedPlayer : BlockedPlayersArray)
										{
											if (BlockedPlayer->GetUserId()->ToString() == FriendAccountIdString)
											{
												bFoundBlockedPlayer = true;
												break;
											}
										}

										TestEqual("Verify that bFoundBlockedPlayer is: True", bFoundBlockedPlayer, true);

										TestDone.Execute();
									}
									else
									{
										UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
										TestDone.Execute();
									}
								}));

								OnlineFriends->BlockPlayer(0, *FriendAccountId);
							}
							else
							{
								UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: IsValid() check on FriendsAccountId failed after a call to OnlineIdentity->CreateUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling BlockPlayer with a valid local user but an invalid player id, this subsystem does not block that player", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						FString FriendAccountIdString = TEXT("0123456789");

						AddExpectedError(ONLINE_EXPECTEDERROR_ACCOUNT_DOESNOTEXIST, EAutomationExpectedErrorFlags::Contains, 0);

						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, FriendAccountIdString, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FUniqueNetIdPtr FriendAccountId = OnlineIdentity->CreateUniquePlayerId(FriendAccountIdString);

							if (FriendAccountId.IsValid())
							{
								OnBlockedPlayerCompleteDelegateHandle = OnlineFriends->AddOnBlockedPlayerCompleteDelegate_Handle(0, FOnBlockedPlayerCompleteDelegate::CreateLambda([this, FriendAccountIdString, TestDone](int32 BlockedPlayerLocalUserNum, bool bBlockedPlayerWasSuccessful, const FUniqueNetId& BlockedPlayerUniqueID, const FString& BlockedPlayerListName, const FString& BlockedPlayerErrorStr)
								{
									TestEqual("Verify that BlockedPlayerLocalUserNum is: 0", BlockedPlayerLocalUserNum == 0, true);
									TestEqual("Verify that bBlockedPlayerWasSuccessful returns as: False", bBlockedPlayerWasSuccessful, false);
									TestEqual("Verify that BlockedPlayerUniqueID is the Id that was originally used", BlockedPlayerUniqueID.ToString() == FriendAccountIdString, true);
									TestEqual("Verify that BlockedPlayerListName is BlockedPlayers", BlockedPlayerListName == TEXT("BlockedPlayers"), true);
									TestEqual("Verify that BlockedPlayerErrorStr returns the expected error code: ONLINE_EXPECTEDERROR_ACCOUNT_DOESNOTEXIST", BlockedPlayerErrorStr.Contains(ONLINE_EXPECTEDERROR_ACCOUNT_DOESNOTEXIST), true);

									TestDone.Execute();
								}));

								OnlineFriends->BlockPlayer(0, *FriendAccountId);
							}
							else
							{
								UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: IsValid() check on FriendAccountId failed after a call to OnlineIdentity->CreateUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling BlockPlayer with a valid player id but an invalid local user (-1), this subsystem does not block that player", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_INVALID_LOCALUSER, EAutomationExpectedErrorFlags::Contains, 0);

						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FString FriendAccountIdString = FOnlineTestCommon::GetSubsystemFriendAccountUniqueId(SubsystemType);
							FUniqueNetIdPtr FriendAccountId = OnlineIdentity->CreateUniquePlayerId(FriendAccountIdString);

							if (FriendAccountId.IsValid())
							{
								bool bBlockPlayerWasStarted = OnlineFriends->BlockPlayer(-1, *FriendAccountId);

								TestEqual("Verify that bBlockPlayerWasStarted returns as: False", bBlockPlayerWasStarted, false);

								TestDone.Execute();
							}
							else
							{
								UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: IsValid() check on FriendAccountId failed after a call to OnlineIdentity->CreateUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentAfterEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineFriends->ClearOnUnblockedPlayerCompleteDelegate_Handle(0, OnUnblockedPlayerCompleteDelegateHandle);

						OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoggedOutLocalUserNum, bool bLogoutWasSuccessful)
						{
							CommonUtils.UnblockFriendOnTestAccount(OnlineIdentity, OnlineFriends, SubsystemType, TestDone);
							OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
						}));

						OnlineIdentity->Logout(0);
					});
				});

				Describe("UnblockPlayer", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						CommonUtils.BlockFriendOnTestAccount(OnlineIdentity, OnlineFriends, SubsystemType, TestDone);
					});

					LatentIt("When calling UnblockPlayer with a valid local user and player id, this subsystem unblocks that player", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FString FriendAccountIdString = FOnlineTestCommon::GetSubsystemFriendAccountUniqueId(SubsystemType);
							FUniqueNetIdPtr FriendAccountId = OnlineIdentity->CreateUniquePlayerId(FriendAccountIdString);

							if (FriendAccountId.IsValid())
							{
								OnUnblockedPlayerCompleteDelegateHandle = OnlineFriends->AddOnUnblockedPlayerCompleteDelegate_Handle(0, FOnUnblockedPlayerCompleteDelegate::CreateLambda([this, FriendAccountIdString, TestDone](int32 UnblockedPlayerLocalUserNum, bool bUnblockedPlayerWasSuccessful, const FUniqueNetId& UnblockedPlayerUniqueID, const FString& UnblockedPlayerListName, const FString& UnblockedPlayerErrorStr)
								{
									TestEqual("Verify that UnblockedPlayerLocalUserNum is: 0", UnblockedPlayerLocalUserNum == 0, true);
									TestEqual("Verify that bUnblockedPlayerWasSuccessful returns as: True", bUnblockedPlayerWasSuccessful, true);
									TestEqual("Verify that UnblockedPlayerUniqueID is the Id that was originally used", UnblockedPlayerUniqueID.ToString() == FriendAccountIdString, true);
									TestEqual("Verify that UnblockedPlayerListName is: BlockedPlayers", UnblockedPlayerListName == TEXT("BlockedPlayers"), true);
									TestEqual("Verify that UnblockedPlayerErrorStr is unpopulated", UnblockedPlayerErrorStr.Len() == 0, true);

									FUniqueNetIdPtr TestAccountId = OnlineIdentity->GetUniquePlayerId(0);
									TArray<TSharedRef<FOnlineBlockedPlayer>> BlockedPlayersArray;

									if (TestAccountId.IsValid())
									{
										OnlineFriends->GetBlockedPlayers(*TestAccountId, BlockedPlayersArray);

										bool bFoundBlockedPlayer = false;

										for (TSharedRef<FOnlineBlockedPlayer> BlockedPlayer : BlockedPlayersArray)
										{
											if (BlockedPlayer->GetUserId()->ToString() == FriendAccountIdString)
											{
												bFoundBlockedPlayer = true;
												break;
											}
										}

										TestEqual("Verify that bFoundBlockedPlayer is: False", bFoundBlockedPlayer, false);

										TestDone.Execute();
									}
									else
									{
										UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
										TestDone.Execute();
									}
								}));

								OnlineFriends->UnblockPlayer(0, *FriendAccountId);
							}
							else
							{
								UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: IsValid() check on FriendAccountId failed after a call to OnlineIdentity->CreateUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});
					// TODO: Bug, unblocking an invalid id does not produce an error
					LatentIt("When calling UnblockPlayer with a valid local user but an invalid player id, this subsystem does not unblock that player", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						FString FriendAccountIdString = TEXT("0123456789");

						AddExpectedError(ONLINE_EXPECTEDERROR_ACCOUNT_DOESNOTEXIST, EAutomationExpectedErrorFlags::Contains, 0);

						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, FriendAccountIdString, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FUniqueNetIdPtr FriendAccountId = OnlineIdentity->CreateUniquePlayerId(FriendAccountIdString);

							if (FriendAccountId.IsValid())
							{
								OnUnblockedPlayerCompleteDelegateHandle = OnlineFriends->AddOnUnblockedPlayerCompleteDelegate_Handle(0, FOnUnblockedPlayerCompleteDelegate::CreateLambda([this, FriendAccountIdString, TestDone](int32 UnblockedPlayerLocalUserNum, bool bUnblockedPlayerWasSuccessful, const FUniqueNetId& UnblockedPlayerUniqueID, const FString& UnblockedPlayerListName, const FString& UnblockedPlayerErrorStr)
								{
									TestEqual("Verify that UnblockedPlayerLocalUserNum is: 0", UnblockedPlayerLocalUserNum == 0, true);
									TestEqual("Verify that bUnblockedPlayerWasSuccessful returns as: False", bUnblockedPlayerWasSuccessful, false);
									TestEqual("Verify that UnblockedPlayerUniqueID is the Id that was originally used", UnblockedPlayerUniqueID.ToString() == FriendAccountIdString, true);
									TestEqual("Verify that UnblockedPlayerListName is: BlockedPlayers", UnblockedPlayerListName == TEXT("BlockedPlayers"), true);
									TestEqual("Verify that UnblockedPlayerErrorStr returns the expected error code: ONLINE_EXPECTEDERROR_ACCOUNT_DOESNOTEXIST", UnblockedPlayerErrorStr.Contains(ONLINE_EXPECTEDERROR_ACCOUNT_DOESNOTEXIST), true);

									TestDone.Execute();
								}));

								OnlineFriends->UnblockPlayer(0, *FriendAccountId);
							}
							else
							{
								UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: IsValid() check on FriendAccountId failed after a call to OnlineIdentity->CreateUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling UnblockPlayer with a valid player id but an invalid local user (-1), this subsystem does not unblock that player", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_INVALID_LOCALUSER, EAutomationExpectedErrorFlags::Contains, 0);

						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FString FriendAccountIdString = FOnlineTestCommon::GetSubsystemFriendAccountUniqueId(SubsystemType);
							FUniqueNetIdPtr FriendAccountId = OnlineIdentity->CreateUniquePlayerId(FriendAccountIdString);

							if (FriendAccountId.IsValid())
							{
								bool bUnblockPlayerWasStarted = OnlineFriends->UnblockPlayer(-1, *FriendAccountId);

								TestEqual("Verify that bUnblockPlayerWasStarted returns as: False", bUnblockPlayerWasStarted, false);

								TestDone.Execute();
							}
							else
							{
								UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: IsValid() check on FriendAccountId failed after a call to OnlineIdentity->CreateUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentAfterEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineFriends->ClearOnUnblockedPlayerCompleteDelegate_Handle(0, OnUnblockedPlayerCompleteDelegateHandle);

						OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoggedOutLocalUserNum, bool bLogoutWasSuccessful)
						{
							CommonUtils.UnblockFriendOnTestAccount(OnlineIdentity, OnlineFriends, SubsystemType, TestDone);
							OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
						}));

						OnlineIdentity->Logout(0);
					});
				});


				Describe("QueryBlockedPlayers", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						CommonUtils.BlockFriendOnTestAccount(OnlineIdentity, OnlineFriends, SubsystemType, TestDone);
					});

					LatentIt("When calling QueryBlockedPlayers with a valid FUniqueNetId, this subsystem will return that user's blocked players", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FString TestAccountIdString = FOnlineTestCommon::GetSubsystemTestAccountUniqueId(SubsystemType);
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->CreateUniquePlayerId(TestAccountIdString);

							OnQueryBlockedPlayersCompleteDelegateHandle = OnlineFriends->AddOnQueryBlockedPlayersCompleteDelegate_Handle(FOnQueryBlockedPlayersCompleteDelegate::CreateLambda([this, TestAccountIdString, TestDone](const FUniqueNetId& QueryBlockedPlayersUserId, bool bQueryBlockedPlayersWasSuccessful, const FString& QueryBlockedPlayersError)
							{
								TestEqual("Verify that QueryBlockedPlayersUserId is the Id that was originally used", QueryBlockedPlayersUserId.ToString() == TestAccountIdString, true);
								TestEqual("Verify that bQueryBlockedPlayersWasSuccessful returns as: True", bQueryBlockedPlayersWasSuccessful, true);
								TestEqual("Verify that QueryBlockedPlayersError is unpopulated", QueryBlockedPlayersError.Len() == 0, true);

								TArray<TSharedRef<FOnlineBlockedPlayer>> BlockedPlayersList;
								OnlineFriends->GetBlockedPlayers(QueryBlockedPlayersUserId, BlockedPlayersList);

								TestEqual("Verify that BlockedPlayersList is populated", BlockedPlayersList.Num() > 0, true);

								TestDone.Execute();
							}));

							if (TestAccountId.IsValid())
							{
								OnlineFriends->QueryBlockedPlayers(*TestAccountId);
							}
							else
							{
								UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->CreateUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					// TODO: Bug? Missing permissions error instead of not exist.
					LatentIt("When calling QueryBlockedPlayers with an invalid FUniqueNetId, this subsystem will return that user's blocked players", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						FString TestAccountIdString = TEXT(" ");

						AddExpectedError(ONLINE_EXPECTEDERROR_ACCOUNT_DOESNOTEXIST, EAutomationExpectedErrorFlags::Contains, 0);

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestAccountIdString, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->CreateUniquePlayerId(TestAccountIdString);

							OnQueryBlockedPlayersCompleteDelegateHandle = OnlineFriends->AddOnQueryBlockedPlayersCompleteDelegate_Handle(FOnQueryBlockedPlayersCompleteDelegate::CreateLambda([this, TestAccountIdString, TestDone](const FUniqueNetId& QueryBlockedPlayersUserId, bool bQueryBlockedPlayersWasSuccessful, const FString& QueryBlockedPlayersError)
							{
								TestEqual("Verify that QueryBlockedPlayersUserId is the Id that was originally used", QueryBlockedPlayersUserId.ToString() == TestAccountIdString, true);
								TestEqual("Verify that bQueryBlockedPlayersWasSuccessful returns as: False", bQueryBlockedPlayersWasSuccessful, false);
								TestEqual("Verify that QueryBlockedPlayersError returns the expected error code: ONLINE_EXPECTEDERROR_ACCOUNT_DOESNOTEXIST", QueryBlockedPlayersError.Contains(ONLINE_EXPECTEDERROR_ACCOUNT_DOESNOTEXIST), true);

								TArray<TSharedRef<FOnlineBlockedPlayer>> BlockedPlayersList;
								OnlineFriends->GetBlockedPlayers(QueryBlockedPlayersUserId, BlockedPlayersList);

								TestEqual("Verify that BlockedPlayersList is not populated", BlockedPlayersList.Num() == 0, true);

								TestDone.Execute();
							}));

							if (TestAccountId.IsValid())
							{
								OnlineFriends->QueryBlockedPlayers(*TestAccountId);
							}
							else
							{
								UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->CreateUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentAfterEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineFriends->ClearOnUnblockedPlayerCompleteDelegate_Handle(0, OnUnblockedPlayerCompleteDelegateHandle);
						OnlineFriends->ClearOnQueryBlockedPlayersCompleteDelegate_Handle(OnQueryBlockedPlayersCompleteDelegateHandle);

						OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoggedOutLocalUserNum, bool bLogoutWasSuccessful)
						{
							CommonUtils.UnblockFriendOnTestAccount(OnlineIdentity, OnlineFriends, SubsystemType, TestDone);
							OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
						}));

						OnlineIdentity->Logout(0);
					});
				});

				Describe("GetBlockedPlayers", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						CommonUtils.BlockFriendOnTestAccount(OnlineIdentity, OnlineFriends, SubsystemType, TestDone);
					});

					LatentIt("When calling GetBlockedPlayers with a valid FUniqueNetId, this subsystem will return that user's blocked players", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FString TestAccountIdString = FOnlineTestCommon::GetSubsystemTestAccountUniqueId(SubsystemType);
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->CreateUniquePlayerId(TestAccountIdString);

							OnQueryBlockedPlayersCompleteDelegateHandle = OnlineFriends->AddOnQueryBlockedPlayersCompleteDelegate_Handle(FOnQueryBlockedPlayersCompleteDelegate::CreateLambda([this, TestAccountIdString, TestDone](const FUniqueNetId& QueryBlockedPlayersUserId, bool bQueryBlockedPlayersWasSuccessful, const FString& QueryBlockedPlayersError)
							{
								TestEqual("Verify that bQueryBlockedPlayersWasSuccessful returns as: True", bQueryBlockedPlayersWasSuccessful, true);

								TArray<TSharedRef<FOnlineBlockedPlayer>> BlockedPlayersList;
								bool bRetrievedBlockedPlayers = OnlineFriends->GetBlockedPlayers(QueryBlockedPlayersUserId, BlockedPlayersList);

								TestEqual("Verify that bRetrievedBlockedPlayers returns as: True", bRetrievedBlockedPlayers, true);
								TestEqual("Verify that BlockedPlayersList is populated", BlockedPlayersList.Num() > 0, true);

								TestDone.Execute();
							}));

							if (TestAccountId.IsValid())
							{
								OnlineFriends->QueryBlockedPlayers(*TestAccountId);
							}
							else
							{
								UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->CreateUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling GetBlockedPlayers with an invalid FUniqueNetId, this subsystem will not return that user's blocked players", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FString TestAccountIdString = FOnlineTestCommon::GetSubsystemTestAccountUniqueId(SubsystemType);
							FUniqueNetIdPtr TestAccountId = OnlineIdentity->CreateUniquePlayerId(TestAccountIdString);

							OnQueryBlockedPlayersCompleteDelegateHandle = OnlineFriends->AddOnQueryBlockedPlayersCompleteDelegate_Handle(FOnQueryBlockedPlayersCompleteDelegate::CreateLambda([this, TestAccountIdString, TestDone](const FUniqueNetId& QueryBlockedPlayersUserId, bool bQueryBlockedPlayersWasSuccessful, const FString& QueryBlockedPlayersError)
							{
								TestEqual("Verify that bQueryBlockedPlayersWasSuccessful returns as: True", bQueryBlockedPlayersWasSuccessful, true);

								FString InvalidUserIdString = TEXT(" ");
								FUniqueNetIdPtr InvalidUserId = OnlineIdentity->CreateUniquePlayerId(InvalidUserIdString);

								if (InvalidUserId.IsValid())
								{
									TArray<TSharedRef<FOnlineBlockedPlayer>> BlockedPlayersList;
									bool bRetrievedBlockedPlayers = OnlineFriends->GetBlockedPlayers(*InvalidUserId, BlockedPlayersList);

									TestEqual("Verify that bRetrievedBlockedPlayers returns as: False", bRetrievedBlockedPlayers, false);
									TestEqual("Verify that BlockedPlayersList is not populated", BlockedPlayersList.Num() == 0, true);

									TestDone.Execute();
								}
								else
								{
									UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: IsValid() check on InvalidUserId failed after a call to OnlineIdentity->CreateUniquePlayerId()"));
									TestDone.Execute();
								}
							}));

							if (TestAccountId.IsValid())
							{
								OnlineFriends->QueryBlockedPlayers(*TestAccountId);
							}
							else
							{
								UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->CreateUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentAfterEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineFriends->ClearOnUnblockedPlayerCompleteDelegate_Handle(0, OnUnblockedPlayerCompleteDelegateHandle);
						OnlineFriends->ClearOnQueryBlockedPlayersCompleteDelegate_Handle(OnQueryBlockedPlayersCompleteDelegateHandle);

						OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoggedOutLocalUserNum, bool bLogoutWasSuccessful)
						{
							CommonUtils.UnblockFriendOnTestAccount(OnlineIdentity, OnlineFriends, SubsystemType, TestDone);
							OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
						}));

						OnlineIdentity->Logout(0);
					});
				});

				Describe("DumpBlockedPlayers", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						CommonUtils.BlockFriendOnTestAccount(OnlineIdentity, OnlineFriends, SubsystemType, TestDone);
					});

					// TODO: Takes no arguments, returns nothing, and just prints stuff to logs. How to test?
					LatentIt("When calling DumpBlockedPlayers, this subsystem will dump the state information about blocked players", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnlineFriends->DumpBlockedPlayers();

							UE_LOG_ONLINE_FRIEND(Error, TEXT("OSS Automation: Test implementation not yet complete. Need to figure out what to check against to test"));
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentAfterEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnlineFriends->ClearOnUnblockedPlayerCompleteDelegate_Handle(0, OnUnblockedPlayerCompleteDelegateHandle);
						OnlineFriends->ClearOnQueryBlockedPlayersCompleteDelegate_Handle(OnQueryBlockedPlayersCompleteDelegateHandle);

						OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoggedOutLocalUserNum, bool bLogoutWasSuccessful)
						{
							CommonUtils.UnblockFriendOnTestAccount(OnlineIdentity, OnlineFriends, SubsystemType, TestDone);
							OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
						}));

						OnlineIdentity->Logout(0);
					});
				});
			});
		});		
	}

	AfterEach(EAsyncExecution::ThreadPool, [this]()
	{
		if (OnlineIdentity.IsValid())
		{
			if (OnlineIdentity->GetLoginStatus(0) == ELoginStatus::LoggedIn)
			{
				OnlineIdentity->Logout(0);
			}

			OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
			OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
			OnlineIdentity = nullptr;
		}

		if (OnlineFriends.IsValid())
		{
			OnlineFriends->ClearOnInviteAcceptedDelegate_Handle(OnInviteAcceptedDelegateHandle);
			OnlineFriends->ClearOnQueryBlockedPlayersCompleteDelegate_Handle(OnQueryBlockedPlayersCompleteDelegateHandle);
			OnlineFriends->ClearOnBlockedPlayerCompleteDelegate_Handle(0, OnBlockedPlayerCompleteDelegateHandle);
			OnlineFriends->ClearOnUnblockedPlayerCompleteDelegate_Handle(0, OnUnblockedPlayerCompleteDelegateHandle);
			OnlineFriends->ClearOnDeleteFriendCompleteDelegate_Handle(0, OnDeleteFriendCompleteDelegateHandle);
			OnlineFriends->ClearOnRejectInviteCompleteDelegate_Handle(0, OnRejectInviteCompleteDelegateHandle);

			OnlineFriends = nullptr;
		}

		FCommandLine::Set(FCommandLine::GetOriginal());
	});

}
