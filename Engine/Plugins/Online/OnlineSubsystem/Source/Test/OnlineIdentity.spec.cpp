// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online.h"
#include "Utils/OnlineErrors.data.h"
#include "Misc/CommandLine.h"
#include "Utils/OnlineTestCommon.h"

/*
THIRD_PARTY_INCLUDES_START
#include "gtest/gtest.h"
#include "gmock/gmock.h"
THIRD_PARTY_INCLUDES_END
*/

BEGIN_DEFINE_SPEC(FOnlineIdentitySpec, "OnlineIdentityInterface", EAutomationTestFlags::EngineFilter | EAutomationTestFlags_ApplicationContextMask)

FOnlineTestCommon CommonUtils;

IOnlineIdentityPtr OnlineIdentity;
FOnlineAccountCredentials AccountCredentials;

// Delegate Handles
FDelegateHandle OnLogoutCompleteDelegateHandle;
FDelegateHandle	OnLoginCompleteDelegateHandle;

END_DEFINE_SPEC(FOnlineIdentitySpec)

void FOnlineIdentitySpec::Define()
{
	TArray<FName> Subsystems = FOnlineTestCommon::GetEnabledTestSubsystems();	
	
	for (int32 Index = 0; Index < Subsystems.Num(); Index++)
	{
		FName SubsystemType = Subsystems[Index];
		
		Describe(SubsystemType.ToString(), [this, SubsystemType]()
		{
			BeforeEach([this, SubsystemType]()
			{
				CommonUtils = FOnlineTestCommon();
				AccountCredentials = FOnlineTestCommon::GetSubsystemTestAccountCredentials(SubsystemType);
				OnlineIdentity = Online::GetIdentityInterface(SubsystemType);

				// If OnlineIdentity is not valid, the following test, including all other nested BeforeEaches, will not run
				if (!OnlineIdentity.IsValid())
				{
					UE_LOG_ONLINE_IDENTITY(Error, TEXT("Failed to get online identity interface for %s"), *SubsystemType.ToString());
				}
			});

			Describe("Online Identity Interface", [this, SubsystemType]()
			{
				Describe("Login", [this, SubsystemType]()
				{
					LatentIt("When calling Login with valid credentials for this subsystem, the user will be logged in successfully", [this, SubsystemType](const FDoneDelegate& TestDone)
					{					
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: True", bLoginWasSuccessful, true);
							TestEqual("Verify that LoginStatus returns as: LoggedIn", OnlineIdentity->GetLoginStatus(LoginLocalUserNum), ELoginStatus::LoggedIn);
							TestDone.Execute();
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});
					/**DISABLED**/
					xLatentIt("When calling Login with an invalid local user (-1), the user will receive an invalid local user error and not be logged in", FTimespan::FromSeconds(10), [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_INVALID_LOCALUSER, EAutomationExpectedErrorFlags::Contains, 0);
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: False", bLoginWasSuccessful, false);
							TestEqual("Verify that the LoginStatus returns as: NotLoggedIn", OnlineIdentity->GetLoginStatus(LoginLocalUserNum), ELoginStatus::NotLoggedIn);
							TestEqual("Verify that LoginError returns the expected error code: ONLINE_EXPECTEDERROR_INVALID_LOCALUSER", LoginError.Contains(ONLINE_EXPECTEDERROR_INVALID_LOCALUSER), true);
							TestDone.Execute();
						}));

						OnlineIdentity->Login(-1, AccountCredentials);
					});
					
					LatentIt("When calling Login with a nonexistent username for this subsystem, the user will receive an invalid credentials error and not be logged in", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_INVALID_ACCOUNTCREDENTIALS, EAutomationExpectedErrorFlags::Contains, 0);
						AccountCredentials.Id = "AWrongUserName";

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: False", bLoginWasSuccessful, false);
							TestEqual("Verify that LoginStatus returns as: NotLoggedIn", OnlineIdentity->GetLoginStatus(LoginLocalUserNum), ELoginStatus::NotLoggedIn);
							TestEqual("Verify that LoginError returns the expected error code: ONLINE_EXPECTEDERROR_INVALID_ACCOUNTCREDENTIALS", LoginError.Contains(ONLINE_EXPECTEDERROR_INVALID_ACCOUNTCREDENTIALS), true);
							TestDone.Execute();
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling Login with an invalid password for this subsystem, the user will receive an invalid credentials error and not be logged in", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_INVALID_ACCOUNTCREDENTIALS, EAutomationExpectedErrorFlags::Contains, 0);
						AccountCredentials.Token = "ABadPassword";

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: False", bLoginWasSuccessful, false);
							TestEqual("Verify that LoginStatus returns as: NotLoggedIn", OnlineIdentity->GetLoginStatus(LoginLocalUserNum), ELoginStatus::NotLoggedIn);
							TestEqual("Verify that LoginError returns the expected error code of: ONLINE_EXPECTEDERROR_INVALID_ACCOUNTCREDENTIALS", LoginError.Contains(ONLINE_EXPECTEDERROR_INVALID_ACCOUNTCREDENTIALS), true);
							TestDone.Execute();
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling Login with an incorrect auth type for this subsystem, the user will receive an invalid auth type error and not be logged in", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_INVALID_AUTHTYPE, EAutomationExpectedErrorFlags::Contains, 0);
						AccountCredentials.Type = "AWrongAuthType";

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: False", bLoginWasSuccessful, false);
							TestEqual("Verify that LoginStatus returns as: NotLoggedIn", OnlineIdentity->GetLoginStatus(LoginLocalUserNum), ELoginStatus::NotLoggedIn);
							TestEqual("Verify that LoginError returns the expected error code of: ONLINE_EXPECTEDERROR_INVALID_AUTHTYPE", LoginError.Contains(ONLINE_EXPECTEDERROR_INVALID_AUTHTYPE), true);
							TestDone.Execute();
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling Login with an empty username for this subsystem, the user will receive an invalid credentials error and not be logged in", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_INVALID_ACCOUNTCREDENTIALS, EAutomationExpectedErrorFlags::Contains, 0);
						AccountCredentials.Id = "";

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: False", bLoginWasSuccessful, false);
							TestEqual("Verify that LoginStatus returns as: NotLoggedIn", OnlineIdentity->GetLoginStatus(LoginLocalUserNum), ELoginStatus::NotLoggedIn);
							TestEqual("Verify that LoginError returns the expected error code of: ONLINE_EXPECTEDERROR_INVALID_ACCOUNTCREDENTIALS", LoginError.Contains(ONLINE_EXPECTEDERROR_INVALID_ACCOUNTCREDENTIALS), true);
							TestDone.Execute();
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					/**DISABLED**/
					xLatentIt("When calling Login an empty password for this subsystem, the user will receive an invalid credentials error and not be logged in", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_INVALID_ACCOUNTCREDENTIALS, EAutomationExpectedErrorFlags::Contains, 0);
						AccountCredentials.Token = "";

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: False", bLoginWasSuccessful, false);
							TestEqual("Verify that LoginStatus returns as: NotLoggedIn", OnlineIdentity->GetLoginStatus(LoginLocalUserNum), ELoginStatus::NotLoggedIn);
							TestEqual("Verify that LoginError returns the expected error code of: ONLINE_EXPECTEDERROR_INVALID_ACCOUNTCREDENTIALS", LoginError.Contains(ONLINE_EXPECTEDERROR_INVALID_ACCOUNTCREDENTIALS), true);
							TestDone.Execute();
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling Login with an empty auth type for this subsystem, the user will receive an invalid auth type error and not be logged in", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_INVALID_AUTHTYPE, EAutomationExpectedErrorFlags::Contains, 0);
						AccountCredentials.Type = "";

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: False", bLoginWasSuccessful, false);
							TestEqual("Verify that LoginStatus returns as: NotLoggedIn", OnlineIdentity->GetLoginStatus(LoginLocalUserNum), ELoginStatus::NotLoggedIn);
							TestEqual("Verify that LoginError returns the expected error code of: ONLINE_EXPECTEDERROR_INVALID_AUTHTYPE", LoginError.Contains(ONLINE_EXPECTEDERROR_INVALID_AUTHTYPE), true);
							TestDone.Execute();
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});
					
				});
				
				Describe("Logout", [this, SubsystemType]()
				{
					LatentBeforeEach( [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: True", bLoginWasSuccessful, true);
							TestDone.Execute();
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When Logout is called with a valid local user, login status returns as ELoginStatus::NotLoggedIn",  [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LogoutLocalUserNum, bool bLogoutWasSuccessful)
						{
							ELoginStatus::Type CurrentLoginStatus = OnlineIdentity->GetLoginStatus(LogoutLocalUserNum);

							TestEqual("Verify that bLogoutWasSuccessful returns as: True", bLogoutWasSuccessful, true);
							TestEqual("Verify that LoginStatus returns as: NotLoggedIn", CurrentLoginStatus, ELoginStatus::NotLoggedIn);

							TestDone.Execute();
						}));

						OnlineIdentity->Logout(0);
					});
					/**DISABLED**/
					xLatentIt("When Logout is called with an invalid local user (-1), they receive a no logged in user error and no logout is performed",  FTimespan::FromSeconds(10), [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_LOCALUSER_NOTLOGGEDIN, EAutomationExpectedErrorFlags::Contains, 0);

						OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LogoutLocalUserNum, bool bLogoutWasSuccessful)
						{
							ELoginStatus::Type CurrentLoginStatus = OnlineIdentity->GetLoginStatus(LogoutLocalUserNum);

							TestEqual("Verify that bLogoutWasSuccessful returns as: False", bLogoutWasSuccessful, false);
							TestEqual("Verify that LoginStatus returns as: NotLoggedIn", CurrentLoginStatus, ELoginStatus::NotLoggedIn);					
							TestDone.Execute();
						}));

						OnlineIdentity->Logout(-1);
					});
				});

				Describe("AutoLogin", [this, SubsystemType]()
				{
					LatentIt("When calling AutoLogin with valid credentials present on the command line, the user is logged in to this subsystem", [this](const FDoneDelegate& TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: True", bLoginWasSuccessful, true);
							TestEqual("Verify that LoginStatus returns as: LoggedIn", OnlineIdentity->GetLoginStatus(LoginLocalUserNum), ELoginStatus::LoggedIn);
							TestEqual("Verify that LoginError is empty", LoginError.Len() == 0, true);
							TestDone.Execute();
						}));

						FCommandLine::Append(*FString::Printf(TEXT(" -AUTH_LOGIN=%s"), *AccountCredentials.Id));
						FCommandLine::Append(*FString::Printf(TEXT(" -AUTH_PASSWORD=%s"), *AccountCredentials.Token));
						FCommandLine::Append(*FString::Printf(TEXT(" -AUTH_TYPE=%s"), *AccountCredentials.Type));

						OnlineIdentity->AutoLogin(0);
					});
					/**DISABLED**/
					xLatentIt("When calling AutoLogin with an invalid local user (-1), the user receives an invalid local user error and is not logged in", FTimespan::FromSeconds(10), [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_INVALID_LOCALUSER, EAutomationExpectedErrorFlags::Contains, 0);
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: False", bLoginWasSuccessful, false);
							TestEqual("Verify that LoginStatus returns as: NotLoggedIn", OnlineIdentity->GetLoginStatus(LoginLocalUserNum), ELoginStatus::NotLoggedIn);
							TestEqual("Verify that LoginError returns the expected error code: ONLINE_EXPECTEDERROR_INVALID_LOCALUSER", LoginError.Contains(ONLINE_EXPECTEDERROR_INVALID_LOCALUSER), true);
							TestDone.Execute();
						}));

						FCommandLine::Append(*FString::Printf(TEXT(" -AUTH_LOGIN=%s"), *AccountCredentials.Id));
						FCommandLine::Append(*FString::Printf(TEXT(" -AUTH_PASSWORD=%s"), *AccountCredentials.Token));
						FCommandLine::Append(*FString::Printf(TEXT(" -AUTH_TYPE=%s"), *AccountCredentials.Type));

						OnlineIdentity->AutoLogin(-1);
					});
					
					LatentIt("When calling AutoLogin with a nonexistent username on the command line, the user will receive an invalid credentials error and not be logged in to this subsystem", [this](const FDoneDelegate& TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_INVALID_ACCOUNTCREDENTIALS, EAutomationExpectedErrorFlags::Contains, 0);

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: False", bLoginWasSuccessful, false);
							TestEqual("Verify that LoginStatus returns as: NotLoggedIn", OnlineIdentity->GetLoginStatus(LoginLocalUserNum), ELoginStatus::NotLoggedIn);
							TestEqual("Verify that LoginError returns the expected error code: ONLINE_EXPECTEDERROR_INVALID_ACCOUNTCREDENTIALS", LoginError.Contains(ONLINE_EXPECTEDERROR_INVALID_ACCOUNTCREDENTIALS), true);
							TestDone.Execute();
						}));

						AccountCredentials.Id = "ThisIsABadUserName";

						FCommandLine::Append(*FString::Printf(TEXT(" -AUTH_LOGIN=%s"), *AccountCredentials.Id));
						FCommandLine::Append(*FString::Printf(TEXT(" -AUTH_PASSWORD=%s"), *AccountCredentials.Token));
						FCommandLine::Append(*FString::Printf(TEXT(" -AUTH_TYPE=%s"), *AccountCredentials.Type));

						OnlineIdentity->AutoLogin(0);
					});

					LatentIt("When calling AutoLogin with an invalid password on the command line, the user will receive an invalid credentials error and not be logged in to this subsystem", [this](const FDoneDelegate& TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_INVALID_ACCOUNTCREDENTIALS, EAutomationExpectedErrorFlags::Contains, 0);

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: False", bLoginWasSuccessful, false);
							TestEqual("Verify that LoginStatus returns as: NotLoggedIn", OnlineIdentity->GetLoginStatus(LoginLocalUserNum), ELoginStatus::NotLoggedIn);
							TestEqual("Verify that LoginError returns the expected error code: ONLINE_EXPECTEDERROR_INVALID_ACCOUNTCREDENTIALS", LoginError.Contains(ONLINE_EXPECTEDERROR_INVALID_ACCOUNTCREDENTIALS), true);
							TestDone.Execute();
						}));

						AccountCredentials.Token = "ThisIsABadPasswordUnlessItsNot";

						FCommandLine::Append(*FString::Printf(TEXT(" -AUTH_LOGIN=%s"), *AccountCredentials.Id));
						FCommandLine::Append(*FString::Printf(TEXT(" -AUTH_PASSWORD=%s"), *AccountCredentials.Token));
						FCommandLine::Append(*FString::Printf(TEXT(" -AUTH_TYPE=%s"), *AccountCredentials.Type));

						OnlineIdentity->AutoLogin(0);
					});

					LatentIt("When calling AutoLogin with an invalid auth type on the command line, the user will receive an invalid auth type error and not be logged in to this subsystem", [this](const FDoneDelegate& TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_INVALID_AUTHTYPE, EAutomationExpectedErrorFlags::Contains, 0);

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: False", bLoginWasSuccessful, false);
							TestEqual("Verify that LoginStatus returns as: NotLoggedIn", OnlineIdentity->GetLoginStatus(LoginLocalUserNum), ELoginStatus::NotLoggedIn);
							TestEqual("Verify that LoginError returns the expected error code: ONLINE_EXPECTEDERROR_INVALID_AUTHTYPE", LoginError.Contains(ONLINE_EXPECTEDERROR_INVALID_AUTHTYPE), true);
							TestDone.Execute();
						}));

						AccountCredentials.Type = "ThisIsABadType";

						FCommandLine::Append(*FString::Printf(TEXT(" -AUTH_LOGIN=%s"), *AccountCredentials.Id));
						FCommandLine::Append(*FString::Printf(TEXT(" -AUTH_PASSWORD=%s"), *AccountCredentials.Token));
						FCommandLine::Append(*FString::Printf(TEXT(" -AUTH_TYPE=%s"), *AccountCredentials.Type));

						OnlineIdentity->AutoLogin(0);
					});

					LatentIt("When calling AutoLogin with a blank username on the command line, the user will receive an invalid credentials error and not be logged in to this subsystem",  [this](const FDoneDelegate& TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_INVALID_ACCOUNTCREDENTIALS, EAutomationExpectedErrorFlags::Contains, 0);

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: False", bLoginWasSuccessful, false);
							TestEqual("Verify that LoginStatus returns as: NotLoggedIn", OnlineIdentity->GetLoginStatus(LoginLocalUserNum), ELoginStatus::NotLoggedIn);
							TestEqual("Verify that LoginError returns the expected error code: ONLINE_EXPECTEDERROR_INVALID_ACCOUNTCREDENTIALS", LoginError.Contains(ONLINE_EXPECTEDERROR_INVALID_ACCOUNTCREDENTIALS), true);
							TestDone.Execute();
						}));

						AccountCredentials.Id = "";

						FCommandLine::Append(*FString::Printf(TEXT(" -AUTH_LOGIN=%s"), *AccountCredentials.Id));
						FCommandLine::Append(*FString::Printf(TEXT(" -AUTH_PASSWORD=%s"), *AccountCredentials.Token));
						FCommandLine::Append(*FString::Printf(TEXT(" -AUTH_TYPE=%s"), *AccountCredentials.Type));

						OnlineIdentity->AutoLogin(0);
					});

					LatentIt("When calling AutoLogin with a blank password on the command line, the user will receive an invalid credentials error and not be logged in to this subsystem",  [this](const FDoneDelegate& TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_INVALID_ACCOUNTCREDENTIALS, EAutomationExpectedErrorFlags::Contains, 0);

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: False", bLoginWasSuccessful, false);
							TestEqual("Verify that LoginStatus returns as: NotLoggedIn", OnlineIdentity->GetLoginStatus(LoginLocalUserNum), ELoginStatus::NotLoggedIn);
							TestEqual("Verify that LoginError returns the expected error code: ONLINE_EXPECTEDERROR_INVALID_ACCOUNTCREDENTIALS", LoginError.Contains(ONLINE_EXPECTEDERROR_INVALID_ACCOUNTCREDENTIALS), true);
							TestDone.Execute();
						}));

						AccountCredentials.Token = "";

						FCommandLine::Append(*FString::Printf(TEXT(" -AUTH_LOGIN=%s"), *AccountCredentials.Id));
						FCommandLine::Append(*FString::Printf(TEXT(" -AUTH_PASSWORD=%s"), *AccountCredentials.Token));
						FCommandLine::Append(*FString::Printf(TEXT(" -AUTH_TYPE=%s"), *AccountCredentials.Type));

						OnlineIdentity->AutoLogin(0);
					});
					/**DISABLED**/
					xLatentIt("When calling AutoLogin with a blank auth type on the command line, the user will receive an invalid auth type error and not be logged in to this subsystem",  [this](const FDoneDelegate& TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_INVALID_AUTHTYPE, EAutomationExpectedErrorFlags::Contains, 0);

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: False", bLoginWasSuccessful, false);
							TestEqual("Verify that LoginStatus returns as: NotLoggedIn", OnlineIdentity->GetLoginStatus(LoginLocalUserNum), ELoginStatus::NotLoggedIn);
							TestEqual("Verify that LoginError returns the expected error code: ONLINE_EXPECTEDERROR_INVALID_ACCOUNTCREDENTIALS", LoginError.Contains(ONLINE_EXPECTEDERROR_INVALID_AUTHTYPE), true);
							TestDone.Execute();
						}));

						AccountCredentials.Type = "";

						FCommandLine::Append(*FString::Printf(TEXT(" -AUTH_LOGIN=%s"), *AccountCredentials.Id));
						FCommandLine::Append(*FString::Printf(TEXT(" -AUTH_PASSWORD=%s"), *AccountCredentials.Token));
						FCommandLine::Append(*FString::Printf(TEXT(" -AUTH_TYPE=%s"), *AccountCredentials.Type));

						OnlineIdentity->AutoLogin(0);
					});
					/**DISABLED**/
					xLatentIt("When calling AutoLogin with no AUTH_LOGIN on the command line, the user will receive an invalid credentials error and not be logged in to this subsystem",  [this](const FDoneDelegate& TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_INVALID_ACCOUNTCREDENTIALS, EAutomationExpectedErrorFlags::Contains, 0);

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: False", bLoginWasSuccessful, false);
							TestEqual("Verify that LoginStatus returns as: NotLoggedIn", OnlineIdentity->GetLoginStatus(LoginLocalUserNum), ELoginStatus::NotLoggedIn);
							TestEqual("Verify that LoginError returns the expected error code: ONLINE_EXPECTEDERROR_INVALID_ACCOUNTCREDENTIALS", LoginError.Contains(ONLINE_EXPECTEDERROR_INVALID_ACCOUNTCREDENTIALS), true);
							TestDone.Execute();
						}));

						FCommandLine::Append(*FString::Printf(TEXT(" -AUTH_PASSWORD=%s"), *AccountCredentials.Token));
						FCommandLine::Append(*FString::Printf(TEXT(" -AUTH_TYPE=%s"), *AccountCredentials.Type));

						OnlineIdentity->AutoLogin(0);
					});
					/**DISABLED**/
					xLatentIt("When calling AutoLogin with no AUTH_PASSWORD on the command line, the user will receive an invalid credentials error and not be logged in to this subsystem",  [this](const FDoneDelegate& TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_INVALID_ACCOUNTCREDENTIALS, EAutomationExpectedErrorFlags::Contains, 0);

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: False", bLoginWasSuccessful, false);
							TestEqual("Verify that LoginStatus returns as: NotLoggedIn", OnlineIdentity->GetLoginStatus(LoginLocalUserNum), ELoginStatus::NotLoggedIn);
							TestEqual("Verify that LoginError returns the expected error code", LoginError.Contains(ONLINE_EXPECTEDERROR_INVALID_ACCOUNTCREDENTIALS), true);
							TestDone.Execute();
						}));

						FCommandLine::Append(*FString::Printf(TEXT(" -AUTH_LOGIN=%s"), *AccountCredentials.Id));
						FCommandLine::Append(*FString::Printf(TEXT(" -AUTH_TYPE=%s"), *AccountCredentials.Type));

						OnlineIdentity->AutoLogin(0);

					});
					/**DISABLED**/
					xLatentIt("When calling AutoLogin with no AUTH_TYPE on the command line, the user will receive an invalid auth type error and not be logged in to this subsystem",  [this](const FDoneDelegate& TestDone)
					{
						AddExpectedError(ONLINE_EXPECTEDERROR_INVALID_AUTHTYPE, EAutomationExpectedErrorFlags::Contains, 0);

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: False", bLoginWasSuccessful, false);
							TestEqual("Verify that LoginStatus returns as: NotLoggedIn", OnlineIdentity->GetLoginStatus(LoginLocalUserNum), ELoginStatus::NotLoggedIn);
							TestEqual("Verify that LoginError returns the expected error code: ONLINE_EXPECTEDERROR_INVALID_ACCOUNTCREDENTIALS", LoginError.Contains(ONLINE_EXPECTEDERROR_INVALID_AUTHTYPE), true);
							TestDone.Execute();
						}));

						FCommandLine::Append(*FString::Printf(TEXT(" -AUTH_LOGIN=%s"), *AccountCredentials.Id));
						FCommandLine::Append(*FString::Printf(TEXT(" -AUTH_PASSWORD=%s"), *AccountCredentials.Token));

						OnlineIdentity->AutoLogin(0);
					}); 

				});

				Describe("GetUserAccount", [this, SubsystemType]()
				{
					LatentBeforeEach( [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: True", bLoginWasSuccessful, true);
							TestDone.Execute();
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					It("When calling GetUserAccount with a valid FUniqueNetId, this subsystem returns valid user information",  [this, SubsystemType]()
					{
						FUniqueNetIdPtr UserId = OnlineIdentity->GetUniquePlayerId(0);

						if (UserId.IsValid())
						{
							TSharedPtr<FUserOnlineAccount> UserAccount = OnlineIdentity->GetUserAccount(*UserId);

							if (UserAccount.IsValid())
							{
								FString AccessTokenString = UserAccount->GetAccessToken();
								FString UserIdString = UserAccount->GetUserId().Get().ToString();

								TestEqual("Verify that the user's AccessTokenString is populated", AccessTokenString.Len() > 0, true);
								TestEqual("Verify that the user's UserIdString is populated", UserIdString.Len() > 0, true);
							}
							else
							{
								UE_LOG_ONLINE_IDENTITY(Error, TEXT("OSS Automation: IsValid() check on UserAccount failed after a call to OnlineIdentity->GetUserAccount()"));
							}
						}
						else
						{
							UE_LOG_ONLINE_IDENTITY(Error, TEXT("OSS Automation: IsValid() check on UserId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
						}
						
					});

					It("When calling GetUserAccount with an invalid FUniqueNetId, this subsystem returns a null object",  [this, SubsystemType]()
					{
						FString InvalidUserIdString = TEXT(" ");
						FUniqueNetIdPtr InvalidUserId = OnlineIdentity->CreateUniquePlayerId(InvalidUserIdString);

						if (InvalidUserId.IsValid())
						{
							TSharedPtr<FUserOnlineAccount> UserAccount = OnlineIdentity->GetUserAccount(*InvalidUserId);
							TestEqual("Verify that the returned UserAccount object is not valid", UserAccount.IsValid(), false);														
						}
						else
						{
							UE_LOG_ONLINE_IDENTITY(Error, TEXT("OSS Automation: IsValid() check on UserId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
						}
						
					});

					LatentAfterEach([this](const FDoneDelegate& AfterEachDone)
					{
						OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, AfterEachDone](int32 LogoutLocalUserNum, bool bLogoutWasSuccessful)
						{
							AfterEachDone.Execute();
						}));

						OnlineIdentity->Logout(0);
					});
				});

				Describe("GetAllUserAccounts", [this, SubsystemType]()
				{
					LatentBeforeEach( [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: True", bLoginWasSuccessful, true);
							TestDone.Execute();
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					It("When calling GetAllUserAccounts on a valid interface, it returns the expected number accounts that were registered with it",  [this, SubsystemType]()
					{
						TArray<TSharedPtr<FUserOnlineAccount> > AllUserAccounts = OnlineIdentity->GetAllUserAccounts();
						TestEqual("Login with one account for this subsystem and verify that UserAccounts array count is equal to (1)", AllUserAccounts.Num() == 1, true);
					});

					LatentAfterEach([this](const FDoneDelegate& AfterEachDone)
					{
						OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, AfterEachDone](int32 LogoutLocalUserNum, bool bLogoutWasSuccessful)
						{
							AfterEachDone.Execute();
						}));

						OnlineIdentity->Logout(0);
					});
				});

				Describe("GetUniquePlayerId", [this, SubsystemType]()
				{
					LatentBeforeEach( [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: True", bLoginWasSuccessful, true);
							TestDone.Execute();
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					It("When calling GetUniquePlayerId with a valid local user, this subsystem returns the user's FUniqueNetId",  [this]()
					{
						FUniqueNetIdPtr UserId = OnlineIdentity->GetUniquePlayerId(0);
						
						if (UserId.IsValid())
						{
							TestEqual("Verify that UserId is populated after calling GetUniquePlayerId", UserId->ToString().Len() > 0, true);
						}
						else
						{
							UE_LOG_ONLINE_IDENTITY(Error, TEXT("OSS Automation: IsValid() check on UserId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
						}
					});

					It("When calling GetUniquePlayerId with an invalid local user (-1), this subsystem returns a null UserId",  [this]()
					{
						FUniqueNetIdPtr UserId = OnlineIdentity->GetUniquePlayerId(-1);
						TestEqual("Verify that UserId is invalid/null after calling GetUniquePlayerId", UserId.IsValid(), false);
					});

					LatentAfterEach([this](const FDoneDelegate& AfterEachDone)
					{
						OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, AfterEachDone](int32 LogoutLocalUserNum, bool bLogoutWasSuccessful)
						{
							AfterEachDone.Execute();
						}));

						OnlineIdentity->Logout(0);
					});
				});

				Describe("GetSponsorUniquePlayerId", [this, SubsystemType]()
				{
					LatentBeforeEach( [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: True", bLoginWasSuccessful, true);
							TestDone.Execute();
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});
					/**DISABLED**/
					xIt("When calling GetSponsorUniquePlayer with a valid local user with a valid assigned sponsor Id, this subsystem returns the user's sponsor's unique id",  [this, SubsystemType]()
					{
						//@Todo: Stub test, needs a better way to be testable
						FUniqueNetIdPtr SponsorId = OnlineIdentity->GetSponsorUniquePlayerId(0);

						if (SponsorId.IsValid())
						{
							UE_LOG_ONLINE_IDENTITY(Error, TEXT("OSS Automation: Test implementation not yet complete. Requires user with set-up sponsor id."));
							TestEqual("Verify that SponsorId is populated", SponsorId->ToString().Len() > 0, true);
						}
						else
						{
							UE_LOG_ONLINE_IDENTITY(Error, TEXT("OSS Automation: IsValid() check on SponsorId failed after a call to OnlineIdentity->GetSponsorUniquePlayerId(0)"));
						}
						
					});

					It("When calling GetSponsorUniquePlayerId with an invalid local user (-1), this subsystem returns no information",  [this]()
					{
						FUniqueNetIdPtr SponsorId = OnlineIdentity->GetSponsorUniquePlayerId(-1);
						TestEqual("Verify that SponsorId is invalid", SponsorId.IsValid(), false);
					});

					LatentAfterEach([this](const FDoneDelegate& AfterEachDone)
					{
						OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, AfterEachDone](int32 LogoutLocalUserNum, bool bLogoutWasSuccessful)
						{
							AfterEachDone.Execute();
						}));

						OnlineIdentity->Logout(0);
					});
				});

				Describe("CreateUniquePlayerId", [this, SubsystemType]()
				{
					It("When calling CreateUniquePlayerId with a valid series of binary data and size, this subsystem creates a unique player id",  [this]()
					{
						FString PlayerGUIDString;
						FGuid PlayerGUID;
						FPlatformMisc::CreateGuid(PlayerGUID);
						PlayerGUIDString = PlayerGUID.ToString();

						FUniqueNetIdPtr UniquePlayerId = OnlineIdentity->CreateUniquePlayerId((uint8*)PlayerGUIDString.GetCharArray().GetData(), PlayerGUIDString.Len());

						if (UniquePlayerId.IsValid())
						{
							TestEqual("Verify that UniquePlayerId is populated", UniquePlayerId->ToString().Len() > 0, true);
						}
						else
						{
							UE_LOG_ONLINE_IDENTITY(Error, TEXT("OSS Automation: IsValid() check on UniquePlayerId failed after a call to OnlineIdentity->CreateUniquePlayerId()"));
						}
					});

					It("When calling CreateUniquePlayerId with a valid series of binary data but no size, this subsystem does not create a unique player id",  [this]()
					{
						FString PlayerGUIDString;
						FGuid PlayerGUID;
						FPlatformMisc::CreateGuid(PlayerGUID);
						PlayerGUIDString = PlayerGUID.ToString();

						FUniqueNetIdPtr UniquePlayerId = OnlineIdentity->CreateUniquePlayerId((uint8*)PlayerGUIDString.GetCharArray().GetData(), 0);

						TestEqual("Verify that UniquePlayerId is not valid", UniquePlayerId.IsValid(), false);
					});

					It("When calling CreateUniquePlayerId with a valid size but no valid series of binary data, this subsystem does not create a unique player id",  [this]()
					{
						FString PlayerGUIDString;
						FGuid PlayerGUID;
						FPlatformMisc::CreateGuid(PlayerGUID);
						PlayerGUIDString = PlayerGUID.ToString();

						FUniqueNetIdPtr UniquePlayerId = OnlineIdentity->CreateUniquePlayerId(0, PlayerGUIDString.Len());

						TestEqual("Verify that UniquePlayerId is not valid", UniquePlayerId.IsValid(), false);
					});

					It("When calling CreateUniquePlayerId with no size or data, this subsystem does not create a unique player id",  [this]()
					{
						FString PlayerGUIDString;
						FGuid PlayerGUID;
						FPlatformMisc::CreateGuid(PlayerGUID);
						PlayerGUIDString = PlayerGUID.ToString();

						FUniqueNetIdPtr UniquePlayerId = OnlineIdentity->CreateUniquePlayerId(0, 0);

						TestEqual("Verify that UniquePlayerId is not valid", UniquePlayerId.IsValid(), false);
					});

					It("When calling CreateUniquePlayerId with a string, this subsystem creates a unique player id",  [this]()
					{
						FString PlayerGUIDString;
						FGuid PlayerGUID;
						FPlatformMisc::CreateGuid(PlayerGUID);
						PlayerGUIDString = PlayerGUID.ToString();

						FUniqueNetIdPtr UniquePlayerId = OnlineIdentity->CreateUniquePlayerId(PlayerGUIDString);

						if (UniquePlayerId.IsValid())
						{
							TestEqual("Verify that UniquePlayerId is populated", UniquePlayerId->ToString().Len() > 0, true);
						}
						else
						{
							UE_LOG_ONLINE_IDENTITY(Error, TEXT("OSS Automation: IsValid() check on UniquePlayerId failed after a call to OnlineIdentity->CreateUniquePlayerId()"));
						}
					});
				});
				
				Describe("GetLoginStatus", [this, SubsystemType]()
				{
					LatentIt("When calling GetLoginStatus with a valid local user, this subsystem correctly returns the user's login status",  [this](const FDoneDelegate& TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: True", bLoginWasSuccessful, true);

							ELoginStatus::Type UserLoginStatus = OnlineIdentity->GetLoginStatus(0);
							TestEqual("Verify that the returned UserLoginStatus is ELoginStatus::LoggedIn", UserLoginStatus, ELoginStatus::LoggedIn);

							TestDone.Execute();
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling GetLoginStatus with a valid FUniqueNetId, this subsystem returns that user's login status",  [this](const FDoneDelegate& TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: True", bLoginWasSuccessful, true);

							ELoginStatus::Type UserLoginStatus = OnlineIdentity->GetLoginStatus(LoginUserId);
							TestEqual("Verify that the returned UserLoginStatus is ELoginStatus::LoggedIn", UserLoginStatus, ELoginStatus::LoggedIn);

							TestDone.Execute();
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					It("When calling GetLoginStatus with an invalid local user, this subsystem returns login status as ELoginStatus::NotLoggedIn",  [this]()
					{
						ELoginStatus::Type UserLoginStatus = OnlineIdentity->GetLoginStatus(-1);
						TestEqual("Verify that the returned UserLoginStatus is ELoginStatus::NotLoggedIn", UserLoginStatus, ELoginStatus::NotLoggedIn);
					});

					It("When calling GetLoginStatus with an invalid FUniqueNetId, this subsystem returns login status as ELoginStatus::NotLoggedIn",  [this]()
					{
						FString InvalidUserIdString = TEXT(" ");
						FUniqueNetIdPtr InvalidUserId = OnlineIdentity->CreateUniquePlayerId(InvalidUserIdString);

						if (InvalidUserId.IsValid())
						{
							ELoginStatus::Type UserLoginStatus = OnlineIdentity->GetLoginStatus(*InvalidUserId);

							TestEqual("Verify that the returned UserLoginStatus is NotLoggedIn", UserLoginStatus, ELoginStatus::NotLoggedIn);
						}
						else
						{
							UE_LOG_ONLINE_IDENTITY(Error, TEXT("OSS Automation: IsValid() check on UserId failed after a call to OnlineIdentity->CreateUniquePlayerId()"));
						}
					});

					It("When calling GetLoginStatus with a valid local user that is not logged in, this subsystem returns login status as ELoginStatus::NotLoggedIn",  [this]()
					{
						ELoginStatus::Type UserLoginStatus = OnlineIdentity->GetLoginStatus(0);
						TestEqual("Verify that the returned UserLoginStatus is ELoginStatus::NotLoggedIn", UserLoginStatus, ELoginStatus::NotLoggedIn);
					});

					LatentIt("When calling GetLoginStatus with a valid FUniqueNetId that is not logged in, this subsystem returns login status as ELoginStatus::NotLoggedIn",  [this](const FDoneDelegate& TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: True", bLoginWasSuccessful, true);

							OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, TestDone](int32 LoggedOutLocalUserNum, bool bLogoutWasSuccessful)
							{
								TestEqual("Verify that bLogoutWasSuccessful returns as: True", bLogoutWasSuccessful, true);

								FUniqueNetIdPtr UserIdToCheck = OnlineIdentity->GetUniquePlayerId(0);

								if (UserIdToCheck.IsValid())
								{
									ELoginStatus::Type UserLoginStatus = OnlineIdentity->GetLoginStatus(*UserIdToCheck);
									TestEqual("Verify that the returned UserLoginStatus is NotLoggedIn", UserLoginStatus, ELoginStatus::NotLoggedIn);
								}
								else
								{
									UE_LOG_ONLINE_IDENTITY(Error, TEXT("OSS Automation: IsValid() check on UserIdToCheck failed after a call to OnlineIdentity->GetUniquePlayerId()"));
								}

								TestDone.Execute();
							}));

							OnlineIdentity->Logout(0);
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});
					
				});
				
				Describe("GetPlayerNickname", [this, SubsystemType]()
				{
					LatentIt("When calling GetPlayerNickname with a valid local user, this subsystem returns the user's nickname",  [this](const FDoneDelegate& TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: True", bLoginWasSuccessful, true);

							FString PlayerNickname = "";
							PlayerNickname = OnlineIdentity->GetPlayerNickname(0);

							TestEqual("Verify that PlayerNickname is populated", PlayerNickname.Len() > 0, true);
							TestDone.Execute();

						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling GetPlayerNickname with a valid FUniqueNetId, this subsystem returns that user's nickname",  [this](const FDoneDelegate& TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: True", bLoginWasSuccessful, true);

							FString PlayerNickname = "";
							PlayerNickname = OnlineIdentity->GetPlayerNickname(LoginUserId);
							
							TestEqual("Verify that PlayerNickname is populated", PlayerNickname.Len() > 0, true);

							TestDone.Execute();
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});
					/**DISABLED**/
					xIt("When calling GetPlayerNickname with a valid local user that is not logged in, this subsystem returns an error as the PlayerNickname",  [this]()
					{
						FString PlayerNickname = "";
						PlayerNickname = OnlineIdentity->GetPlayerNickname(0);

						TestEqual("Verify that PlayerNickname is the expected error code: ONLINE_EXPECTEDERROR_INVALID_USERID", PlayerNickname, ONLINE_EXPECTEDERROR_INVALID_USERID);
					});
					/**DISABLED**/
					xIt("When calling GetPlayerNickname with a invalid local user (-1), this subsystem returns an error as the PlayerNickname",  [this]()
					{
						FString PlayerNickname = OnlineIdentity->GetPlayerNickname(-1);
						TestEqual("Verify that PlayerNickname is the expected error code: ONLINE_EXPECTEDERROR_INVALID_USERID", PlayerNickname, ONLINE_EXPECTEDERROR_INVALID_USERID);
					});
					/**DISABLED**/
					xIt("When calling GetPlayerNickname with an invalid FUniqueNetId, this subsystem returns an error as the PlayerNickname",  [this]()
					{
						FString InvalidUserIdString = TEXT(" ");
						FUniqueNetIdPtr InvalidUserId = OnlineIdentity->CreateUniquePlayerId(InvalidUserIdString);

						if (InvalidUserId.IsValid())
						{
							FString PlayerNickname = "";
							PlayerNickname = OnlineIdentity->GetPlayerNickname(*InvalidUserId);
							TestEqual("Verify that PlayerNickname is the expected error code: ONLINE_EXPECTEDERROR_INVALID_USERID", PlayerNickname, ONLINE_EXPECTEDERROR_INVALID_USERID);
						}
						else
						{
							UE_LOG_ONLINE_IDENTITY(Error, TEXT("OSS Automation: IsValid() check on UserId failed after a call to OnlineIdentity->CreateUniquePlayerId()"));
						}
					});

					LatentAfterEach([this](const FDoneDelegate& AfterEachDone)
					{
						OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, AfterEachDone](int32 LogoutLocalUserNum, bool bLogoutWasSuccessful)
						{
							AfterEachDone.Execute();
						}));

						OnlineIdentity->Logout(0);
					});
				});

				Describe("GetAuthToken", [this, SubsystemType]()
				{
					LatentIt("When calling GetAuthToken with a valid local user, this subsystem returns the current auth token assigned to this user",  [this](const FDoneDelegate& TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: True", bLoginWasSuccessful, true);

							FString PlayerAuthToken = OnlineIdentity->GetAuthToken(0);
							TestEqual("Verify that PlayerAuthToken is populated", PlayerAuthToken.Len() > 0, true);

							TestDone.Execute();
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					It("When calling GetAuthToken with an invalid local user (-1), this subsystem returns an empty auth token",  [this]()
					{
						FString PlayerAuthToken = OnlineIdentity->GetAuthToken(-1);
						TestEqual("Verify that PlayerAuthToken is not populated", PlayerAuthToken.Len() == 0, true);
					});


					It("When calling GetAuthToken with a local user that is not logged in, this subsystem returns an empty string",  [this]()
					{
						FString PlayerAuthToken = OnlineIdentity->GetAuthToken(0);
						TestEqual("Verify that PlayerAuthToken is not populated", PlayerAuthToken.Len() == 0, true);
					});
					
				});

				Describe("RevokeAuthToken", [this, SubsystemType]()
				{
					LatentBeforeEach( [this](const FDoneDelegate& TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: True", bLoginWasSuccessful, true);
							TestDone.Execute();
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling RevokeAuthToken with a valid FUniqueNetId, this subsystem revokes that user's auth token",  [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						FUniqueNetIdPtr UserIdToCheck = OnlineIdentity->GetUniquePlayerId(0);

						if (UserIdToCheck.IsValid())
						{
							OnlineIdentity->RevokeAuthToken(*UserIdToCheck, FOnRevokeAuthTokenCompleteDelegate::CreateLambda([this, TestDone, SubsystemType](const FUniqueNetId& RevokeAuthTokenUserId, const FOnlineError& RevokeAuthTokenError)
							{
								TestEqual("Verify that RevokeAuthTokenError.bSucceeded returns as: True", RevokeAuthTokenError.WasSuccessful(), true);
								TestDone.Execute();
							}));
						}
						else
						{
							UE_LOG_ONLINE_IDENTITY(Error, TEXT("OSS Automation: IsValid() check on UserIdToCheck failed after a call to OnlineIdentity->GetUniquePlayerId()"));
							TestDone.Execute();
						}
						
					});

					LatentIt("When calling RevokeAuthToken with an invalid FUniqueNetId, this subsystem returns an error",  [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						FString InvalidUserIdString = TEXT(" ");
						FUniqueNetIdPtr InvalidUserIdToCheck = OnlineIdentity->CreateUniquePlayerId(InvalidUserIdString);

						if (InvalidUserIdToCheck.IsValid())
						{
							OnlineIdentity->RevokeAuthToken(*InvalidUserIdToCheck, FOnRevokeAuthTokenCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](const FUniqueNetId& RevokeAuthTokenUserId, const FOnlineError& RevokeAuthTokenError)
							{
								TestEqual("Verify that RevokeAuthTokenError.bSucceeded returns as: False", RevokeAuthTokenError.WasSuccessful(), false);
								TestDone.Execute();
							}));
						}
						else
						{
							UE_LOG_ONLINE_IDENTITY(Error, TEXT("OSS Automation: IsValid() check on UserIdToCheck failed after a call to OnlineIdentity->GetUniquePlayerId()"));
							TestDone.Execute();
						}
					});

					LatentAfterEach([this](const FDoneDelegate& AfterEachDone)
					{
						OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, AfterEachDone](int32 LogoutLocalUserNum, bool bLogoutWasSuccessful)
						{
							AfterEachDone.Execute();
						}));

						OnlineIdentity->Logout(0);
					});

				});

				xDescribe("GetUserPrivilege", [this, SubsystemType]()
				{
					LatentIt("When calling GetUserPrivilege with a valid FUniqueNetId, EUserPrivileges::Type and Delegate, this subsystem Delegate call back returns NoFailures as the PrivilegeResult",  [this](const FDoneDelegate& TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: True", bLoginWasSuccessful, true);

							OnlineIdentity->GetUserPrivilege(LoginUserId, EUserPrivileges::CanPlayOnline, IOnlineIdentity::FOnGetUserPrivilegeCompleteDelegate::CreateLambda([this, TestDone](const FUniqueNetId& GetUserPrivilegeUniqueId, EUserPrivileges::Type GetUserPrivilegePrivilege, uint32 GetUserPrivilegePrivilegeResult)
							{
								TestTrue("Verify that this Delegate was called", true);
								TestEqual("Verify that the GetUserPrivilegePrivilegeResult is: NoFailures", GetUserPrivilegePrivilegeResult == (uint32)IOnlineIdentity::EPrivilegeResults::NoFailures, true);

								UE_LOG_ONLINE_IDENTITY(Error, TEXT("OSS Automation: Test implementation not yet complete. Requires a valid backend configuration"));
								TestDone.Execute();
							}));						
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					// TODO: Figure out how to induce a patch required state
					LatentIt("When calling GetUserPrivilege with a valid FUniqueNetId who requires a patch before they can play, this subsystem Delegate call back returns a RequiredPatchAvailable as the PrivilegeResult",  [this](const FDoneDelegate& TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: True", bLoginWasSuccessful, true);

							OnlineIdentity->GetUserPrivilege(LoginUserId, EUserPrivileges::CanPlayOnline, IOnlineIdentity::FOnGetUserPrivilegeCompleteDelegate::CreateLambda([this, TestDone](const FUniqueNetId& GetUserPrivilegeUniqueId, EUserPrivileges::Type GetUserPrivilegePrivilege, uint32 GetUserPrivilegePrivilegeResult)
							{
								TestEqual("Verify that GetUserPrivilegePrivilege is: CanPlayOnline", GetUserPrivilegePrivilege == EUserPrivileges::CanPlayOnline, true);
								TestEqual("Verify that GetUserPrivilegePrivilegeResult is: RequiredPatchAvailable", GetUserPrivilegePrivilegeResult == (uint32)IOnlineIdentity::EPrivilegeResults::RequiredPatchAvailable, true);
								
								UE_LOG_ONLINE_IDENTITY(Error, TEXT("OSS Automation: Test implementation not yet complete. Requires a valid backend configuration"));
								UE_LOG_ONLINE_IDENTITY(Error, TEXT("OSS Automation: Test implementation not yet complete. Needs a way to induce a patch required state"));
								TestDone.Execute();
							}));
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					// TODO: WIP Figure out how to induce a RequiredSystemUpdate
					LatentIt("When calling GetUserPrivilege with a valid FUniqueNetId who requires a system update before they can play, this subsystem Delegate call back returns a RequiredSystemUpdate as the PrivilegeResult",  [this](const FDoneDelegate& TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: True", bLoginWasSuccessful, true);

							OnlineIdentity->GetUserPrivilege(LoginUserId, EUserPrivileges::CanPlayOnline, IOnlineIdentity::FOnGetUserPrivilegeCompleteDelegate::CreateLambda([this, TestDone](const FUniqueNetId& GetUserPrivilegeUniqueId, EUserPrivileges::Type GetUserPrivilegePrivilege, uint32 GetUserPrivilegePrivilegeResult)
							{
								TestEqual("Verify that GetUserPrivilegePrivilege is: CanPlayOnline", GetUserPrivilegePrivilege == EUserPrivileges::CanPlayOnline, true);
								TestEqual("Verify that GetUserPrivilegePrivilegeResult is: RequiredSystemUpdate", GetUserPrivilegePrivilegeResult == (uint32)IOnlineIdentity::EPrivilegeResults::RequiredSystemUpdate, true);
								
								UE_LOG_ONLINE_IDENTITY(Error, TEXT("OSS Automation: Test implementation not yet complete. Requires a valid backend configuration"));
								UE_LOG_ONLINE_IDENTITY(Error, TEXT("OSS Automation: Test implementation not yet complete. Needs a way to induce a required system update state"));
								TestDone.Execute();
							}));
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					// TODO: WIP Figure out how to induce a AgeRestrictionFailure
					LatentIt("When calling GetUserPrivilege with a valid FUniqueNetId who is age restricted from play, this subsystem Delegate call back returns a AgeRestrictionFailure as the PrivilegeResult",  [this](const FDoneDelegate& TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: True", bLoginWasSuccessful, true);

							OnlineIdentity->GetUserPrivilege(LoginUserId, EUserPrivileges::CanPlayOnline, IOnlineIdentity::FOnGetUserPrivilegeCompleteDelegate::CreateLambda([this, TestDone](const FUniqueNetId& GetUserPrivilegeUniqueId, EUserPrivileges::Type GetUserPrivilegePrivilege, uint32 GetUserPrivilegePrivilegeResult)
							{
								TestEqual("Verify that GetUserPrivilegePrivilege is: CanPlayOnline", GetUserPrivilegePrivilege == EUserPrivileges::CanPlayOnline, true);
								TestEqual("Verify that GetUserPrivilegePrivilegeResult is: AgeRestrictionFailure", GetUserPrivilegePrivilegeResult == (uint32)IOnlineIdentity::EPrivilegeResults::AgeRestrictionFailure, true);
								
								UE_LOG_ONLINE_IDENTITY(Error, TEXT("OSS Automation: Test implementation not yet complete. Requires a valid backend configuration"));
								UE_LOG_ONLINE_IDENTITY(Error, TEXT("OSS Automation: Test implementation not yet complete. Needs a way to induce an age restricted state"));
								TestDone.Execute();
							}));
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					// TODO: WIP Figure out how to induce a AccountTypeFailure
					LatentIt("When calling GetUserPrivilege with a valid FUniqueNetId who requires a special account type before they can play, this subsystem Delegate call back returns a AccountTypeFailure as the PrivilegeResult",  [this](const FDoneDelegate& TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: True", bLoginWasSuccessful, true);

							OnlineIdentity->GetUserPrivilege(LoginUserId, EUserPrivileges::CanPlayOnline, IOnlineIdentity::FOnGetUserPrivilegeCompleteDelegate::CreateLambda([this, TestDone](const FUniqueNetId& GetUserPrivilegeUniqueId, EUserPrivileges::Type GetUserPrivilegePrivilege, uint32 GetUserPrivilegePrivilegeResult)
							{
								TestEqual("Verify that GetUserPrivilegePrivilege is: CanPlayOnline", GetUserPrivilegePrivilege == EUserPrivileges::CanPlayOnline, true);
								TestEqual("Verify that GetUserPrivilegePrivilegeResult is: AccountTypeFailure", GetUserPrivilegePrivilegeResult == (uint32)IOnlineIdentity::EPrivilegeResults::AccountTypeFailure, true);
								
								UE_LOG_ONLINE_IDENTITY(Error, TEXT("OSS Automation: Test implementation not yet complete. Requires a valid backend configuration"));
								UE_LOG_ONLINE_IDENTITY(Error, TEXT("OSS Automation: Test implementation not yet complete. Needs a way to induce an account type failure state"));
								TestDone.Execute();
							}));
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling GetUserPrivilege with invalid FUniqueNetId, this subsystem Delegate call back returns a UserNotFound as the PrivilegeResult",  [this](const FDoneDelegate& TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: True", bLoginWasSuccessful, true);

							FString InvalidUserIdString = TEXT(" ");
							FUniqueNetIdPtr InvalidUserId = OnlineIdentity->CreateUniquePlayerId(InvalidUserIdString);

							if (InvalidUserId.IsValid())
							{
								OnlineIdentity->GetUserPrivilege(*InvalidUserId, EUserPrivileges::CanPlayOnline, IOnlineIdentity::FOnGetUserPrivilegeCompleteDelegate::CreateLambda([this, TestDone](const FUniqueNetId& GetUserPrivilegeUniqueId, EUserPrivileges::Type GetUserPrivilegePrivilege, uint32 GetUserPrivilegePrivilegeResult)
								{
									TestTrue("Verify that this Delegate was called.", true);
									TestEqual("Verify that GetUserPrivilegePrivilegeResult is: UserNotFound", GetUserPrivilegePrivilegeResult == (uint32)IOnlineIdentity::EPrivilegeResults::UserNotFound, true);
									TestDone.Execute();
								}));
							}
							else
							{
								UE_LOG_ONLINE_IDENTITY(Error, TEXT("OSS Automation: IsValid() check on InvalidUserId failed after a call to OnlineIdentity->CreateUniquePlayerId()"));
								TestDone.Execute();
							}
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling GetUserPrivilege with a valid FUniqueNetId who is not logged in, this subsystem Delegate call back returns a UserNotLoggedIn as the PrivilegeResult",  [this](const FDoneDelegate& TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: True", bLoginWasSuccessful, true);

							OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, TestDone](int32 LogoutLocalUserNum, bool bLogoutWasSuccessful)
							{
								ELoginStatus::Type CurrentLoginStatus = OnlineIdentity->GetLoginStatus(LogoutLocalUserNum);
								TestEqual("Verify that bLogoutWasSuccessful returns as: True", bLogoutWasSuccessful, true);
								TestEqual("Verify that LoginStatus is: NotLoggedIn", CurrentLoginStatus, ELoginStatus::NotLoggedIn);

								FUniqueNetIdPtr InnerUserId = OnlineIdentity->GetUniquePlayerId(0);
								if (InnerUserId.IsValid())
								{
									OnlineIdentity->GetUserPrivilege(*InnerUserId, EUserPrivileges::CanPlayOnline, IOnlineIdentity::FOnGetUserPrivilegeCompleteDelegate::CreateLambda([this, TestDone](const FUniqueNetId& GetUserPrivilegeUniqueId, EUserPrivileges::Type GetUserPrivilegePrivilege, uint32 GetUserPrivilegePrivilegeResult)
									{
										TestEqual("Verify that GetUserPrivilegePrivilege is: CanPlayOnline", GetUserPrivilegePrivilege == EUserPrivileges::CanPlayOnline, true);
										TestEqual("Verify that GetUserPrivilegePrivilegeResult is: UserNotLoggedIn", GetUserPrivilegePrivilegeResult == (uint32)IOnlineIdentity::EPrivilegeResults::UserNotLoggedIn, true);
										TestDone.Execute();
									}));
								}
								else
								{
									UE_LOG_ONLINE_IDENTITY(Error, TEXT("OSS Automation: IsValid() check on UserId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
									TestDone.Execute();
								}
							}));

							OnlineIdentity->Logout(0);
							
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					// TODO: Figure out how to induce a ChatRestriction
					LatentIt("When calling GetUserPrivilege with a valid FUniqueNetId who is restricted from chat, this subsystem Delegate call back returns a ChatRestriction as the PrivilegeResult",  [this](const FDoneDelegate& TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: True", bLoginWasSuccessful, true);

							OnlineIdentity->GetUserPrivilege(LoginUserId, EUserPrivileges::CanPlayOnline, IOnlineIdentity::FOnGetUserPrivilegeCompleteDelegate::CreateLambda([this, TestDone](const FUniqueNetId& GetUserPrivilegeUniqueId, EUserPrivileges::Type GetUserPrivilegePrivilege, uint32 GetUserPrivilegePrivilegeResult)
							{
								TestEqual("Verify that GetUserPrivilegePrivilege is: CanPlayOnline", GetUserPrivilegePrivilege == EUserPrivileges::CanPlayOnline, true);
								TestEqual("Verify that GetUserPrivilegePrivilegeResult is: ChatRestriction", GetUserPrivilegePrivilegeResult == (uint32)IOnlineIdentity::EPrivilegeResults::ChatRestriction, true);
								
								UE_LOG_ONLINE_IDENTITY(Error, TEXT("OSS Automation: Test implementation not yet complete. Requires a valid backend configuration"));
								UE_LOG_ONLINE_IDENTITY(Error, TEXT("OSS Automation: Test implementation not yet complete. Needs a way to induce a Chat Restricted state"));
								TestDone.Execute();
							}));
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					// TODO: Figure out how to induce a UGCRestriction
					LatentIt("When calling GetUserPrivilege with a valid FUniqueNetId who is restricted from User Generated Content, this subsystem Delegate call back returns a UGCRestriction as the PrivilegeResult",  [this](const FDoneDelegate& TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: True", bLoginWasSuccessful, true);

							OnlineIdentity->GetUserPrivilege(LoginUserId, EUserPrivileges::CanPlayOnline, IOnlineIdentity::FOnGetUserPrivilegeCompleteDelegate::CreateLambda([this, TestDone](const FUniqueNetId& GetUserPrivilegeUniqueId, EUserPrivileges::Type GetUserPrivilegePrivilege, uint32 GetUserPrivilegePrivilegeResult)
							{
								TestEqual("Verify that GetUserPrivilegePrivilege is: CanPlayOnline", GetUserPrivilegePrivilege == EUserPrivileges::CanPlayOnline, true);
								TestEqual("Verify that GetUserPrivilegePrivilegeResult is: UGCRestriction", GetUserPrivilegePrivilegeResult == (uint32)IOnlineIdentity::EPrivilegeResults::UGCRestriction, true);
								
								UE_LOG_ONLINE_IDENTITY(Error, TEXT("OSS Automation: Test implementation not yet complete. Requires a valid backend configuration"));
								UE_LOG_ONLINE_IDENTITY(Error, TEXT("OSS Automation: Test implementation not yet complete. Needs a way to induce a user generated content restricted state"));
								TestDone.Execute();
							}));
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					// TODO: Figure out how to induce a OnlinePlayRestricted
					LatentIt("When calling GetUserPrivilege with a valid FUniqueNetId who is restricted from online play, this subsystem Delegate call back returns a OnlinePlayRestricted as the PrivilegeResult",  [this](const FDoneDelegate& TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: True", bLoginWasSuccessful, true);

							OnlineIdentity->GetUserPrivilege(LoginUserId, EUserPrivileges::CanPlayOnline, IOnlineIdentity::FOnGetUserPrivilegeCompleteDelegate::CreateLambda([this, TestDone](const FUniqueNetId& GetUserPrivilegeUniqueId, EUserPrivileges::Type GetUserPrivilegePrivilege, uint32 GetUserPrivilegePrivilegeResult)
							{
								TestEqual("Verify that GetUserPrivilegePrivilege is: CanPlayOnline", GetUserPrivilegePrivilege == EUserPrivileges::CanPlayOnline, true);
								TestEqual("Verify that GetUserPrivilegePrivilegeResult is: OnlinePlayRestricted", GetUserPrivilegePrivilegeResult == (uint32)IOnlineIdentity::EPrivilegeResults::OnlinePlayRestricted, true);
								
								UE_LOG_ONLINE_IDENTITY(Error, TEXT("OSS Automation: Test implementation not yet complete. Requires a valid backend configuration"));
								UE_LOG_ONLINE_IDENTITY(Error, TEXT("OSS Automation: Test implementation not yet complete. Needs a way to induce an online play restricted state"));
								TestDone.Execute();
							}));
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});
				});

				Describe("GetPlatformUserIdFromUniqueNetId", [this, SubsystemType]()
				{
					LatentIt("When calling GetPlatformUserIdFromUniqueNetId with a valid FUniqueNetId, the subsystem returns the user's platform id", [this](const FDoneDelegate& TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							TestEqual("Verify that bLoginWasSuccessful returns as: True", bLoginWasSuccessful, true);

							FPlatformUserId PlatformId = PLATFORMUSERID_NONE;
							PlatformId = OnlineIdentity->GetPlatformUserIdFromUniqueNetId(LoginUserId);
							TestNotEqual("Verify that the PlatformId is populated", PlatformId, PLATFORMUSERID_NONE);

							TestDone.Execute();
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					It("When calling GetPlatformUserIdFromUniqueNetId with an invalid FUniqueNetId, the subsystem returns no information", [this]()
					{
						FString InvalidUserIdString = TEXT(" ");
						FUniqueNetIdPtr InvalidUserId = OnlineIdentity->CreateUniquePlayerId(InvalidUserIdString);

						FPlatformUserId PlatformId = PLATFORMUSERID_NONE;

						if (InvalidUserId.IsValid())
						{
							PlatformId = OnlineIdentity->GetPlatformUserIdFromUniqueNetId(*InvalidUserId);
							TestEqual("Verify that the PlatformId has not changed", PlatformId, PLATFORMUSERID_NONE);
						}
						else
						{
							UE_LOG_ONLINE_IDENTITY(Error, TEXT("OSS Automation: IsValid() check on UserId failed after a call to OnlineIdentity->CreateUniquePlayerId()"));
						}
					});
				});

				Describe("GetAuthType", [this, SubsystemType]()
				{
					It("When calling GetAuthType, verify that it returns a non-null FString of 0 or greater size", [this, SubsystemType]()
					{
						FString AuthType = OnlineIdentity->GetAuthType();
						TestEqual("Verify that it returns a non - null FString of 0 or greater size", true, AuthType.Len() >= 0);
					});
				});
			});
		});		
	}

	AfterEach( [this]()
	{
		if (OnlineIdentity.IsValid())
		{
			if (OnlineIdentity->GetLoginStatus(0) == ELoginStatus::LoggedIn)
			{
				OnlineIdentity->Logout(0);
			}

			OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
			OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
			OnlineIdentity = nullptr;			
		}

		FCommandLine::Set(FCommandLine::GetOriginal());
	});

}
