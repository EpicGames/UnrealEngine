// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online.h"
#include "Interfaces/OnlineLeaderboardInterface.h"
#include "Utils/OnlineTestCommon.h"

BEGIN_DEFINE_SPEC(FOnlineLeaderboardSpec, "OnlineLeaderboardInterface", EAutomationTestFlags::EngineFilter | EAutomationTestFlags_ApplicationContextMask)

IOnlineSubsystem* OnlineSubsystem;

IOnlineIdentityPtr OnlineIdentity;
IOnlineLeaderboardsPtr OnlineLeaderboards;

FOnlineAccountCredentials AccountCredentials;
FOnlineAccountCredentials FriendAccountCredentials;

FOnlineTestCommon CommonUtils;

// Delegate Handles
FDelegateHandle OnLogoutCompleteDelegateHandle;
FDelegateHandle	OnLoginCompleteDelegateHandle;
FDelegateHandle OnReadLeaderboardsCompleteDelegateHandle;
FDelegateHandle OnLeaderboardFlushCompleteDelegateHandle;

END_DEFINE_SPEC(FOnlineLeaderboardSpec)

void FOnlineLeaderboardSpec::Define()
{
	TArray<FName> Subsystems = FOnlineTestCommon::GetEnabledTestSubsystems();

	for (int32 Index = 0; Index < Subsystems.Num(); Index++)
	{
		FName SubsystemType = Subsystems[Index];

		Describe(SubsystemType.ToString(), [this, SubsystemType]()
		{
			BeforeEach([this, SubsystemType]()
			{
				CommonUtils = FOnlineTestCommon();
				AccountCredentials = FOnlineTestCommon::GetSubsystemTestAccountCredentials(SubsystemType);
				FriendAccountCredentials = FOnlineTestCommon::GetSubsystemFriendAccountCredentials(SubsystemType);

				OnlineIdentity = Online::GetIdentityInterface(SubsystemType);
				OnlineLeaderboards = Online::GetLeaderboardsInterface(SubsystemType);

				// If OnlineIdentity or OnlineLeaderboards is not valid, the following test, including all other nested BeforeEaches, will not run
				if (!OnlineIdentity.IsValid())
				{
					UE_LOG_ONLINE_LEADERBOARD(Error, TEXT("OSS Automation: Failed to load OnlineIdentity Interface for %s"), *SubsystemType.ToString());
				}

				if (!OnlineLeaderboards.IsValid())
				{
					UE_LOG_ONLINE_LEADERBOARD(Error, TEXT("OSS Automation: Failed to load OnlineLeaderboards Interface for %s"), *SubsystemType.ToString());
				}

			});

			// TODO: No Tests have been validated yet for functionality
			Describe("Online Leaderboard", [this, SubsystemType]()
			{
				xDescribe("ReadLeaderboards", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						TestDone.Execute();
					});

					LatentIt("When calling ReadLeaderboards with a valid Players array and ReadObject, this subsystem returns data about those players from the leaderboards", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_LEADERBOARD(Error, TEXT("OSS Automation: Test not yet implemented"));

						FString TestAccountIdString = CommonUtils.GetSubsystemTestAccountUniqueId(SubsystemType);
						FUniqueNetIdPtr TestAccountId = OnlineIdentity->CreateUniquePlayerId(TestAccountIdString);

						TArray<FUniqueNetIdRef> Players;
						Players.Add(TestAccountId.ToSharedRef());

						FOnlineLeaderboardReadPtr ReadObject = MakeShareable(new FOnlineLeaderboardRead());
						FOnlineLeaderboardReadRef ReadObjectRef = ReadObject.ToSharedRef();

						OnReadLeaderboardsCompleteDelegateHandle = OnlineLeaderboards->AddOnLeaderboardReadCompleteDelegate_Handle(FOnLeaderboardReadCompleteDelegate::CreateLambda([this, ReadObjectRef, TestDone](bool bReadLeaderboardsWasSuccessful)
						{
							TestEqual("Verify that bReadLeaderboardsWasSuccessful returns as: True", bReadLeaderboardsWasSuccessful, true);

							TestEqual("Verify that ReadObject is populated", ReadObjectRef->Rows.Num() > 0, true);

							TestDone.Execute();
						}));

						bool bCallStarted = OnlineLeaderboards->ReadLeaderboards(Players, ReadObjectRef);

						TestEqual("Verify that call started", bCallStarted, true);
					});

					LatentIt("When calling ReadLeaderboards with a valid ReadObject but an invalid Players array, this subsystem does not return data about those players from the leaderboards", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_LEADERBOARD(Error, TEXT("OSS Automation: Test not yet implemented"));

						FUniqueNetIdPtr TestAccountId = OnlineIdentity->CreateUniquePlayerId(TEXT("0123456789"));

						TArray<FUniqueNetIdRef> Players;
						Players.Add(TestAccountId.ToSharedRef());

						FOnlineLeaderboardReadPtr ReadObject = MakeShareable(new FOnlineLeaderboardRead());
						FOnlineLeaderboardReadRef ReadObjectRef = ReadObject.ToSharedRef();

						OnReadLeaderboardsCompleteDelegateHandle = OnlineLeaderboards->AddOnLeaderboardReadCompleteDelegate_Handle(FOnLeaderboardReadCompleteDelegate::CreateLambda([this, ReadObjectRef, TestDone](bool bReadLeaderboardsWasSuccessful)
						{
							TestEqual("Verify that bReadLeaderboardsWasSuccessful returns as: True", bReadLeaderboardsWasSuccessful, true);

							TestEqual("Verify that ReadObject is populated", ReadObjectRef->Rows.Num() > 0, true);

							TestDone.Execute();
						}));

						OnlineLeaderboards->ReadLeaderboards(Players, ReadObjectRef);
					});

					LatentIt("When calling ReadLeaderboards with a valid Players array but an invalid ReadObject, this subsystem does not return data about those players from the leaderboards", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_LEADERBOARD(Error, TEXT("OSS Automation: Test not yet implemented"));

						FString TestAccountIdString = CommonUtils.GetSubsystemTestAccountUniqueId(SubsystemType);
						FUniqueNetIdPtr TestAccountId = OnlineIdentity->CreateUniquePlayerId(TestAccountIdString);

						TArray<FUniqueNetIdRef> Players;
						Players.Add(TestAccountId.ToSharedRef());

						FOnlineLeaderboardReadPtr ReadObject = MakeShareable(new FOnlineLeaderboardRead());
						FOnlineLeaderboardReadRef ReadObjectRef = ReadObject.ToSharedRef();

						OnReadLeaderboardsCompleteDelegateHandle = OnlineLeaderboards->AddOnLeaderboardReadCompleteDelegate_Handle(FOnLeaderboardReadCompleteDelegate::CreateLambda([this, ReadObjectRef, TestDone](bool bReadLeaderboardsWasSuccessful)
						{
							TestEqual("Verify that bReadLeaderboardsWasSuccessful returns as: True", bReadLeaderboardsWasSuccessful, true);

							TestEqual("Verify that ReadObject is populated", ReadObjectRef->Rows.Num() > 0, true);

							TestDone.Execute();
						}));
						
						OnlineLeaderboards->ReadLeaderboards(Players, ReadObjectRef);
					});
				});

				xDescribe("ReadLeaderboardsForFriends", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						TestDone.Execute();
					});

					LatentIt("When calling ReadLeaderboardsForFriends with a valid LocalUserNum and ReadObject, this subsystem returns data about the user's friends from the leaderboards", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_LEADERBOARD(Error, TEXT("OSS Automation: Test not yet implemented"));

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FOnlineLeaderboardReadPtr ReadObject = MakeShareable(new FOnlineLeaderboardRead());
							FOnlineLeaderboardReadRef ReadObjectRef = ReadObject.ToSharedRef();

							OnReadLeaderboardsCompleteDelegateHandle = OnlineLeaderboards->AddOnLeaderboardReadCompleteDelegate_Handle(FOnLeaderboardReadCompleteDelegate::CreateLambda([this, ReadObjectRef, TestDone](bool bReadLeaderboardsWasSuccessful)
							{
								TestEqual("Verify that bReadLeaderboardsWasSuccessful returns as: True", bReadLeaderboardsWasSuccessful, true);

								TestEqual("Verify that ReadObject is populated", ReadObjectRef->Rows.Num() > 0, true);

								TestDone.Execute();
							}));

							OnlineLeaderboards->ReadLeaderboardsForFriends(0, ReadObjectRef);
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling ReadLeaderboardsForFriends with a valid ReadObject but an invalid LocalUserNum, this subsystem does not return data about any user's friends from the leaderboards", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_LEADERBOARD(Error, TEXT("OSS Automation: Test not yet implemented"));

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FOnlineLeaderboardReadPtr ReadObject = MakeShareable(new FOnlineLeaderboardRead());
							FOnlineLeaderboardReadRef ReadObjectRef = ReadObject.ToSharedRef();

							OnReadLeaderboardsCompleteDelegateHandle = OnlineLeaderboards->AddOnLeaderboardReadCompleteDelegate_Handle(FOnLeaderboardReadCompleteDelegate::CreateLambda([this, ReadObjectRef, TestDone](bool bReadLeaderboardsWasSuccessful)
							{
								TestEqual("Verify that bReadLeaderboardsWasSuccessful returns as: True", bReadLeaderboardsWasSuccessful, true);
								TestEqual("Verify that ReadObject is not populated", ReadObjectRef->Rows.Num() == 0, true);

								TestDone.Execute();
							}));

							OnlineLeaderboards->ReadLeaderboardsForFriends(-1, ReadObjectRef);
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling ReadLeaderboardsForFriends with a valid LocalUserNum but an invalid ReadObject, this subsystem does not return data about any user's friends from the leaderboards", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_LEADERBOARD(Error, TEXT("OSS Automation: Test not yet implemented"));

						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							FOnlineLeaderboardReadRef ReadObjectRef;

							OnReadLeaderboardsCompleteDelegateHandle = OnlineLeaderboards->AddOnLeaderboardReadCompleteDelegate_Handle(FOnLeaderboardReadCompleteDelegate::CreateLambda([this, ReadObjectRef, TestDone](bool bReadLeaderboardsWasSuccessful)
							{
								TestEqual("Verify that bReadLeaderboardsWasSuccessful returns as: True", bReadLeaderboardsWasSuccessful, true);
								TestEqual("Verify that ReadObject is not populated", ReadObjectRef->Rows.Num() == 0, true);

								TestDone.Execute();
							}));

							OnlineLeaderboards->ReadLeaderboardsForFriends(0, ReadObjectRef);
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});
				});

				// TODO: Test with multiple accounts that are on the leaderboards?
				xDescribe("ReadLeaderboardsAroundRank", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						TestDone.Execute();
					});

					LatentIt("When calling ReadLeaderboardsAroundRank with a valid Rank, Range, and ReadObject, this subsystem returns data about players in that rank and range from the leaderboards", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_LEADERBOARD(Error, TEXT("OSS Automation: Test not yet implemented"));

						FOnlineLeaderboardReadPtr ReadObject = MakeShareable(new FOnlineLeaderboardRead());
						FOnlineLeaderboardReadRef ReadObjectRef = ReadObject.ToSharedRef();

						OnlineLeaderboards->ReadLeaderboardsAroundRank(1, 1, ReadObjectRef);

						TestEqual("Verify that ReadObject is populated", ReadObjectRef->Rows.Num() > 0, true);
					});

					LatentIt("When calling ReadLeaderboardsAroundRank with a valid Range and ReadObject but an invalid Rank, this subsystem does not return data about players in that rank and range from the leaderboards", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_LEADERBOARD(Error, TEXT("OSS Automation: Test not yet implemented"));

						FOnlineLeaderboardReadPtr ReadObject = MakeShareable(new FOnlineLeaderboardRead());
						FOnlineLeaderboardReadRef ReadObjectRef = ReadObject.ToSharedRef();

						OnlineLeaderboards->ReadLeaderboardsAroundRank(-1, 1, ReadObjectRef);

						TestEqual("Verify that ReadObject is not populated", ReadObjectRef->Rows.Num() == 0, true);
					});

					LatentIt("When calling ReadLeaderboardsAroundRank with a valid Rank and ReadObject but an invalid Range, this subsystem does not return data about players in that rank and range from the leaderboards", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_LEADERBOARD(Error, TEXT("OSS Automation: Test not yet implemented"));

						FOnlineLeaderboardReadPtr ReadObject = MakeShareable(new FOnlineLeaderboardRead());
						FOnlineLeaderboardReadRef ReadObjectRef = ReadObject.ToSharedRef();

						OnlineLeaderboards->ReadLeaderboardsAroundRank(1, -1, ReadObjectRef);

						TestEqual("Verify that ReadObject is not populated", ReadObjectRef->Rows.Num() == 0, true);
					});

					LatentIt("When calling ReadLeaderboardsAroundRank with a valid Rank and Range but an invalid ReadObject, this subsystem does not return data about players in that rank and range from the leaderboards", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_LEADERBOARD(Error, TEXT("OSS Automation: Test not yet implemented"));

						FOnlineLeaderboardReadPtr ReadObject = MakeShareable(new FOnlineLeaderboardRead());
						FOnlineLeaderboardReadRef ReadObjectRef = ReadObject.ToSharedRef();

						OnlineLeaderboards->ReadLeaderboardsAroundRank(1, 1, ReadObjectRef);

						TestEqual("Verify that ReadObject is not populated", ReadObjectRef->Rows.Num() == 0, true);
					});
				});

				xDescribe("ReadLeaderboardsAroundUser", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						TestDone.Execute();
					});

					LatentIt("When calling ReadLeaderboardsAroundUser with a valid Player, Range, and ReadObject, this subsystem returns data around that user from the leaderboards", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_LEADERBOARD(Error, TEXT("OSS Automation: Test not yet implemented"));

						FString TestAccountIdString = CommonUtils.GetSubsystemTestAccountUniqueId(SubsystemType);
						FUniqueNetIdPtr TestAccountId = OnlineIdentity->CreateUniquePlayerId(TestAccountIdString);

						FOnlineLeaderboardReadPtr ReadObject = MakeShareable(new FOnlineLeaderboardRead());
						FOnlineLeaderboardReadRef ReadObjectRef = ReadObject.ToSharedRef();

						OnlineLeaderboards->ReadLeaderboardsAroundUser(TestAccountId.ToSharedRef(), 1, ReadObjectRef);

						TestEqual("Verify that ReadObject is populated", ReadObjectRef->Rows.Num() > 0, true);
					});

					LatentIt("When calling ReadLeaderboardsAroundUser with a valid Range and ReadObject but an invalid Player, this subsystem does not return data around any user from the leaderboards", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_LEADERBOARD(Error, TEXT("OSS Automation: Test not yet implemented"));

						FUniqueNetIdPtr TestAccountId = OnlineIdentity->CreateUniquePlayerId(TEXT("0123456789"));

						FOnlineLeaderboardReadPtr ReadObject = MakeShareable(new FOnlineLeaderboardRead());
						FOnlineLeaderboardReadRef ReadObjectRef = ReadObject.ToSharedRef();

						OnlineLeaderboards->ReadLeaderboardsAroundUser(TestAccountId.ToSharedRef(), 1, ReadObjectRef);

						TestEqual("Verify that ReadObject is not populated", ReadObjectRef->Rows.Num() == 0, true);
					});

					LatentIt("When calling ReadLeaderboardsAroundUser with a valid Player and ReadObject but an invalid Range, this subsystem does not return data around that user from the leaderboards", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_LEADERBOARD(Error, TEXT("OSS Automation: Test not yet implemented"));

						FString TestAccountIdString = CommonUtils.GetSubsystemTestAccountUniqueId(SubsystemType);
						FUniqueNetIdPtr TestAccountId = OnlineIdentity->CreateUniquePlayerId(TestAccountIdString);

						FOnlineLeaderboardReadPtr ReadObject = MakeShareable(new FOnlineLeaderboardRead());
						FOnlineLeaderboardReadRef ReadObjectRef = ReadObject.ToSharedRef();

						OnlineLeaderboards->ReadLeaderboardsAroundUser(TestAccountId.ToSharedRef(), -1, ReadObjectRef);

						TestEqual("Verify that ReadObject is not populated", ReadObjectRef->Rows.Num() == 0, true);
					});

					LatentIt("When calling ReadLeaderboardsAroundUser with a valid Player and Range but an invalid ReadObject, this subsystem does not return data around that user from the leaderboards", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_LEADERBOARD(Error, TEXT("OSS Automation: Test not yet implemented"));

						FString TestAccountIdString = CommonUtils.GetSubsystemTestAccountUniqueId(SubsystemType);
						FUniqueNetIdPtr TestAccountId = OnlineIdentity->CreateUniquePlayerId(TestAccountIdString);

						FOnlineLeaderboardReadRef ReadObjectRef;

						OnlineLeaderboards->ReadLeaderboardsAroundUser(TestAccountId.ToSharedRef(), 1, ReadObjectRef);

						TestEqual("Verify that ReadObject is not populated", ReadObjectRef->Rows.Num() == 0, true);
					});
				});

				xDescribe("FreeStats", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						TestDone.Execute();
					});

					LatentIt("When calling FreeStats with a valid ReadObject, this subsystem cleans up allocated stats data", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_LEADERBOARD(Error, TEXT("OSS Automation: Test not yet implemented"));

						FOnlineLeaderboardReadPtr ReadObject = MakeShareable(new FOnlineLeaderboardRead());
						FOnlineLeaderboardReadRef ReadObjectRef = ReadObject.ToSharedRef();

						OnlineLeaderboards->FreeStats(*ReadObjectRef);

						TestEqual("Verify that ReadObject is not populated", ReadObjectRef->Rows.Num() == 0, true);

						TestDone.Execute();
					});

					LatentIt("When calling FreeStats with an invalid ReadObject, this subsystem does not clean up allocated stats data", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_LEADERBOARD(Error, TEXT("OSS Automation: Test not yet implemented"));

						FOnlineLeaderboardReadPtr ReadObject = MakeShareable(new FOnlineLeaderboardRead());
						FOnlineLeaderboardReadRef ReadObjectRef = ReadObject.ToSharedRef();

						OnlineLeaderboards->FreeStats(*ReadObjectRef);
					});
				});

				xDescribe("WriteLeaderboards", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						TestDone.Execute();
					});

					LatentIt("When calling WriteLeaderboards with a valid SessionName, Player, and WriteObject, this subsystem writes stats to to the subsystem's cache", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_LEADERBOARD(Error, TEXT("OSS Automation: Test not yet implemented"));

						const FString TestAccountIdString = CommonUtils.GetSubsystemTestAccountUniqueId(SubsystemType);
						const FString TestStatString = TEXT("TestStat");
						FUniqueNetIdPtr TestAccountId = OnlineIdentity->CreateUniquePlayerId(TestAccountIdString);

						FOnlineLeaderboardWrite LeaderboardWriteObject;
						LeaderboardWriteObject.LeaderboardNames.Add(FString(TEXT("TestLeaderboard")));
						LeaderboardWriteObject.RatedStat = TestStatString;
						LeaderboardWriteObject.DisplayFormat = ELeaderboardFormat::Number;
						LeaderboardWriteObject.SortMethod = ELeaderboardSort::Descending;
						LeaderboardWriteObject.UpdateMethod = ELeaderboardUpdateMethod::KeepBest;

						LeaderboardWriteObject.SetIntStat(TestStatString, 50);

						bool bCallStarted = OnlineLeaderboards->WriteLeaderboards(TEXT("TestSessionName"), *TestAccountId, LeaderboardWriteObject);
						
						TestEqual("Verify that bCallStarted returns as: True", bCallStarted, true);

						// How to see what's in cache? Maybe write to leaderboard then read from leaderboard?
					});

					LatentIt("When calling WriteLeaderboards with a valid Player and WriteObject but an invalid SessionName, this subsystem does not write stats to to the subsystem's cache", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_LEADERBOARD(Error, TEXT("OSS Automation: Test not yet implemented"));

						const FString TestAccountIdString = CommonUtils.GetSubsystemTestAccountUniqueId(SubsystemType);
						const FString TestStatString = TEXT("TestStat");
						FUniqueNetIdPtr TestAccountId = OnlineIdentity->CreateUniquePlayerId(TestAccountIdString);

						FOnlineLeaderboardWrite LeaderboardWriteObject;
						LeaderboardWriteObject.LeaderboardNames.Add(FString(TEXT("TestLeaderboard")));
						LeaderboardWriteObject.RatedStat = TestStatString;
						LeaderboardWriteObject.DisplayFormat = ELeaderboardFormat::Number;
						LeaderboardWriteObject.SortMethod = ELeaderboardSort::Descending;
						LeaderboardWriteObject.UpdateMethod = ELeaderboardUpdateMethod::KeepBest;

						LeaderboardWriteObject.SetIntStat(TestStatString, 50);

						OnlineLeaderboards->WriteLeaderboards(TEXT(""), *TestAccountId, LeaderboardWriteObject);

						// How to see what's in cache?
					});

					LatentIt("When calling WriteLeaderboards with a valid SessionName and WriteObject but an invalid Player, this subsystem does not write stats to to the subsystem's cache", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_LEADERBOARD(Error, TEXT("OSS Automation: Test not yet implemented"));

						const FString TestStatString = TEXT("TestStat");
						FUniqueNetIdPtr TestAccountId = OnlineIdentity->CreateUniquePlayerId(TEXT("0123456789"));

						FOnlineLeaderboardWrite LeaderboardWriteObject;
						LeaderboardWriteObject.LeaderboardNames.Add(FString(TEXT("TestLeaderboard")));
						LeaderboardWriteObject.RatedStat = TestStatString;
						LeaderboardWriteObject.DisplayFormat = ELeaderboardFormat::Number;
						LeaderboardWriteObject.SortMethod = ELeaderboardSort::Descending;
						LeaderboardWriteObject.UpdateMethod = ELeaderboardUpdateMethod::KeepBest;

						LeaderboardWriteObject.SetIntStat(TestStatString, 50);

						OnlineLeaderboards->WriteLeaderboards(TEXT("TestSessionName"), *TestAccountId, LeaderboardWriteObject);

						// How to see what's in cache?
					});

					LatentIt("When calling WriteLeaderboards with a valid SessionName and Player but an invalid WriteObject, this subsystem does not write stats to to the subsystem's cache", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_LEADERBOARD(Error, TEXT("OSS Automation: Test not yet implemented"));

						FString TestAccountIdString = CommonUtils.GetSubsystemTestAccountUniqueId(SubsystemType);
						FUniqueNetIdPtr TestAccountId = OnlineIdentity->CreateUniquePlayerId(TestAccountIdString);

						FOnlineLeaderboardWrite LeaderboardWriteObject;

						OnlineLeaderboards->WriteLeaderboards(TEXT("TestSessionName"), *TestAccountId, LeaderboardWriteObject);

						// How to see what's in cache?
					});
				});

				// How to clean up Leaderboard after writing to it?
				xDescribe("FlushLeaderboards", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						TestDone.Execute();
					});

					LatentIt("When calling FlushLeaderboards with a valid SessionName and stats in the cache, this subsystem commits those stats changes to the leaderboard", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_LEADERBOARD(Error, TEXT("OSS Automation: Test not yet implemented"));

						const FString TestAccountIdString = CommonUtils.GetSubsystemTestAccountUniqueId(SubsystemType);
						const FString TestStatString = TEXT("TestStat");
						FUniqueNetIdPtr TestAccountId = OnlineIdentity->CreateUniquePlayerId(TestAccountIdString);

						FOnlineLeaderboardWrite LeaderboardWriteObject;
						LeaderboardWriteObject.LeaderboardNames.Add(FString(TEXT("TestLeaderboard")));
						LeaderboardWriteObject.RatedStat = TestStatString;
						LeaderboardWriteObject.DisplayFormat = ELeaderboardFormat::Number;
						LeaderboardWriteObject.SortMethod = ELeaderboardSort::Descending;
						LeaderboardWriteObject.UpdateMethod = ELeaderboardUpdateMethod::KeepBest;

						LeaderboardWriteObject.SetIntStat(TestStatString, 50);

						OnlineLeaderboards->WriteLeaderboards(TEXT("TestSessionName"), *TestAccountId, LeaderboardWriteObject);

						OnlineLeaderboards->AddOnLeaderboardFlushCompleteDelegate_Handle(FOnLeaderboardFlushCompleteDelegate::CreateLambda([this, SubsystemType, TestAccountId, TestDone](const FName LeaderboardFlushSessionName, bool bLeaderboardFlushWasSuccessful)
						{
							TestEqual("Verify that LeaderboardFlushSessionName is: TestSessionName", LeaderboardFlushSessionName == TEXT("TestSessionName"), true);
							TestEqual("Verify that bCallStarted returns as: True", bLeaderboardFlushWasSuccessful, true);

							TArray<FUniqueNetIdRef> Players;
							Players.Add(TestAccountId.ToSharedRef());

							FOnlineLeaderboardReadPtr ReadObject = MakeShareable(new FOnlineLeaderboardRead());
							FOnlineLeaderboardReadRef ReadObjectRef = ReadObject.ToSharedRef();

							OnReadLeaderboardsCompleteDelegateHandle = OnlineLeaderboards->AddOnLeaderboardReadCompleteDelegate_Handle(FOnLeaderboardReadCompleteDelegate::CreateLambda([this, ReadObjectRef, TestDone](bool bReadLeaderboardsWasSuccessful)
							{
								TestEqual("Verify that bReadLeaderboardsWasSuccessful returns as: True", bReadLeaderboardsWasSuccessful, true);
								TestEqual("Verify that ReadObject is populated", ReadObjectRef->Rows.Num() > 0, true);
								TestDone.Execute();
							}));

							OnlineLeaderboards->ReadLeaderboards(Players, ReadObjectRef);
						}));

						OnlineLeaderboards->FlushLeaderboards(TEXT("TestSessionName"));
					});

					LatentIt("When calling FlushLeaderboards with an invalid SessionName and stats in the cache, this subsystem does not commit those stats changes to the leaderboard", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_LEADERBOARD(Error, TEXT("OSS Automation: Test not yet implemented"));
					});

					LatentIt("When calling FlushLeaderboards with a valid SessionName but no stats in the cache, this subsystem does not commit any stats changes to the leaderboard", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_LEADERBOARD(Error, TEXT("OSS Automation: Test not yet implemented"));
					});
				});

				// FOnlinePlayerScore is not implemented
				xDescribe("WriteOnlinePlayerRatings", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						TestDone.Execute();
					});

					LatentIt("When calling WriteOnlinePlayerRatings with a valid SessionName, LeaderboardId, and PlayerScores array, this subsystem writes that score data to the leaderboard", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_LEADERBOARD(Error, TEXT("OSS Automation: Test not yet implemented"));

						TArray<FOnlinePlayerScore> PlayerScores;

						OnlineLeaderboards->WriteOnlinePlayerRatings(TEXT("TestSessionName"), 1, PlayerScores);
					});

					LatentIt("When calling WriteOnlinePlayerRatings with a valid LeaderboardId and PlayerScores array but an invalid SessionName, this subsystem does not write that score data to the leaderboard", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_LEADERBOARD(Error, TEXT("OSS Automation: Test not yet implemented"));
					});

					LatentIt("When calling WriteOnlinePlayerRatings with a valid SessionName and PlayerScores array but an invalid LeaderboardId, this subsystem does not write that score data to the leaderboard", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_LEADERBOARD(Error, TEXT("OSS Automation: Test not yet implemented"));
					});

					LatentIt("When calling WriteOnlinePlayerRatings with a valid SessionName and LeaderboardId but an invalid PlayerScores array, this subsystem does not write that score data to the leaderboard", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						UE_LOG_ONLINE_LEADERBOARD(Error, TEXT("OSS Automation: Test not yet implemented"));
					});
				});
			});

			AfterEach(EAsyncExecution::ThreadPool, [this]()
			{
				// Clean up Identity
				if (OnlineIdentity.IsValid())
				{
					if (OnlineIdentity->GetLoginStatus(0) == ELoginStatus::LoggedIn)
					{
						OnlineIdentity->Logout(0);
					}

					OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
					OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
					OnlineIdentity = nullptr;
				}

				// Clean up OnlineLeaderboards
				if (OnlineLeaderboards.IsValid())
				{
					OnlineLeaderboards->ClearOnLeaderboardReadCompleteDelegate_Handle(OnReadLeaderboardsCompleteDelegateHandle);
					OnlineLeaderboards->ClearOnLeaderboardFlushCompleteDelegate_Handle(OnLeaderboardFlushCompleteDelegateHandle);
					OnlineLeaderboards = nullptr;
				}
			});
		});
	}
}
