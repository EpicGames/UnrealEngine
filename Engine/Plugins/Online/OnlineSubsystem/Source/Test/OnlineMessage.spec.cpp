// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online.h"
#include "Interfaces/OnlinePresenceInterface.h"
#include "Utils/OnlineTestCommon.h"

BEGIN_DEFINE_SPEC(FOnlineMessageSpec, "OnlineMessageInterface", EAutomationTestFlags::EngineFilter | EAutomationTestFlags_ApplicationContextMask)

IOnlineSubsystem* OnlineSubsystem;

IOnlineIdentityPtr OnlineIdentity;
IOnlineFriendsPtr OnlineFriends;
IOnlineMessagePtr OnlineMessage;

FOnlineAccountCredentials AccountCredentials;
FOnlineAccountCredentials FriendAccountCredentials;

FOnlineTestCommon CommonUtils;

const IOnlinePresence::FOnPresenceTaskCompleteDelegate PresenceCompleteDelegate;

// Delegate Handles
FDelegateHandle OnLogoutCompleteDelegateHandle;
FDelegateHandle	OnLoginCompleteDelegateHandle;
FDelegateHandle OnEnumerateMessagesCompleteDelegateHandle;
FDelegateHandle OnSendMessageCompleteDelegateHandle;
FDelegateHandle OnDeleteMessageCompleteDelegateHandle;

END_DEFINE_SPEC(FOnlineMessageSpec)

void FOnlineMessageSpec::Define()
{
	TArray<FName> Subsystems = FOnlineTestCommon::GetEnabledTestSubsystems();

	for (int32 Index = 0; Index < Subsystems.Num(); Index++)
	{
		FName SubsystemType = Subsystems[Index];

		Describe(SubsystemType.ToString(), [this, SubsystemType]()
		{
			BeforeEach([this, SubsystemType]()
			{
				CommonUtils = FOnlineTestCommon();
				AccountCredentials = FOnlineTestCommon::GetSubsystemTestAccountCredentials(SubsystemType);
				FriendAccountCredentials = FOnlineTestCommon::GetSubsystemFriendAccountCredentials(SubsystemType);

				OnlineSubsystem = IOnlineSubsystem::Get(SubsystemType);

				OnlineIdentity = Online::GetIdentityInterface(SubsystemType);
				OnlineFriends = Online::GetFriendsInterface(SubsystemType);
				OnlineMessage = OnlineSubsystem->GetMessageInterface();

				// If OnlineIdentity, OnlineFriends, or OnlineMessage is not valid, the following test, including all other nested BeforeEaches, will not run
				if (!OnlineIdentity.IsValid())
				{
					UE_LOG_ONLINE(Error, TEXT("OSS Automation: Failed to load OnlineIdentity Interface for %s"), *SubsystemType.ToString());
				}

				if (!OnlineFriends.IsValid())
				{
					UE_LOG_ONLINE(Error, TEXT("OSS Automation: Failed to load OnlineFriends Interface for %s"), *SubsystemType.ToString());
				}

				if (!OnlineMessage.IsValid())
				{
					UE_LOG_ONLINE(Error, TEXT("OSS Automation: Failed to load OnlineMessage Interface for %s"), *SubsystemType.ToString());
				}

			});

			// TODO: No Tests have been validated yet for functionality
			Describe("Online Message", [this, SubsystemType]()
			{
				Describe("EnumerateMessages", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						CommonUtils.SendMessageToTestAccount(OnlineIdentity, OnlineFriends, OnlineMessage, SubsystemType, TestDone);
					});

					LatentIt("When calling EnumerateMessages with a valid local user, this subsystem populates the cached message headers array", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnEnumerateMessagesCompleteDelegateHandle = OnlineMessage->AddOnEnumerateMessagesCompleteDelegate_Handle(0, FOnEnumerateMessagesCompleteDelegate::CreateLambda([this, TestDone](int32 EnumerateMessagesLocalUserNum, bool bEnumerateMessageWasSuccessful, const FString& EnumerateMessagesErrorStr)
							{
								TestEqual("Verify that EnumerateMessagesLocalUserNum is: 0", EnumerateMessagesLocalUserNum, 0);
								TestEqual("Verify that bEnumerateMessageWasSuccessful returns as: True", bEnumerateMessageWasSuccessful, true);
								TestEqual("Verify that EnumerateMessagesErrorStr is empty", EnumerateMessagesErrorStr.IsEmpty(), true);

								TArray<TSharedRef<FOnlineMessageHeader>> MessageHeaders;
								OnlineMessage->GetMessageHeaders(0, MessageHeaders);

								TestEqual("Verify that MessageHeaders is populated", MessageHeaders.Num() > 0, true);

								TestDone.Execute();
							}));

							OnlineMessage->EnumerateMessages(0);
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					xLatentIt("When calling EnumerateMessages with an invalid local user (-1), this subsystem does not populate the cached message headers array", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnEnumerateMessagesCompleteDelegateHandle = OnlineMessage->AddOnEnumerateMessagesCompleteDelegate_Handle(0, FOnEnumerateMessagesCompleteDelegate::CreateLambda([this, TestDone](int32 EnumerateMessagesLocalUserNum, bool bEnumerateMessageWasSuccessful, const FString& EnumerateMessagesErrorStr)
							{
								TestEqual("Verify that EnumerateMessagesLocalUserNum is: 0", EnumerateMessagesLocalUserNum, 0);
								TestEqual("Verify that bEnumerateMessageWasSuccessful returns as: False", bEnumerateMessageWasSuccessful, false);
								TestEqual("Verify that EnumerateMessagesErrorStr is empty", EnumerateMessagesErrorStr.IsEmpty(), true);

								TestDone.Execute();
							}));

							OnlineMessage->EnumerateMessages(-1);
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});
				});

				xDescribe("GetMessageHeaders", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						CommonUtils.SendMessageToTestAccount(OnlineIdentity, OnlineFriends, OnlineMessage, SubsystemType, TestDone);
					});

					LatentIt("When calling GetMessageHeaders with a valid local user, this subsystem returns the cached message headers array", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnEnumerateMessagesCompleteDelegateHandle = OnlineMessage->AddOnEnumerateMessagesCompleteDelegate_Handle(0, FOnEnumerateMessagesCompleteDelegate::CreateLambda([this, TestDone](int32 EnumerateMessagesLocalUserNum, bool bEnumerateMessageWasSuccessful, const FString& EnumerateMessagesErrorStr)
							{
								TestEqual("Verify that bEnumerateMessageWasSuccessful returns as: True", bEnumerateMessageWasSuccessful, true);

								TArray<TSharedRef<FOnlineMessageHeader>> MessageHeaders;
								OnlineMessage->GetMessageHeaders(0, MessageHeaders);

								TestEqual("Verify that MessageHeaders is populated", MessageHeaders.Num() > 0, true);
								TestDone.Execute();
							}));

							OnlineMessage->EnumerateMessages(0);
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling GetMessageHeaders with an invalid local user (-1), this subsystem does not return the cached message headers array", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnEnumerateMessagesCompleteDelegateHandle = OnlineMessage->AddOnEnumerateMessagesCompleteDelegate_Handle(0, FOnEnumerateMessagesCompleteDelegate::CreateLambda([this, TestDone](int32 EnumerateMessagesLocalUserNum, bool bEnumerateMessageWasSuccessful, const FString& EnumerateMessagesErrorStr)
							{
								TestEqual("Verify that bEnumerateMessageWasSuccessful returns as: True", bEnumerateMessageWasSuccessful, true);

								TArray<TSharedRef<FOnlineMessageHeader>> MessageHeaders;
								OnlineMessage->GetMessageHeaders(-1, MessageHeaders);

								TestEqual("Verify that MessageHeaders is not populated", MessageHeaders.Num() == 0, true);
								TestDone.Execute();
							}));

							OnlineMessage->EnumerateMessages(0);
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});
				});

				xDescribe("ClearMessageHeaders", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						CommonUtils.SendMessageToTestAccount(OnlineIdentity, OnlineFriends, OnlineMessage, SubsystemType, TestDone);
					});

					LatentIt("When calling ClearMessageHeaders with a valid local user, this subsystem will clear the given cached message", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnEnumerateMessagesCompleteDelegateHandle = OnlineMessage->AddOnEnumerateMessagesCompleteDelegate_Handle(0, FOnEnumerateMessagesCompleteDelegate::CreateLambda([this, TestDone](int32 EnumerateMessagesLocalUserNum, bool bEnumerateMessageWasSuccessful, const FString& EnumerateMessagesErrorStr)
							{
								TestEqual("Verify that bEnumerateMessageWasSuccessful returns as: True", bEnumerateMessageWasSuccessful, true);

								OnlineMessage->ClearMessageHeaders(0);

								TArray<TSharedRef<FOnlineMessageHeader>> MessageHeaders;
								OnlineMessage->GetMessageHeaders(0, MessageHeaders);

								TestEqual("Verify that MessageHeaders is not populated", MessageHeaders.Num() == 0, true);
								TestDone.Execute();
							}));

							OnlineMessage->EnumerateMessages(0);
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling ClearMessageHeaders with an invalid local user (-1), this subsystem will not clear the given cached message", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnEnumerateMessagesCompleteDelegateHandle = OnlineMessage->AddOnEnumerateMessagesCompleteDelegate_Handle(0, FOnEnumerateMessagesCompleteDelegate::CreateLambda([this, TestDone](int32 EnumerateMessagesLocalUserNum, bool bEnumerateMessageWasSuccessful, const FString& EnumerateMessagesErrorStr)
							{
								TestEqual("Verify that bEnumerateMessageWasSuccessful returns as: True", bEnumerateMessageWasSuccessful, true);

								OnlineMessage->ClearMessageHeaders(-1);

								TArray<TSharedRef<FOnlineMessageHeader>> MessageHeaders;
								OnlineMessage->GetMessageHeaders(0, MessageHeaders);

								TestEqual("Verify that MessageHeaders is populated", MessageHeaders.Num() > 0, true);
								TestDone.Execute();
							}));

							OnlineMessage->EnumerateMessages(0);
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});
				});

				xDescribe("ReadMessage", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						CommonUtils.SendMessageToTestAccount(OnlineIdentity, OnlineFriends, OnlineMessage, SubsystemType, TestDone);
					});

					LatentIt("When calling ReadMessage with a valid local user and MessageId, this subsystem will cache that message's contents", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnEnumerateMessagesCompleteDelegateHandle = OnlineMessage->AddOnEnumerateMessagesCompleteDelegate_Handle(0, FOnEnumerateMessagesCompleteDelegate::CreateLambda([this, TestDone](int32 EnumerateMessagesLocalUserNum, bool bEnumerateMessageWasSuccessful, const FString& EnumerateMessagesErrorStr)
							{
								TestEqual("Verify that bEnumerateMessageWasSuccessful returns as: True", bEnumerateMessageWasSuccessful, true);

								TArray<TSharedRef<FOnlineMessageHeader>> MessageHeaders;
								OnlineMessage->GetMessageHeaders(0, MessageHeaders);

								if (MessageHeaders.Num() > 0)
								{
									FUniqueMessageIdRef MsgId = MessageHeaders[0]->MessageId;

									OnlineMessage->AddOnReadMessageCompleteDelegate_Handle(0, FOnReadMessageCompleteDelegate::CreateLambda([this, MsgId, TestDone](int32 ReadMessageLocalUserNum, bool bReadMessageWasSuccessful, const FUniqueMessageId& ReadMessageMessageId, const FString& ReadMessageErrorStr)
									{
										TestEqual("Verify that ReadMessageLocalUserNum is: 0", ReadMessageLocalUserNum, 0);
										TestEqual("Verify that bReadMessageWasSuccessful returns as: True", bReadMessageWasSuccessful, true);
										TestEqual("Verify that ReadMessageMessageId is still equal to MsgId", ReadMessageMessageId == *MsgId, true);
										TestEqual("Verify that EnumerateMessagesErrorStr is empty", ReadMessageErrorStr.IsEmpty(), true);

										TSharedPtr<FOnlineMessage> ReceivedMessage;
										ReceivedMessage = OnlineMessage->GetMessage(0, ReadMessageMessageId);

										TestEqual("Verify that ReceivedMessage pointer is valid", ReceivedMessage.IsValid(), true);
										TestDone.Execute();
									}));

									OnlineMessage->ReadMessage(0, *MsgId);
								}
								else
								{
									UE_LOG_ONLINE(Error, TEXT("OSS Automation: MessageHeaders still unpopulated after a call to OnlineMessage->GetMessageHeaders()"));
									TestDone.Execute();
								}
							}));

							OnlineMessage->EnumerateMessages(0);
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					// TODO: See what a MessageId actually is and then how to fake it
					LatentIt("When calling ReadMessage with a valid local user and an invalid MessageId, this subsystem will not cache any message's contents", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						TestDone.Execute();
					});

					LatentIt("When calling ReadMessage with a valid MessageId but an invalid local user (-1), this subsystem will not cache that message's contents", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnEnumerateMessagesCompleteDelegateHandle = OnlineMessage->AddOnEnumerateMessagesCompleteDelegate_Handle(0, FOnEnumerateMessagesCompleteDelegate::CreateLambda([this, TestDone](int32 EnumerateMessagesLocalUserNum, bool bEnumerateMessageWasSuccessful, const FString& EnumerateMessagesErrorStr)
							{
								TestEqual("Verify that bEnumerateMessageWasSuccessful returns as: True", bEnumerateMessageWasSuccessful, true);

								TArray<TSharedRef<FOnlineMessageHeader>> MessageHeaders;
								OnlineMessage->GetMessageHeaders(0, MessageHeaders);

								if (MessageHeaders.Num() > 0)
								{
									FUniqueMessageIdRef MsgId = MessageHeaders[0]->MessageId;

									OnlineMessage->AddOnReadMessageCompleteDelegate_Handle(0, FOnReadMessageCompleteDelegate::CreateLambda([this, MsgId, TestDone](int32 ReadMessageLocalUserNum, bool bReadMessageWasSuccessful, const FUniqueMessageId& ReadMessageMessageId, const FString& ReadMessageErrorStr)
									{
										TestEqual("Verify that ReadMessageLocalUserNum is: 0", ReadMessageLocalUserNum, 0);
										TestEqual("Verify that bReadMessageWasSuccessful returns as: False", bReadMessageWasSuccessful, false);
										TestEqual("Verify that ReadMessageMessageId is still equal to MsgId", ReadMessageMessageId == *MsgId, true);
										TestEqual("Verify that EnumerateMessagesErrorStr is empty", ReadMessageErrorStr.IsEmpty(), true);

										TSharedPtr<FOnlineMessage> ReceivedMessage;
										ReceivedMessage = OnlineMessage->GetMessage(-1, ReadMessageMessageId);

										TestEqual("Verify that ReceivedMessage pointer is invalid", ReceivedMessage.IsValid(), false);
										TestDone.Execute();
									}));

									OnlineMessage->ReadMessage(0, *MsgId);
								}
								else
								{
									UE_LOG_ONLINE(Error, TEXT("OSS Automation: MessageHeaders still unpopulated after a call to OnlineMessage->GetMessageHeaders()"));
									TestDone.Execute();
								}
							}));

							OnlineMessage->EnumerateMessages(0);
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});
				});

				xDescribe("GetMessage", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						CommonUtils.SendMessageToTestAccount(OnlineIdentity, OnlineFriends, OnlineMessage, SubsystemType, TestDone);
					});

					LatentIt("When calling GetMessage with a valid local user and MessageId, this subsystem will return that message's contents", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnEnumerateMessagesCompleteDelegateHandle = OnlineMessage->AddOnEnumerateMessagesCompleteDelegate_Handle(0, FOnEnumerateMessagesCompleteDelegate::CreateLambda([this, TestDone](int32 EnumerateMessagesLocalUserNum, bool bEnumerateMessageWasSuccessful, const FString& EnumerateMessagesErrorStr)
							{
								TestEqual("Verify that bEnumerateMessageWasSuccessful returns as: True", bEnumerateMessageWasSuccessful, true);

								TArray<TSharedRef<FOnlineMessageHeader>> MessageHeaders;
								OnlineMessage->GetMessageHeaders(0, MessageHeaders);

								if (MessageHeaders.Num() > 0)
								{
									FUniqueMessageIdRef MsgId = MessageHeaders[0]->MessageId;

									OnlineMessage->AddOnReadMessageCompleteDelegate_Handle(0, FOnReadMessageCompleteDelegate::CreateLambda([this, MsgId, TestDone](int32 ReadMessageLocalUserNum, bool bReadMessageWasSuccessful, const FUniqueMessageId& ReadMessageMessageId, const FString& ReadMessageErrorStr)
									{
										TestEqual("Verify that bReadMessageWasSuccessful returns as: True", bReadMessageWasSuccessful, true);

										TSharedPtr<FOnlineMessage> ReceivedMessage;
										ReceivedMessage = OnlineMessage->GetMessage(0, ReadMessageMessageId);

										if (ReceivedMessage.IsValid())
										{
											FString MessageString = ReceivedMessage->Payload.ToJsonStr();
											TestEqual("Verify that MessageString is populated", MessageString.Len() > 0, true);
											TestDone.Execute();
										}
										else
										{
											UE_LOG_ONLINE(Error, TEXT("OSS Automation: IsValid() check on ReceivedMessage failed after a call to OnlineMessage->GetMessage()"));
											TestDone.Execute();
										}
									}));

									OnlineMessage->ReadMessage(0, *MsgId);
								}
								else
								{
									UE_LOG_ONLINE(Error, TEXT("OSS Automation: MessageHeaders still unpopulated after a call to OnlineMessage->GetMessageHeaders()"));
									TestDone.Execute();
								}
							}));

							OnlineMessage->EnumerateMessages(0);
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					// TODO: See what a MessageId actually is and then how to fake it
					LatentIt("When calling GetMessage with a valid local user and MessageId, this subsystem will return that message's contents", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
					});

					LatentIt("When calling GetMessage with a valid MessageId but an invalid local user (-1), this subsystem will not return that message's contents", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnEnumerateMessagesCompleteDelegateHandle = OnlineMessage->AddOnEnumerateMessagesCompleteDelegate_Handle(0, FOnEnumerateMessagesCompleteDelegate::CreateLambda([this, TestDone](int32 EnumerateMessagesLocalUserNum, bool bEnumerateMessageWasSuccessful, const FString& EnumerateMessagesErrorStr)
							{
								TestEqual("Verify that bEnumerateMessageWasSuccessful returns as: True", bEnumerateMessageWasSuccessful, true);

								TArray<TSharedRef<FOnlineMessageHeader>> MessageHeaders;
								OnlineMessage->GetMessageHeaders(0, MessageHeaders);

								if (MessageHeaders.Num() > 0)
								{
									FUniqueMessageIdRef MsgId = MessageHeaders[0]->MessageId;

									OnlineMessage->AddOnReadMessageCompleteDelegate_Handle(0, FOnReadMessageCompleteDelegate::CreateLambda([this, MsgId, TestDone](int32 ReadMessageLocalUserNum, bool bReadMessageWasSuccessful, const FUniqueMessageId& ReadMessageMessageId, const FString& ReadMessageErrorStr)
									{
										TestEqual("Verify that bReadMessageWasSuccessful returns as: True", bReadMessageWasSuccessful, true);

										TSharedPtr<FOnlineMessage> ReceivedMessage;
										ReceivedMessage = OnlineMessage->GetMessage(-1, ReadMessageMessageId);

										TestEqual("Verify that ReceivedMessage pointer is invalid", ReceivedMessage.IsValid(), false);
										TestDone.Execute();
									}));

									OnlineMessage->ReadMessage(0, *MsgId);
								}
								else
								{
									UE_LOG_ONLINE(Error, TEXT("OSS Automation: MessageHeaders still unpopulated after a call to OnlineMessage->GetMessageHeaders()"));
									TestDone.Execute();
								}
							}));

							OnlineMessage->EnumerateMessages(0);
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});
				});

				xDescribe("ClearMessage", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						CommonUtils.SendMessageToTestAccount(OnlineIdentity, OnlineFriends, OnlineMessage, SubsystemType, TestDone);
					});

					LatentIt("When calling ClearMessage with a valid local user and MessageId, this subsystem clears that message from the cache", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnEnumerateMessagesCompleteDelegateHandle = OnlineMessage->AddOnEnumerateMessagesCompleteDelegate_Handle(0, FOnEnumerateMessagesCompleteDelegate::CreateLambda([this, TestDone](int32 EnumerateMessagesLocalUserNum, bool bEnumerateMessageWasSuccessful, const FString& EnumerateMessagesErrorStr)
							{
								TestEqual("Verify that bEnumerateMessageWasSuccessful returns as: True", bEnumerateMessageWasSuccessful, true);

								TArray<TSharedRef<FOnlineMessageHeader>> MessageHeaders;
								OnlineMessage->GetMessageHeaders(0, MessageHeaders);

								if (MessageHeaders.Num() > 0)
								{
									FUniqueMessageIdRef MsgId = MessageHeaders[0]->MessageId;

									OnlineMessage->AddOnReadMessageCompleteDelegate_Handle(0, FOnReadMessageCompleteDelegate::CreateLambda([this, MsgId, TestDone](int32 ReadMessageLocalUserNum, bool bReadMessageWasSuccessful, const FUniqueMessageId& ReadMessageMessageId, const FString& ReadMessageErrorStr)
									{
										TestEqual("Verify that bReadMessageWasSuccessful returns as: True", bReadMessageWasSuccessful, true);

										TSharedPtr<FOnlineMessage> ReceivedMessage;
										ReceivedMessage = OnlineMessage->GetMessage(0, ReadMessageMessageId);

										if (ReceivedMessage.IsValid())
										{
											OnlineMessage->ClearMessage(0, ReadMessageMessageId);
											ReceivedMessage = OnlineMessage->GetMessage(0, ReadMessageMessageId);

											TestEqual("Verify that ReceivedMessage pointer is invalid", ReceivedMessage.IsValid(), false);
											TestDone.Execute();
										}
										else
										{
											UE_LOG_ONLINE(Error, TEXT("OSS Automation: IsValid() check on ReceivedMessage failed after a call to OnlineMessage->GetMessage()"));
											TestDone.Execute();
										}
									}));

									OnlineMessage->ReadMessage(0, *MsgId);
								}
								else
								{
									UE_LOG_ONLINE(Error, TEXT("OSS Automation: MessageHeaders still unpopulated after a call to OnlineMessage->GetMessageHeaders()"));
									TestDone.Execute();
								}
							}));

							OnlineMessage->EnumerateMessages(0);
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling ClearMessage with a valid local user but an invalid MessageId, this subsystem does not clear any message from the cache", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnEnumerateMessagesCompleteDelegateHandle = OnlineMessage->AddOnEnumerateMessagesCompleteDelegate_Handle(0, FOnEnumerateMessagesCompleteDelegate::CreateLambda([this, TestDone](int32 EnumerateMessagesLocalUserNum, bool bEnumerateMessageWasSuccessful, const FString& EnumerateMessagesErrorStr)
							{
								TestEqual("Verify that bEnumerateMessageWasSuccessful returns as: True", bEnumerateMessageWasSuccessful, true);

								TArray<TSharedRef<FOnlineMessageHeader>> MessageHeaders;
								OnlineMessage->GetMessageHeaders(0, MessageHeaders);

								if (MessageHeaders.Num() > 0)
								{
									FUniqueMessageIdRef MsgId = MessageHeaders[0]->MessageId;

									OnlineMessage->AddOnReadMessageCompleteDelegate_Handle(0, FOnReadMessageCompleteDelegate::CreateLambda([this, MsgId, TestDone](int32 ReadMessageLocalUserNum, bool bReadMessageWasSuccessful, const FUniqueMessageId& ReadMessageMessageId, const FString& ReadMessageErrorStr)
									{
										TestEqual("Verify that bReadMessageWasSuccessful returns as: True", bReadMessageWasSuccessful, true);

										TSharedPtr<FOnlineMessage> ReceivedMessage;
										ReceivedMessage = OnlineMessage->GetMessage(0, ReadMessageMessageId);

										if (ReceivedMessage.IsValid())
										{
											//Mess up messageID here
											OnlineMessage->ClearMessage(0, ReadMessageMessageId);
											ReceivedMessage = OnlineMessage->GetMessage(0, ReadMessageMessageId);

											TestEqual("Verify that ReceivedMessage pointer is invalid", ReceivedMessage.IsValid(), false);
											TestDone.Execute();
										}
										else
										{
											UE_LOG_ONLINE(Error, TEXT("OSS Automation: IsValid() check on ReceivedMessage failed after a call to OnlineMessage->GetMessage()"));
											TestDone.Execute();
										}
									}));

									OnlineMessage->ReadMessage(0, *MsgId);
								}
								else
								{
									UE_LOG_ONLINE(Error, TEXT("OSS Automation: MessageHeaders still unpopulated after a call to OnlineMessage->GetMessageHeaders()"));
									TestDone.Execute();
								}
							}));

							OnlineMessage->EnumerateMessages(0);
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling ClearMessage with a valid MessageId but an invalid local user (-1), this subsystem does not clear that message from the cache", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnEnumerateMessagesCompleteDelegateHandle = OnlineMessage->AddOnEnumerateMessagesCompleteDelegate_Handle(0, FOnEnumerateMessagesCompleteDelegate::CreateLambda([this, TestDone](int32 EnumerateMessagesLocalUserNum, bool bEnumerateMessageWasSuccessful, const FString& EnumerateMessagesErrorStr)
							{
								TestEqual("Verify that bEnumerateMessageWasSuccessful returns as: True", bEnumerateMessageWasSuccessful, true);

								TArray<TSharedRef<FOnlineMessageHeader>> MessageHeaders;
								OnlineMessage->GetMessageHeaders(0, MessageHeaders);

								if (MessageHeaders.Num() > 0)
								{
									FUniqueMessageIdRef MsgId = MessageHeaders[0]->MessageId;

									OnlineMessage->AddOnReadMessageCompleteDelegate_Handle(0, FOnReadMessageCompleteDelegate::CreateLambda([this, MsgId, TestDone](int32 ReadMessageLocalUserNum, bool bReadMessageWasSuccessful, const FUniqueMessageId& ReadMessageMessageId, const FString& ReadMessageErrorStr)
									{
										TestEqual("Verify that bReadMessageWasSuccessful returns as: True", bReadMessageWasSuccessful, true);

										TSharedPtr<FOnlineMessage> ReceivedMessage;
										ReceivedMessage = OnlineMessage->GetMessage(0, ReadMessageMessageId);

										if (ReceivedMessage.IsValid())
										{
											OnlineMessage->ClearMessage(-1, ReadMessageMessageId);
											ReceivedMessage = OnlineMessage->GetMessage(0, ReadMessageMessageId);

											TestEqual("Verify that ReceivedMessage pointer is valid", ReceivedMessage.IsValid(), true);
											TestDone.Execute();
										}
										else
										{
											UE_LOG_ONLINE(Error, TEXT("OSS Automation: IsValid() check on ReceivedMessage failed after a call to OnlineMessage->GetMessage()"));
											TestDone.Execute();
										}
									}));

									OnlineMessage->ReadMessage(0, *MsgId);
								}
								else
								{
									UE_LOG_ONLINE(Error, TEXT("OSS Automation: MessageHeaders still unpopulated after a call to OnlineMessage->GetMessageHeaders()"));
									TestDone.Execute();
								}
							}));

							OnlineMessage->EnumerateMessages(0);
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});
				});

				xDescribe("ClearMessages", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						CommonUtils.SendMessageToTestAccount(OnlineIdentity, OnlineFriends, OnlineMessage, SubsystemType, TestDone);
					});

					LatentIt("When calling ClearMessages with a valid local user, this subsystem clears all messages from the cache", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnEnumerateMessagesCompleteDelegateHandle = OnlineMessage->AddOnEnumerateMessagesCompleteDelegate_Handle(0, FOnEnumerateMessagesCompleteDelegate::CreateLambda([this, TestDone](int32 EnumerateMessagesLocalUserNum, bool bEnumerateMessageWasSuccessful, const FString& EnumerateMessagesErrorStr)
							{
								TestEqual("Verify that bEnumerateMessageWasSuccessful returns as: True", bEnumerateMessageWasSuccessful, true);

								TArray<TSharedRef<FOnlineMessageHeader>> MessageHeaders;
								OnlineMessage->GetMessageHeaders(0, MessageHeaders);

								if (MessageHeaders.Num() > 0)
								{
									FUniqueMessageIdRef MsgId = MessageHeaders[0]->MessageId;

									OnlineMessage->AddOnReadMessageCompleteDelegate_Handle(0, FOnReadMessageCompleteDelegate::CreateLambda([this, MsgId, TestDone](int32 ReadMessageLocalUserNum, bool bReadMessageWasSuccessful, const FUniqueMessageId& ReadMessageMessageId, const FString& ReadMessageErrorStr)
									{
										TestEqual("Verify that bReadMessageWasSuccessful returns as: True", bReadMessageWasSuccessful, true);

										TSharedPtr<FOnlineMessage> ReceivedMessage;
										ReceivedMessage = OnlineMessage->GetMessage(0, ReadMessageMessageId);

										if (ReceivedMessage.IsValid())
										{
											OnlineMessage->ClearMessages(0);
											ReceivedMessage = OnlineMessage->GetMessage(0, ReadMessageMessageId);

											TestEqual("Verify that ReceivedMessage pointer is invalid", ReceivedMessage.IsValid(), false);
											TestDone.Execute();
										}
										else
										{
											UE_LOG_ONLINE(Error, TEXT("OSS Automation: IsValid() check on ReceivedMessage failed after a call to OnlineMessage->GetMessage()"));
											TestDone.Execute();
										}
									}));

									OnlineMessage->ReadMessage(0, *MsgId);
								}
								else
								{
									UE_LOG_ONLINE(Error, TEXT("OSS Automation: MessageHeaders still unpopulated after a call to OnlineMessage->GetMessageHeaders()"));
									TestDone.Execute();
								}
							}));

							OnlineMessage->EnumerateMessages(0);
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling ClearMessages with an invalid local user (-1), this subsystem does not clear any messages from the cache", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							OnEnumerateMessagesCompleteDelegateHandle = OnlineMessage->AddOnEnumerateMessagesCompleteDelegate_Handle(0, FOnEnumerateMessagesCompleteDelegate::CreateLambda([this, TestDone](int32 EnumerateMessagesLocalUserNum, bool bEnumerateMessageWasSuccessful, const FString& EnumerateMessagesErrorStr)
							{
								TestEqual("Verify that bEnumerateMessageWasSuccessful returns as: True", bEnumerateMessageWasSuccessful, true);

								TArray<TSharedRef<FOnlineMessageHeader>> MessageHeaders;
								OnlineMessage->GetMessageHeaders(0, MessageHeaders);

								if (MessageHeaders.Num() > 0)
								{
									FUniqueMessageIdRef MsgId = MessageHeaders[0]->MessageId;

									OnlineMessage->AddOnReadMessageCompleteDelegate_Handle(0, FOnReadMessageCompleteDelegate::CreateLambda([this, MsgId, TestDone](int32 ReadMessageLocalUserNum, bool bReadMessageWasSuccessful, const FUniqueMessageId& ReadMessageMessageId, const FString& ReadMessageErrorStr)
									{
										TestEqual("Verify that bReadMessageWasSuccessful returns as: True", bReadMessageWasSuccessful, true);

										TSharedPtr<FOnlineMessage> ReceivedMessage;
										ReceivedMessage = OnlineMessage->GetMessage(0, ReadMessageMessageId);

										if (ReceivedMessage.IsValid())
										{
											OnlineMessage->ClearMessages(-1);
											ReceivedMessage = OnlineMessage->GetMessage(0, ReadMessageMessageId);

											TestEqual("Verify that ReceivedMessage pointer is valid", ReceivedMessage.IsValid(), true);
											TestDone.Execute();
										}
										else
										{
											UE_LOG_ONLINE(Error, TEXT("OSS Automation: IsValid() check on ReceivedMessage failed after a call to OnlineMessage->GetMessage()"));
											TestDone.Execute();
										}
									}));

									OnlineMessage->ReadMessage(0, *MsgId);
								}
								else
								{
									UE_LOG_ONLINE(Error, TEXT("OSS Automation: MessageHeaders still unpopulated after a call to OnlineMessage->GetMessageHeaders()"));
									TestDone.Execute();
								}
							}));

							OnlineMessage->EnumerateMessages(0);
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					xDescribe("SendMessage", [this, SubsystemType]()
					{
						LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
						{
						});

						LatentIt("When calling SendMessage with a valid local user, array of RecipientIds, MessageType, and Payload, this subsystem delivers that payload to the RecipientIds", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
						{
							OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNumFriendAccount, bool bLoginWasSuccessfulFriendAccount, const FUniqueNetId& LoginUserIdFriendAccount, const FString& LoginErrorFriendAccount)
							{
								FString TestAccountIdString = FOnlineTestCommon::GetSubsystemTestAccountUniqueId(SubsystemType);
								FUniqueNetIdPtr TestAccountId = OnlineIdentity->CreateUniquePlayerId(TestAccountIdString);

								TArray<FUniqueNetIdRef> Recipients;
								Recipients.Add(TestAccountId.ToSharedRef());

								FOnlineMessagePayload TestPayload;
								TArray<uint8> TestData;
								TestData.Add(0xde);

								TestPayload.SetAttribute(TEXT("STRINGValue"), FVariantData(TestData));

								OnSendMessageCompleteDelegateHandle = OnlineMessage->AddOnSendMessageCompleteDelegate_Handle(0, FOnSendMessageCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 SendMessageLocalUserNum, bool bSendMessageWasSuccessful, const FString& SendMessagePayload)
								{
									TestEqual("Verify that SendMessageLocalUserNum is: 0", SendMessageLocalUserNum == 0, true);
									TestEqual("Verify that bSendMessageWasSuccessful returns as: True", bSendMessageWasSuccessful, true);
									TestEqual("Verify that SendMessagePayload is populated", SendMessagePayload.Len() > 0, true);

									// Log into other account to verify that they received the message
									OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
									OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoggedOutLocalUserNum, bool bLogoutWasSuccessful)
									{
										OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
										OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNumTestAccount, bool bLoginWasSuccessfulTestAccount, const FUniqueNetId& LoginUserIdTestAccount, const FString& LoginErrorTestAccount)
										{
											OnEnumerateMessagesCompleteDelegateHandle = OnlineMessage->AddOnEnumerateMessagesCompleteDelegate_Handle(0, FOnEnumerateMessagesCompleteDelegate::CreateLambda([this, TestDone](int32 EnumerateMessagesLocalUserNum, bool bEnumerateMessageWasSuccessful, const FString& EnumerateMessagesErrorStr)
											{
												TestEqual("Verify that bEnumerateMessageWasSuccessful returns as: True", bEnumerateMessageWasSuccessful, true);

												TArray<TSharedRef<FOnlineMessageHeader>> MessageHeaders;
												OnlineMessage->GetMessageHeaders(0, MessageHeaders);

												if (MessageHeaders.Num() > 0)
												{
													FUniqueMessageIdRef MsgId = MessageHeaders[0]->MessageId;

													OnlineMessage->AddOnReadMessageCompleteDelegate_Handle(0, FOnReadMessageCompleteDelegate::CreateLambda([this, MsgId, TestDone](int32 ReadMessageLocalUserNum, bool bReadMessageWasSuccessful, const FUniqueMessageId& ReadMessageMessageId, const FString& ReadMessageErrorStr)
													{
														TestEqual("Verify that bReadMessageWasSuccessful returns as: True", bReadMessageWasSuccessful, true);

														TSharedPtr<FOnlineMessage> ReceivedMessage;
														ReceivedMessage = OnlineMessage->GetMessage(0, ReadMessageMessageId);

														TestEqual("Verify that ReceivedMessage pointer is valid", ReceivedMessage.IsValid(), true);
													}));

													OnlineMessage->ReadMessage(0, *MsgId);
												}
												else
												{
													UE_LOG_ONLINE(Error, TEXT("OSS Automation: MessageHeaders still unpopulated after a call to OnlineMessage->GetMessageHeaders()"));
													TestDone.Execute();
												}
											}));
										}));

										OnlineIdentity->Login(0, AccountCredentials);
									}));

									OnlineIdentity->Logout(0);
								}));

								OnlineMessage->SendMessage(0, Recipients, TEXT("TEST"), TestPayload);
							}));

							OnlineIdentity->Login(0, FriendAccountCredentials);
						});

						LatentIt("When calling SendMessage with a valid local user, array of RecipientIds, and MessageType but an invalid Payload, this subsystem does not deliver any payload to the RecipientIds", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
						{
							OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNumFriendAccount, bool bLoginWasSuccessfulFriendAccount, const FUniqueNetId& LoginUserIdFriendAccount, const FString& LoginErrorFriendAccount)
							{
								FString TestAccountIdString = FOnlineTestCommon::GetSubsystemTestAccountUniqueId(SubsystemType);
								FUniqueNetIdPtr TestAccountId = OnlineIdentity->CreateUniquePlayerId(TestAccountIdString);

								TArray<FUniqueNetIdRef> Recipients;
								Recipients.Add(TestAccountId.ToSharedRef());

								// TODO: Create invalid payload
								FOnlineMessagePayload TestPayload;
								TArray<uint8> TestData;
								TestData.Add(0xde);

								TestPayload.SetAttribute(TEXT("STRINGValue"), FVariantData(TestData));

								OnSendMessageCompleteDelegateHandle = OnlineMessage->AddOnSendMessageCompleteDelegate_Handle(0, FOnSendMessageCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 SendMessageLocalUserNum, bool bSendMessageWasSuccessful, const FString& SendMessagePayload)
								{
									TestEqual("Verify that SendMessageLocalUserNum is: 0", SendMessageLocalUserNum == 0, true);
									TestEqual("Verify that bSendMessageWasSuccessful returns as: True", bSendMessageWasSuccessful, true);
									TestEqual("Verify that SendMessagePayload is populated", SendMessagePayload.Len() > 0, true);

									// Log into other account to verify that they did not receive the message
									OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
									OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoggedOutLocalUserNum, bool bLogoutWasSuccessful)
									{
										OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
										OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNumTestAccount, bool bLoginWasSuccessfulTestAccount, const FUniqueNetId& LoginUserIdTestAccount, const FString& LoginErrorTestAccount)
										{
											OnEnumerateMessagesCompleteDelegateHandle = OnlineMessage->AddOnEnumerateMessagesCompleteDelegate_Handle(0, FOnEnumerateMessagesCompleteDelegate::CreateLambda([this, TestDone](int32 EnumerateMessagesLocalUserNum, bool bEnumerateMessageWasSuccessful, const FString& EnumerateMessagesErrorStr)
											{
												TestEqual("Verify that bEnumerateMessageWasSuccessful returns as: True", bEnumerateMessageWasSuccessful, true);

												TArray<TSharedRef<FOnlineMessageHeader>> MessageHeaders;
												OnlineMessage->GetMessageHeaders(0, MessageHeaders);

												TestEqual("Verify that MessageHeaders is not populated", MessageHeaders.Num() == 0, true);
											}));
										}));

										OnlineIdentity->Login(0, AccountCredentials);
									}));

									OnlineIdentity->Logout(0);
								}));

								OnlineMessage->SendMessage(0, Recipients, TEXT("TEST"), TestPayload);
							}));

							OnlineIdentity->Login(0, FriendAccountCredentials);
						});

						LatentIt("When calling SendMessage with a valid local user, array of RecipientIds, and Payload but an invalid MessageType, this subsystem does not deliver that payload to the RecipientIds", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
						{
							OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNumFriendAccount, bool bLoginWasSuccessfulFriendAccount, const FUniqueNetId& LoginUserIdFriendAccount, const FString& LoginErrorFriendAccount)
							{
								FString TestAccountIdString = FOnlineTestCommon::GetSubsystemTestAccountUniqueId(SubsystemType);
								FUniqueNetIdPtr TestAccountId = OnlineIdentity->CreateUniquePlayerId(TestAccountIdString);

								TArray<FUniqueNetIdRef> Recipients;
								Recipients.Add(TestAccountId.ToSharedRef());

								FOnlineMessagePayload TestPayload;
								TArray<uint8> TestData;
								TestData.Add(0xde);

								TestPayload.SetAttribute(TEXT("STRINGValue"), FVariantData(TestData));

								OnSendMessageCompleteDelegateHandle = OnlineMessage->AddOnSendMessageCompleteDelegate_Handle(0, FOnSendMessageCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 SendMessageLocalUserNum, bool bSendMessageWasSuccessful, const FString& SendMessagePayload)
								{
									TestEqual("Verify that SendMessageLocalUserNum is: 0", SendMessageLocalUserNum == 0, true);
									TestEqual("Verify that bSendMessageWasSuccessful returns as: True", bSendMessageWasSuccessful, true);
									TestEqual("Verify that SendMessagePayload is populated", SendMessagePayload.Len() > 0, true);

									// Log into other account to verify that they did not receive the message
									OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
									OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoggedOutLocalUserNum, bool bLogoutWasSuccessful)
									{
										OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
										OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNumTestAccount, bool bLoginWasSuccessfulTestAccount, const FUniqueNetId& LoginUserIdTestAccount, const FString& LoginErrorTestAccount)
										{
											OnEnumerateMessagesCompleteDelegateHandle = OnlineMessage->AddOnEnumerateMessagesCompleteDelegate_Handle(0, FOnEnumerateMessagesCompleteDelegate::CreateLambda([this, TestDone](int32 EnumerateMessagesLocalUserNum, bool bEnumerateMessageWasSuccessful, const FString& EnumerateMessagesErrorStr)
											{
												TestEqual("Verify that bEnumerateMessageWasSuccessful returns as: True", bEnumerateMessageWasSuccessful, true);

												TArray<TSharedRef<FOnlineMessageHeader>> MessageHeaders;
												OnlineMessage->GetMessageHeaders(0, MessageHeaders);

												TestEqual("Verify that MessageHeaders is not populated", MessageHeaders.Num() == 0, true);
											}));
										}));

										OnlineIdentity->Login(0, AccountCredentials);
									}));

									OnlineIdentity->Logout(0);
								}));

								//TODO: Create invalid message type
								OnlineMessage->SendMessage(0, Recipients, TEXT("TEST"), TestPayload);
							}));

							OnlineIdentity->Login(0, FriendAccountCredentials);
						});

						LatentIt("When calling SendMessage with a valid local user, MessageType, and Payload but an invalid array of RecipientIds, this subsystem does not deliver that payload to the RecipientIds", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
						{
							OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNumFriendAccount, bool bLoginWasSuccessfulFriendAccount, const FUniqueNetId& LoginUserIdFriendAccount, const FString& LoginErrorFriendAccount)
							{
								FUniqueNetIdPtr TestAccountId = OnlineIdentity->CreateUniquePlayerId(TEXT("0123456789"));

								TArray<FUniqueNetIdRef> Recipients;
								Recipients.Add(TestAccountId.ToSharedRef());

								FOnlineMessagePayload TestPayload;
								TArray<uint8> TestData;
								TestData.Add(0xde);

								TestPayload.SetAttribute(TEXT("STRINGValue"), FVariantData(TestData));

								OnSendMessageCompleteDelegateHandle = OnlineMessage->AddOnSendMessageCompleteDelegate_Handle(0, FOnSendMessageCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 SendMessageLocalUserNum, bool bSendMessageWasSuccessful, const FString& SendMessagePayload)
								{
									TestEqual("Verify that SendMessageLocalUserNum is: 0", SendMessageLocalUserNum == 0, true);
									TestEqual("Verify that bSendMessageWasSuccessful returns as: True", bSendMessageWasSuccessful, true);
									TestEqual("Verify that SendMessagePayload is populated", SendMessagePayload.Len() > 0, true);

									// Log into other account to verify that they did not receive the message
									OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
									OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoggedOutLocalUserNum, bool bLogoutWasSuccessful)
									{
										OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
										OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNumTestAccount, bool bLoginWasSuccessfulTestAccount, const FUniqueNetId& LoginUserIdTestAccount, const FString& LoginErrorTestAccount)
										{
											OnEnumerateMessagesCompleteDelegateHandle = OnlineMessage->AddOnEnumerateMessagesCompleteDelegate_Handle(0, FOnEnumerateMessagesCompleteDelegate::CreateLambda([this, TestDone](int32 EnumerateMessagesLocalUserNum, bool bEnumerateMessageWasSuccessful, const FString& EnumerateMessagesErrorStr)
											{
												TestEqual("Verify that bEnumerateMessageWasSuccessful returns as: True", bEnumerateMessageWasSuccessful, true);

												TArray<TSharedRef<FOnlineMessageHeader>> MessageHeaders;
												OnlineMessage->GetMessageHeaders(0, MessageHeaders);

												TestEqual("Verify that MessageHeaders is not populated", MessageHeaders.Num() == 0, true);
											}));
										}));

										OnlineIdentity->Login(0, AccountCredentials);
									}));

									OnlineIdentity->Logout(0);
								}));

								OnlineMessage->SendMessage(0, Recipients, TEXT("TEST"), TestPayload);
							}));

							OnlineIdentity->Login(0, FriendAccountCredentials);
						});

						LatentIt("When calling SendMessage with a valid array of RecipientIds, MessageType, and Payload but an invalid local user (-1), this subsystem does not deliver that payload to the RecipientIds", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
						{
							OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNumFriendAccount, bool bLoginWasSuccessfulFriendAccount, const FUniqueNetId& LoginUserIdFriendAccount, const FString& LoginErrorFriendAccount)
							{
								FString TestAccountIdString = FOnlineTestCommon::GetSubsystemTestAccountUniqueId(SubsystemType);
								FUniqueNetIdPtr TestAccountId = OnlineIdentity->CreateUniquePlayerId(TestAccountIdString);

								TArray<FUniqueNetIdRef> Recipients;
								Recipients.Add(TestAccountId.ToSharedRef());

								FOnlineMessagePayload TestPayload;
								TArray<uint8> TestData;
								TestData.Add(0xde);

								TestPayload.SetAttribute(TEXT("STRINGValue"), FVariantData(TestData));

								OnSendMessageCompleteDelegateHandle = OnlineMessage->AddOnSendMessageCompleteDelegate_Handle(0, FOnSendMessageCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 SendMessageLocalUserNum, bool bSendMessageWasSuccessful, const FString& SendMessagePayload)
								{
									TestEqual("Verify that SendMessageLocalUserNum is: 0", SendMessageLocalUserNum == 0, true);
									TestEqual("Verify that bSendMessageWasSuccessful returns as: True", bSendMessageWasSuccessful, true);
									TestEqual("Verify that SendMessagePayload is populated", SendMessagePayload.Len() > 0, true);

									// Log into other account to verify that they did not receive the message
									OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
									OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoggedOutLocalUserNum, bool bLogoutWasSuccessful)
									{
										OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
										OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNumTestAccount, bool bLoginWasSuccessfulTestAccount, const FUniqueNetId& LoginUserIdTestAccount, const FString& LoginErrorTestAccount)
										{
											OnEnumerateMessagesCompleteDelegateHandle = OnlineMessage->AddOnEnumerateMessagesCompleteDelegate_Handle(0, FOnEnumerateMessagesCompleteDelegate::CreateLambda([this, TestDone](int32 EnumerateMessagesLocalUserNum, bool bEnumerateMessageWasSuccessful, const FString& EnumerateMessagesErrorStr)
											{
												TestEqual("Verify that bEnumerateMessageWasSuccessful returns as: True", bEnumerateMessageWasSuccessful, true);

												TArray<TSharedRef<FOnlineMessageHeader>> MessageHeaders;
												OnlineMessage->GetMessageHeaders(0, MessageHeaders);

												TestEqual("Verify that MessageHeaders is not populated", MessageHeaders.Num() == 0, true);
											}));
										}));

										OnlineIdentity->Login(0, AccountCredentials);
									}));

									OnlineIdentity->Logout(0);
								}));

								OnlineMessage->SendMessage(-1, Recipients, TEXT("TEST"), TestPayload);
							}));

							OnlineIdentity->Login(0, FriendAccountCredentials);
						});
					});

					xDescribe("DeleteMessage", [this, SubsystemType]()
					{
						LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& TestDone)
						{
							CommonUtils.SendMessageToTestAccount(OnlineIdentity, OnlineFriends, OnlineMessage, SubsystemType, TestDone);
						});

						LatentIt("When calling DeleteMessage with a valid local user and MessageId, this subsystem deletes that message", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
						{
							OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
							{
								OnEnumerateMessagesCompleteDelegateHandle = OnlineMessage->AddOnEnumerateMessagesCompleteDelegate_Handle(0, FOnEnumerateMessagesCompleteDelegate::CreateLambda([this, TestDone](int32 EnumerateMessagesLocalUserNum, bool bEnumerateMessageWasSuccessful, const FString& EnumerateMessagesErrorStr)
								{
									TestEqual("Verify that bEnumerateMessageWasSuccessful returns as: True", bEnumerateMessageWasSuccessful, true);

									TArray<TSharedRef<FOnlineMessageHeader>> MessageHeaders;
									OnlineMessage->GetMessageHeaders(0, MessageHeaders);

									if (MessageHeaders.Num() > 0)
									{
										FUniqueMessageIdRef MsgId = MessageHeaders[0]->MessageId;

										OnlineMessage->ClearMessageHeaders(0);

										OnlineMessage->AddOnDeleteMessageCompleteDelegate_Handle(0, FOnDeleteMessageCompleteDelegate::CreateLambda([this, &MessageHeaders, TestDone](int32 DeleteMessageLocalUserNum, bool bDeleteMessageWasSuccessful, const FUniqueMessageId& DeleteMessageMessageId, const FString& DeleteMessageErrorStr)
										{
											OnlineMessage->ClearOnEnumerateMessagesCompleteDelegate_Handle(0, OnEnumerateMessagesCompleteDelegateHandle);
											OnEnumerateMessagesCompleteDelegateHandle = OnlineMessage->AddOnEnumerateMessagesCompleteDelegate_Handle(0, FOnEnumerateMessagesCompleteDelegate::CreateLambda([this, &MessageHeaders, TestDone](int32 SecondEnumerateMessagesLocalUserNum, bool bSecondEnumerateMessageWasSuccessful, const FString& SecondEnumerateMessagesErrorStr)
											{
												TestEqual("Verify that bEnumerateMessageWasSuccessful returns as: True", bSecondEnumerateMessageWasSuccessful, true);

												MessageHeaders.Empty();
												OnlineMessage->GetMessageHeaders(0, MessageHeaders);

												TestEqual("Verify that MessageHeaders is not populated", MessageHeaders.Num() == 0, true);
											}));

											OnlineMessage->EnumerateMessages(0);
										}));

										OnlineMessage->DeleteMessage(0, *MsgId);
									}
									else
									{
										UE_LOG_ONLINE(Error, TEXT("OSS Automation: MessageHeaders still unpopulated after a call to OnlineMessage->GetMessageHeaders()"));
										TestDone.Execute();
									}
								}));

								OnlineMessage->EnumerateMessages(0);
							}));

							OnlineIdentity->Login(0, AccountCredentials);
						});

						LatentIt("When calling DeleteMessage with a valid local user but an invalid MessageId, this subsystem does not delete any message", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
						{
							OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
							{
								OnEnumerateMessagesCompleteDelegateHandle = OnlineMessage->AddOnEnumerateMessagesCompleteDelegate_Handle(0, FOnEnumerateMessagesCompleteDelegate::CreateLambda([this, TestDone](int32 EnumerateMessagesLocalUserNum, bool bEnumerateMessageWasSuccessful, const FString& EnumerateMessagesErrorStr)
								{
									TestEqual("Verify that bEnumerateMessageWasSuccessful returns as: True", bEnumerateMessageWasSuccessful, true);

									TArray<TSharedRef<FOnlineMessageHeader>> MessageHeaders;
									OnlineMessage->GetMessageHeaders(0, MessageHeaders);

									if (MessageHeaders.Num() > 0)
									{
										//TODO: Figure out how to mess with message ID
										FUniqueMessageIdRef MsgId = MessageHeaders[0]->MessageId;

										OnlineMessage->ClearMessageHeaders(0);

										OnlineMessage->AddOnDeleteMessageCompleteDelegate_Handle(0, FOnDeleteMessageCompleteDelegate::CreateLambda([this, &MessageHeaders, TestDone](int32 DeleteMessageLocalUserNum, bool bDeleteMessageWasSuccessful, const FUniqueMessageId& DeleteMessageMessageId, const FString& DeleteMessageErrorStr)
										{
											OnlineMessage->ClearOnEnumerateMessagesCompleteDelegate_Handle(0, OnEnumerateMessagesCompleteDelegateHandle);
											OnEnumerateMessagesCompleteDelegateHandle = OnlineMessage->AddOnEnumerateMessagesCompleteDelegate_Handle(0, FOnEnumerateMessagesCompleteDelegate::CreateLambda([this, &MessageHeaders, TestDone](int32 SecondEnumerateMessagesLocalUserNum, bool bSecondEnumerateMessageWasSuccessful, const FString& SecondEnumerateMessagesErrorStr)
											{
												TestEqual("Verify that bEnumerateMessageWasSuccessful returns as: True", bSecondEnumerateMessageWasSuccessful, true);

												MessageHeaders.Empty();
												OnlineMessage->GetMessageHeaders(0, MessageHeaders);

												TestEqual("Verify that MessageHeaders is populated", MessageHeaders.Num() > 0, true);
											}));

											OnlineMessage->EnumerateMessages(0);
										}));

										OnlineMessage->DeleteMessage(0, *MsgId);
									}
									else
									{
										UE_LOG_ONLINE(Error, TEXT("OSS Automation: MessageHeaders still unpopulated after a call to OnlineMessage->GetMessageHeaders()"));
										TestDone.Execute();
									}
								}));

								OnlineMessage->EnumerateMessages(0);
							}));

							OnlineIdentity->Login(0, AccountCredentials);
						});

						LatentIt("When calling DeleteMessage with a valid MessageId but an invalid local user (-1), this subsystem does not delete that message", EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate TestDone)
						{
							OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, SubsystemType, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
							{
								OnEnumerateMessagesCompleteDelegateHandle = OnlineMessage->AddOnEnumerateMessagesCompleteDelegate_Handle(0, FOnEnumerateMessagesCompleteDelegate::CreateLambda([this, TestDone](int32 EnumerateMessagesLocalUserNum, bool bEnumerateMessageWasSuccessful, const FString& EnumerateMessagesErrorStr)
								{
									TestEqual("Verify that bEnumerateMessageWasSuccessful returns as: True", bEnumerateMessageWasSuccessful, true);

									TArray<TSharedRef<FOnlineMessageHeader>> MessageHeaders;
									OnlineMessage->GetMessageHeaders(0, MessageHeaders);

									if (MessageHeaders.Num() > 0)
									{
										FUniqueMessageIdRef MsgId = MessageHeaders[0]->MessageId;

										OnlineMessage->ClearMessageHeaders(0);

										OnlineMessage->AddOnDeleteMessageCompleteDelegate_Handle(0, FOnDeleteMessageCompleteDelegate::CreateLambda([this, &MessageHeaders, TestDone](int32 DeleteMessageLocalUserNum, bool bDeleteMessageWasSuccessful, const FUniqueMessageId& DeleteMessageMessageId, const FString& DeleteMessageErrorStr)
										{
											OnlineMessage->ClearOnEnumerateMessagesCompleteDelegate_Handle(0, OnEnumerateMessagesCompleteDelegateHandle);
											OnEnumerateMessagesCompleteDelegateHandle = OnlineMessage->AddOnEnumerateMessagesCompleteDelegate_Handle(0, FOnEnumerateMessagesCompleteDelegate::CreateLambda([this, &MessageHeaders, TestDone](int32 SecondEnumerateMessagesLocalUserNum, bool bSecondEnumerateMessageWasSuccessful, const FString& SecondEnumerateMessagesErrorStr)
											{
												TestEqual("Verify that bEnumerateMessageWasSuccessful returns as: True", bSecondEnumerateMessageWasSuccessful, true);

												MessageHeaders.Empty();
												OnlineMessage->GetMessageHeaders(0, MessageHeaders);

												TestEqual("Verify that MessageHeaders is populated", MessageHeaders.Num() > 0, true);
											}));

											OnlineMessage->EnumerateMessages(0);
										}));

										OnlineMessage->DeleteMessage(-1, *MsgId);
									}
									else
									{
										UE_LOG_ONLINE(Error, TEXT("OSS Automation: MessageHeaders still unpopulated after a call to OnlineMessage->GetMessageHeaders()"));
										TestDone.Execute();
									}
								}));

								OnlineMessage->EnumerateMessages(0);
							}));

							OnlineIdentity->Login(0, AccountCredentials);
						});
					});
				});
			});

			AfterEach(EAsyncExecution::ThreadPool, [this]()
			{
				// Clean up Identity
				if (OnlineIdentity.IsValid())
				{
					if (OnlineIdentity->GetLoginStatus(0) == ELoginStatus::LoggedIn)
					{
						OnlineIdentity->Logout(0);
					}

					OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
					OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
					OnlineIdentity = nullptr;
				}

				// Clean up Friends
				if (OnlineFriends.IsValid())
				{
					OnlineFriends = nullptr;
				}

				// Clean up Message
				if (OnlineMessage.IsValid())
				{
					OnlineMessage->ClearOnEnumerateMessagesCompleteDelegate_Handle(0, OnEnumerateMessagesCompleteDelegateHandle);
					OnlineMessage->ClearOnSendMessageCompleteDelegate_Handle(0, OnSendMessageCompleteDelegateHandle);
					OnlineMessage->ClearOnDeleteMessageCompleteDelegate_Handle(0, OnDeleteMessageCompleteDelegateHandle);
					OnlineMessage = nullptr;
				}
			});
		});
	}
}
