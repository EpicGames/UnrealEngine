// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online.h"
#include "Interfaces/OnlinePresenceInterface.h"
#include "Utils/OnlineTestCommon.h"

BEGIN_DEFINE_SPEC(FOnlinePresenceSpec, "OnlinePresenceInterface", EAutomationTestFlags::EngineFilter | EAutomationTestFlags_ApplicationContextMask)

IOnlineIdentityPtr OnlineIdentity;
IOnlineFriendsPtr OnlineFriends;
IOnlinePresencePtr OnlinePresence;

FOnlineAccountCredentials AccountCredentials;
FOnlineAccountCredentials FriendAccountCredentials;

FOnlineTestCommon CommonUtils;

const IOnlinePresence::FOnPresenceTaskCompleteDelegate PresenceCompleteDelegate;

// Delegate Handles
FDelegateHandle OnLogoutCompleteDelegateHandle;
FDelegateHandle	OnLoginCompleteDelegateHandle;

END_DEFINE_SPEC(FOnlinePresenceSpec)

void FOnlinePresenceSpec::Define()
{
	TArray<FName> Subsystems = FOnlineTestCommon::GetEnabledTestSubsystems();

	for (int32 Index = 0; Index < Subsystems.Num(); Index++)
	{
		FName SubsystemType = Subsystems[Index];

		Describe(SubsystemType.ToString(), [this, SubsystemType]()
		{
			BeforeEach([this, SubsystemType]()
			{
				CommonUtils = FOnlineTestCommon();
				AccountCredentials = FOnlineTestCommon::GetSubsystemTestAccountCredentials(SubsystemType);
				FriendAccountCredentials = FOnlineTestCommon::GetSubsystemFriendAccountCredentials(SubsystemType);

				OnlineIdentity = Online::GetIdentityInterface(SubsystemType);
				OnlineFriends = Online::GetFriendsInterface(SubsystemType);
				OnlinePresence = Online::GetPresenceInterface(SubsystemType);

				// If OnlineIdentity, OnlineFriends, or OnlinePresence is not valid, the following test, including all other nested BeforeEaches, will not run
				if (!OnlineIdentity.IsValid())
				{
					UE_LOG_ONLINE_PRESENCE(Error, TEXT("OSS Automation: Failed to load OnlineIdentity Interface for %s"), *SubsystemType.ToString());
				}
				if (!OnlinePresence.IsValid())
				{
					UE_LOG_ONLINE_PRESENCE(Error, TEXT("OSS Automation: Failed to load OnlinePresence Interface for %s"), *SubsystemType.ToString());
				}

			});

			Describe("Online Presence", [this, SubsystemType]()
			{
				Describe("SetPresence", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& BeforeEachDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, BeforeEachDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							BeforeEachDone.Execute();
						}));

						OnlineIdentity->Login(0, AccountCredentials);	
					});

					LatentIt("When calling SetPresence with a valid FUniqueNetId and a status that is different from the user's current status, this subsystem correctly changes the user's cached presence", [this, SubsystemType](const FDoneDelegate& TestDone)
					{											
						FUniqueNetIdPtr CurrentUser = OnlineIdentity->GetUniquePlayerId(0);

						if (CurrentUser->IsValid())
						{
							FOnlineUserPresenceStatus NewPresenceStatus = FOnlineUserPresenceStatus();
							FVariantData Val1;

							Val1.SetValue(TEXT("PresenceTestString"));
							NewPresenceStatus.StatusStr = TEXT("Testing");
							NewPresenceStatus.Properties.Add(DefaultPlatformKey, Val1);

							OnlinePresence->SetPresence(*CurrentUser, NewPresenceStatus, IOnlinePresence::FOnPresenceTaskCompleteDelegate::CreateLambda([this, TestDone, NewPresenceStatus](const FUniqueNetId& PresLoginUserId, const bool bWasSuccessful)
							{
								TestEqual("Verify that bWasSuccessful returns as: True", bWasSuccessful, true);

								TSharedPtr<FOnlineUserPresence> PostCachedPresence;
								OnlinePresence->GetCachedPresence(PresLoginUserId, PostCachedPresence);

								if (PostCachedPresence.IsValid())
								{
									TestEqual("Verify that login Status.StatusStr is set to : Testing", PostCachedPresence->Status.StatusStr, TEXT("Testing"));
									TestTrue("The number of keys in the tested presence are the same or more than the cached", NewPresenceStatus.Properties.Num() <= PostCachedPresence->Status.Properties.Num());

									FString TestValue;
									for (FPresenceProperties::TConstIterator Itr(NewPresenceStatus.Properties); Itr; ++Itr)
									{
										const FString& Key = Itr.Key();
										const FString& Value = Itr.Value().ToString();

										if (PostCachedPresence->Status.Properties.Find(Key) == nullptr)
										{
											UE_LOG_ONLINE_PRESENCE(Error, TEXT("Presence test fails, missing key %s"), *Key);
											TestDone.Execute();
										}

										PostCachedPresence->Status.Properties.Find(Key)->GetValue(TestValue);
										if (Value != TestValue)
										{
											UE_LOG_ONLINE_PRESENCE(Error, TEXT("Presence test fails, key %s has different values. Cached=%s Has=%s"), *Key, *Value, *TestValue);
											TestDone.Execute();
										}
									}
								}
								else
								{
									UE_LOG_ONLINE_PRESENCE(Error, TEXT("OSS Automation: IsValid() check on PostCachedPresence failed after a call to OnlinePresence->GetCachedPresence()"))
								}
					
								TestDone.Execute();
							}));
						}
						else
						{
							UE_LOG_ONLINE_PRESENCE(Error, TEXT("OSS Automation: IsValid() check on CurrentUser failed on OnlinePresence->GetUniquePlayerId()"));
						}
					});

					LatentAfterEach([this](const FDoneDelegate& AfterEachDone)
					{
						OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, AfterEachDone](int32 LogoutLocalUserNum, bool bLogoutWasSuccessful)
						{
							AfterEachDone.Execute();
						}));

						OnlineIdentity->Logout(0);
					});
				});

				Describe("QueryPresence", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& BeforeEachDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, BeforeEachDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							BeforeEachDone.Execute();
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling QueryPresence with a valid FUniqueNetId, this subsystem will get the cached presence status of that user", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						FUniqueNetIdPtr CurrentUser = OnlineIdentity->GetUniquePlayerId(0);

						if (CurrentUser.IsValid())
						{
							OnlinePresence->QueryPresence(*CurrentUser, IOnlinePresence::FOnPresenceTaskCompleteDelegate::CreateLambda([this, CurrentUser, TestDone](const FUniqueNetId& QueryPresenceUniqueId, bool bQueryPresenceWasSuccessful)
							{
								TestEqual("Verify that bQueryPresenceSuccess returns as: True", bQueryPresenceWasSuccessful, true);
								TestEqual("Verify that QueryPresenceUniqueId is the Id that was originally used", QueryPresenceUniqueId.ToString() == *CurrentUser->ToString(), true);
								TestDone.Execute();
							}));
						}
						else
						{
							UE_LOG_ONLINE_PRESENCE(Error, TEXT("OSS Automation: IsValid() check on CurrentUser failed after a call to OnlineIdentity->GetUniquePlayerId()"));
							TestDone.Execute();
						}
					});

					LatentAfterEach([this](const FDoneDelegate& AfterEachDone)
					{
						OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, AfterEachDone](int32 LogoutLocalUserNum, bool bLogoutWasSuccessful)
						{
							AfterEachDone.Execute();
						}));

						OnlineIdentity->Logout(0);
					});
				});
				
				Describe("GetCachedPresence", [this, SubsystemType]() 
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& BeforeEachDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, BeforeEachDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							BeforeEachDone.Execute();
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});

					LatentIt("When calling GetCachedPresence with a valid FUniqueNetId after polling for the local user's presence data, this subsystem will return that user's presence", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						FUniqueNetIdPtr CurrentUser = OnlineIdentity->GetUniquePlayerId(0);

						OnlinePresence->QueryPresence(*CurrentUser, IOnlinePresence::FOnPresenceTaskCompleteDelegate::CreateLambda([this, CurrentUser, TestDone](const FUniqueNetId& QueryPresenceUniqueId, bool bQueryPresenceWasSuccessful)
						{
							TestEqual("Verify that bQueryPresenceSuccess returns as: True", bQueryPresenceWasSuccessful, true);

							TSharedPtr<FOnlineUserPresence> CachedPresence;
							OnlinePresence->GetCachedPresence(*CurrentUser, CachedPresence);

							if (CachedPresence.IsValid())
							{
								TestEqual("Verify that Status.State is: Online", CachedPresence->Status.State, EOnlinePresenceState::Online);
								TestDone.Execute();
							}
							else
							{
								UE_LOG_ONLINE_PRESENCE(Error, TEXT("OSS Automation: IsValid() check on CachedPresence failed after a call to OnlinePresence->GetCachedPresence()"));
								TestDone.Execute();
							}
						}));

					});

					LatentIt("When calling GetCachedPresence with an invalid FUniqueNetId, this subsystem will not return a presence status", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						FString InvalidUserIdString = TEXT(" ");
						FUniqueNetIdPtr InvalidUserId = OnlineIdentity->CreateUniquePlayerId(InvalidUserIdString);

						if (InvalidUserId.IsValid())
						{
							TSharedPtr<FOnlineUserPresence> CachedPresence;
							OnlinePresence->GetCachedPresence(*InvalidUserId, CachedPresence);

							TestEqual("Verify that CachedPresence pointer is: Invalid", CachedPresence.IsValid(), false);

							TestDone.Execute();
						}
						else
						{
							UE_LOG_ONLINE_PRESENCE(Error, TEXT("OSS Automation: IsValid() check on InvalidUserId failed after a call to OnlineIdentity->CreateUniquePlayerId()"));
							TestDone.Execute();
						}
					});

					LatentAfterEach([this](const FDoneDelegate& AfterEachDone)
					{
						OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, AfterEachDone](int32 LogoutLocalUserNum, bool bLogoutWasSuccessful)
						{
							AfterEachDone.Execute();
						}));

						OnlineIdentity->Logout(0);
					});
				});

				Describe("GetCachedPresenceForApp", [this, SubsystemType]()
				{
					LatentBeforeEach(EAsyncExecution::ThreadPool, [this, SubsystemType](const FDoneDelegate& BeforeEachDone)
					{
						OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, BeforeEachDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
						{
							BeforeEachDone.Execute();
						}));

						OnlineIdentity->Login(0, AccountCredentials);
					});
					LatentIt("When calling GetCachedPresenceForApp with an valid FUniqueNetId, this subsystem will return a presence status of EOnlineCachedResult::Type::NotFound", [this, SubsystemType](const FDoneDelegate& TestDone)
					{
						FUniqueNetIdPtr CurrentUser = OnlineIdentity->GetUniquePlayerId(0);
						const FString& TestAppId = TEXT("TestAppId");
						TSharedPtr<FOnlineUserPresence> OutPresence;

						EOnlineCachedResult::Type Result = OnlinePresence->GetCachedPresenceForApp(*CurrentUser, *CurrentUser, TestAppId, OutPresence);
						TestEqual("Verify GetCachedPresenceForApp returns EOnlineCachedResult::NotFound", Result, EOnlineCachedResult::NotFound);
						TestEqual("Verify that OutPresence is null", OutPresence.IsValid(), false);
						TestDone.Execute();
					});
					LatentAfterEach([this](const FDoneDelegate& AfterEachDone)
					{
						OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, AfterEachDone](int32 LogoutLocalUserNum, bool bLogoutWasSuccessful)
						{
							AfterEachDone.Execute();
						}));

						OnlineIdentity->Logout(0);
					});
				});
			});

			AfterEach([this]()
			{
				// Clean up Identity
				if (OnlineIdentity.IsValid())
				{
					if (OnlineIdentity->GetLoginStatus(0) == ELoginStatus::LoggedIn)
					{
						OnlineIdentity->Logout(0);
					}

					OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
					OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
					OnlineIdentity = nullptr;
				}

				// Clean up Friends
				if (OnlineFriends.IsValid())
				{
					OnlineFriends = nullptr;
				}

				// Clean up Presence
				if (OnlinePresence.IsValid())
				{
					OnlinePresence = nullptr;
				}
			});
		});
	}
}
