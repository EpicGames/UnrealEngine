// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef ONLINE_EXPECTEDERROR_INVALID_ACCOUNTCREDENTIALS
#define ONLINE_EXPECTEDERROR_INVALID_ACCOUNTCREDENTIALS TEXT("errors.com.epicgames.account.invalid_account_credentials")
#endif

#ifndef ONLINE_EXPECTEDERROR_INVALID_AUTHTYPE
#define ONLINE_EXPECTEDERROR_INVALID_AUTHTYPE TEXT("errors.com.epicgames.account.ext_auth.unknown_external_auth_type")
#endif

#ifndef ONLINE_EXPECTEDERROR_AUTOLOGIN_FAILED
#define ONLINE_EXPECTEDERROR_AUTOLOGIN_FAILED TEXT("errors.com.epicgames.account.autologin_failed")
#endif

#ifndef ONLINE_EXPECTEDERROR_INVALID_NAME
#define ONLINE_EXPECTEDERROR_INVALID_NAME TEXT("errors.com.epicgames.users.invalid_name")
#endif

#ifndef ONLINE_EXPECTEDERROR_MISSINGUSER
#define ONLINE_EXPECTEDERROR_MISSINGUSER TEXT("errors.com.epicgames.users.missing_user")
#endif

#ifndef ONLINE_EXPECTEDERROR_INVALID_LOCALUSER
#define ONLINE_EXPECTEDERROR_INVALID_LOCALUSER TEXT("errors.com.epicgames.localuser.invalid_localusernum")
#endif

#ifndef ONLINE_EXPECTEDERROR_LOCALUSER_NOTLOGGEDIN
#define ONLINE_EXPECTEDERROR_LOCALUSER_NOTLOGGEDIN TEXT("errors.com.epicgames.localuser.localuser_not_logged_in")
#endif

#ifndef ONLINE_EXPECTEDERROR_ALREADYFRIENDS
#define ONLINE_EXPECTEDERROR_ALREADYFRIENDS TEXT("errors.com.epicgames.account.friendship_already_exists")
#endif

#ifndef ONLINE_EXPECTEDERROR_ACCOUNT_DOESNOTEXIST
#define ONLINE_EXPECTEDERROR_ACCOUNT_DOESNOTEXIST TEXT("errors.com.epicgames.account.account_does_not_exist")
#endif

#ifndef ONLINE_EXPECTEDERROR_NOCACHEDFRIEND
#define ONLINE_EXPECTEDERROR_NOCACHEDFRIEND TEXT("errors.com.epicgames.account.no_cached_friend")
#endif

#ifndef ONLINE_EXPECTEDERROR_INVALID_USERID
#define ONLINE_EXPECTEDERROR_INVALID_USERID TEXT("errors.com.epicgames.users.invalid_userid")
#endif

#ifndef ONLINE_EXPECTEDERROR_NONLOCALPLAYER
#define ONLINE_EXPECTEDERROR_NONLOCALPLAYER TEXT("errors.com.epicgames.users.non_local_player")
#endif

#ifndef ONLINE_EXPECTEDERROR_INVALID_ACHIEVEMENT
#define ONLINE_EXPECTEDERROR_INVALID_ACHIEVEMENT TEXT("errors.com.epicgames.achievements.invalid_achievement")
#endif