// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineTestCommon.h"
#include "Interfaces/OnlineAchievementsInterface.h"
#include "Misc/ConfigCacheIni.h"
#include "Interfaces/OnlineMessageInterface.h"

FOnlineTestCommon::FOnlineTestCommon()
{

}

void FOnlineTestCommon::Cleanup()
{
	if (OnlineIdentity.IsValid())
	{
		OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
		OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
		OnlineIdentity = nullptr;
	}

	if (OnlineFriends.IsValid())
	{
		OnlineFriends->ClearOnBlockedPlayerCompleteDelegate_Handle(0, OnBlockedPlayerCompleteDelegateHandle);
		OnlineFriends->ClearOnUnblockedPlayerCompleteDelegate_Handle(0, OnUnblockedPlayerCompleteDelegateHandle);
		OnlineFriends->ClearOnDeleteFriendCompleteDelegate_Handle(0, OnDeleteFriendCompleteDelegateHandle);
		OnlineFriends->ClearOnRejectInviteCompleteDelegate_Handle(0, OnRejectInviteCompleteDelegateHandle);
		OnlineFriends = nullptr;
	}

	if (OnlineMessage.IsValid())
	{
		OnlineMessage->ClearOnSendMessageCompleteDelegate_Handle(0, OnSendMessageCompleteDelegateHandle);
		OnlineMessage = nullptr;
	}

	if (OnlineAchievements.IsValid())
	{
		OnlineAchievements = nullptr;
	}

	SubsystemType = TEXT("");

	AccountCredentials = FOnlineAccountCredentials(TEXT(""), TEXT(""), TEXT(""));
}

TArray<FName> FOnlineTestCommon::GetEnabledTestSubsystems()
{
	TArray<FName> EnabledTestSubsystems;
	TArray<FString> SubSystemsFromIni;

	GConfig->GetArray(TEXT("OnlineSubsystemAutomation"), TEXT("EnabledTestSubsystem"), SubSystemsFromIni, GEngineIni);
	
	for (auto It : SubSystemsFromIni)
	{
		EnabledTestSubsystems.Add(FName(*It));
	}

	return EnabledTestSubsystems;
}

FOnlineAccountCredentials FOnlineTestCommon::GetSubsystemCredentials(FName Subsystem, FString ConfigPrefix)
{
	FString TestAccountIniEntry = ConfigPrefix.Append(Subsystem.ToString());
	FString OutTestCredentialsFromIni;

	GConfig->GetString(TEXT("OnlineSubsystemAutomation"), *TestAccountIniEntry, OutTestCredentialsFromIni, GEngineIni);

	if (!OutTestCredentialsFromIni.IsEmpty())
	{
		FString InType, InId, InToken;
		FString OutFirstRemainder;
		FString OutSecondRemainder;
		FString OutUniqueUserId;	

		if (OutTestCredentialsFromIni.Split(TEXT(":"), &InId, &OutFirstRemainder))
		{
			if (OutFirstRemainder.Split(TEXT(":"), &InToken, &OutSecondRemainder))
			{
				if (!OutSecondRemainder.Split(TEXT(":"), &InType, &OutUniqueUserId) && !OutSecondRemainder.IsEmpty())
				{
					InType = OutSecondRemainder;
				}
			}
		}

		return FOnlineAccountCredentials(InType, InId, InToken);
	}

	return FOnlineAccountCredentials();
}

FOnlineAccountCredentials FOnlineTestCommon::GetSubsystemTestAccountCredentials(FName Subsystem)
{
	return GetSubsystemCredentials(Subsystem, TEXT("TestAccountCredentials"));
}

FOnlineAccountCredentials FOnlineTestCommon::GetSubsystemFriendAccountCredentials(FName Subsystem)
{
	return GetSubsystemCredentials(Subsystem, TEXT("FriendAccountCredentials"));
}

FString FOnlineTestCommon::GetSubsystemUniqueId(FName Subsystem, FString ConfigPrefix)
{
	FString OutUniqueUserId;

	FString TestAccountIniEntry = ConfigPrefix.Append(Subsystem.ToString());
	FString OutTestCredentialsFromIni;

	GConfig->GetString(TEXT("OnlineSubsystemAutomation"), *TestAccountIniEntry, OutTestCredentialsFromIni, GEngineIni);

	if (!OutTestCredentialsFromIni.IsEmpty())
	{
		FString OutRemainder;

		OutTestCredentialsFromIni.Split(TEXT(":"), &OutRemainder, &OutUniqueUserId, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
	}

	return OutUniqueUserId;
}

FString FOnlineTestCommon::GetSubsystemTestAccountUniqueId(FName Subsystem)
{
	return GetSubsystemUniqueId(Subsystem, TEXT("TestAccountCredentials"));
}

FString FOnlineTestCommon::GetSubsystemFriendAccountUniqueId(FName Subsystem)
{
	return GetSubsystemUniqueId(Subsystem, TEXT("FriendAccountCredentials"));
}

void FOnlineTestCommon::SendInviteToFriendAccount(IOnlineIdentityPtr OI, IOnlineFriendsPtr OF, FName ST, const FDoneDelegate& TestDone)
{
	OnlineIdentity = OI;
	OnlineFriends = OF;
	SubsystemType = ST;

	AccountCredentials = FOnlineTestCommon::GetSubsystemTestAccountCredentials(SubsystemType);

	OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
	{
		FString FriendAccountIdString = FOnlineTestCommon::GetSubsystemFriendAccountUniqueId(SubsystemType);
		FUniqueNetIdPtr FriendAccountId = OnlineIdentity->CreateUniquePlayerId(FriendAccountIdString);

		OnlineFriends->SendInvite(0, *FriendAccountId, EFriendsLists::ToString(EFriendsLists::Default), FOnSendInviteComplete::CreateLambda([this, TestDone](int32 SendInviteLocalPlayerNum, bool bSendInviteWasSuccessful, const FUniqueNetId& SendInviteFriendId, const FString& SendInviteListName, const FString& SendInviteErrorStr)
		{
			OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
			OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, TestDone](int32 LoggedOutLocalUserNum, bool bLogoutWasSuccessful)
			{
				OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
				TestDone.Execute();
				Cleanup();
			}));

			OnlineIdentity->Logout(0);
		}));
	}));

	OnlineIdentity->Login(0, AccountCredentials);
}

void FOnlineTestCommon::AddFriendToTestAccount(IOnlineIdentityPtr OI, IOnlineFriendsPtr OF, FName ST, const FDoneDelegate& TestDone)
{
	OnlineIdentity = OI;
	OnlineFriends = OF;
	SubsystemType = ST;

	AccountCredentials = FOnlineTestCommon::GetSubsystemTestAccountCredentials(SubsystemType);

	OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalPlayerNumTestAccount, bool bLoginWasSuccessfulTestAccount, const FUniqueNetId& LoginUserIdTestAccount, const FString& LoginErrorTestAccount)
	{
		FString FriendAccountIdString = FOnlineTestCommon::GetSubsystemFriendAccountUniqueId(SubsystemType);
		FUniqueNetIdPtr FriendAccountId = OnlineIdentity->CreateUniquePlayerId(FriendAccountIdString);

		OnlineFriends->SendInvite(0, *FriendAccountId, EFriendsLists::ToString(EFriendsLists::Default), FOnSendInviteComplete::CreateLambda([this, TestDone](int32 SendInviteLocalPlayerNumTestAccount, bool bSendInviteWasSuccessfulTestAccount, const FUniqueNetId& SendInviteFriendIdTestAccount, const FString& SendInviteListNameTestAccount, const FString& SendInviteErrorStrTestAccount)
		{
			OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, TestDone](int32 LoggedOutLocalUserNumTestAccount, bool bLogoutWasSuccessfulTestAccount)
			{
				OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);

				OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalPlayerNumFriendAccount, bool bLoginWasSuccessfulFriendAccount, const FUniqueNetId& LoginUserIdFriendAccount, const FString& LoginErrorFriendAccount)
				{
					OnlineFriends->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateLambda([this, TestDone](int32 ReadFriendsListLocalPlayerNumFriendAccount, bool bReadFriendsListWasSuccessfulFriendAccount, const FString& ReadFriendsListListNameFriendAccount, const FString& ReadFriendsListErrorStrFriendAccount)
					{
						FString TestAccountIdString = FOnlineTestCommon::GetSubsystemTestAccountUniqueId(SubsystemType);
						FUniqueNetIdPtr TestAccountId = OnlineIdentity->CreateUniquePlayerId(TestAccountIdString);

						OnlineFriends->AcceptInvite(0, *TestAccountId, EFriendsLists::ToString(EFriendsLists::Default), FOnAcceptInviteComplete::CreateLambda([this, TestDone](int32 AcceptInviteLocalPlayerNumFriendAccount, bool bAcceptInviteWasSuccessfulFriendAccount, const FUniqueNetId& AcceptInviteFriendIdFriendAccount, const FString& AcceptInviteListNameFriendAccount, const FString& AcceptInviteErrorStrFriendAccount)
						{
							OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);

							OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, TestDone](int32 LoggedOutLocalUserNumFriendAccount, bool bLogoutWasSuccessfulFriendAccount)
							{
								TestDone.Execute();
								Cleanup();
							}));

							OnlineIdentity->Logout(0);
						}));
					}));
				}));

				FOnlineAccountCredentials FriendAccountCredentials = FOnlineTestCommon::GetSubsystemFriendAccountCredentials(SubsystemType);

				OnlineIdentity->Login(0, FriendAccountCredentials);
			}));

			OnlineIdentity->Logout(0);
		}));
	}));

	OnlineIdentity->Login(0, AccountCredentials);
}

void FOnlineTestCommon::RemoveFriendFromTestAccount(IOnlineIdentityPtr OI, IOnlineFriendsPtr OF, FName ST, const FDoneDelegate& TestDone)
{
	OnlineIdentity = OI;
	OnlineFriends = OF;
	SubsystemType = ST;

	AccountCredentials = FOnlineTestCommon::GetSubsystemTestAccountCredentials(SubsystemType);

	OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
	OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
	{
		OnlineFriends->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateLambda([this, TestDone](int32 ReadFriendsListLocalPlayerNum, bool bReadFriendsListWasSuccessful, const FString& ReadFriendsListListName, const FString& ReadFriendsListErrorStr)
		{
			TArray<TSharedRef<FOnlineFriend>> FriendsList;
			OnlineFriends->GetFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FriendsList);
			if (FriendsList.Num() > 0)
			{
				FString FriendAccountIdString = FOnlineTestCommon::GetSubsystemFriendAccountUniqueId(SubsystemType);
				FUniqueNetIdPtr FriendAccountId = OnlineIdentity->CreateUniquePlayerId(FriendAccountIdString);

				OnlineFriends->ClearOnDeleteFriendCompleteDelegate_Handle(0, OnDeleteFriendCompleteDelegateHandle);
				OnDeleteFriendCompleteDelegateHandle = OnlineFriends->AddOnDeleteFriendCompleteDelegate_Handle(0, FOnDeleteFriendCompleteDelegate::CreateLambda([this, TestDone](int32 DeleteFriendLocalUserNum, bool bDeleteFriendWasSuccessful, const FUniqueNetId& DeleteFriendFriendId, const FString& DeleteFriendListName, const FString& DeleteFriendErrorStr)
				{
					OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
					OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, TestDone](int32 LoggedOutLocalUserNum, bool bLogoutWasSuccessful)
					{
						TestDone.Execute();
						Cleanup();
					}));

					OnlineIdentity->Logout(0);
				}));

				OnlineFriends->DeleteFriend(0, *FriendAccountId, EFriendsLists::ToString(EFriendsLists::Default));
			}
			else
			{
				OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
				OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, TestDone](int32 LoggedOutLocalUserNum, bool bLogoutWasSuccessful)
				{
					TestDone.Execute();
					Cleanup();
				}));

				OnlineIdentity->Logout(0);
			}
		}));
	}));

	OnlineIdentity->Login(0, AccountCredentials);
}

void FOnlineTestCommon::RejectInviteOnFriendAccount(IOnlineIdentityPtr OI, IOnlineFriendsPtr OF, FName ST, const FDoneDelegate& TestDone)
{
	OnlineIdentity = OI;
	OnlineFriends = OF;
	SubsystemType = ST;

	FOnlineAccountCredentials FriendAccountCredentials = FOnlineTestCommon::GetSubsystemFriendAccountCredentials(SubsystemType);

	OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
	OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
	{
		OnlineFriends->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateLambda([this, TestDone](int32 ReadFriendsListLocalUserNum, bool bReadFriendsListWasSuccessful, const FString& ReadFriendsListListName, const FString& ReadFriendsListErrorStr)
		{
			TArray<TSharedRef<FOnlineFriend>> FriendsList;
			OnlineFriends->GetFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FriendsList);
			if (FriendsList.Num() > 0)
			{
				FString TestAccountUserIdString = FOnlineTestCommon::GetSubsystemTestAccountUniqueId(SubsystemType);
				FUniqueNetIdPtr TestAccountUserId = OnlineIdentity->CreateUniquePlayerId(TestAccountUserIdString);

				OnRejectInviteCompleteDelegateHandle = OnlineFriends->AddOnRejectInviteCompleteDelegate_Handle(0, FOnRejectInviteCompleteDelegate::CreateLambda([this, TestDone](int32 RejectInviteLocalUserNum, bool bRejectInviteWasSuccessful, const FUniqueNetId& RejectInviteFriendId, const FString& RejectInviteListName, const FString& RejectInviteErrorStr)
				{
					OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
					OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, TestDone](int32 LoggedOutLocalUserNum, bool bLogoutWasSuccessful)
					{
						OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
						TestDone.Execute();
						Cleanup();
					}));

					OnlineIdentity->Logout(0);
				}));

				OnlineFriends->RejectInvite(0, *TestAccountUserId, EFriendsLists::ToString(EFriendsLists::Default));
			}
			else
			{
				OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
				OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, TestDone](int32 LoggedOutLocalUserNum, bool bLogoutWasSuccessful)
				{
					TestDone.Execute();
					Cleanup();
				}));

				OnlineIdentity->Logout(0);
			}
		}));
	}));

	OnlineIdentity->Login(0, FriendAccountCredentials);
}

void FOnlineTestCommon::BlockFriendOnTestAccount(IOnlineIdentityPtr OI, IOnlineFriendsPtr OF, FName ST, const FDoneDelegate& TestDone)
{
	OnlineIdentity = OI;
	OnlineFriends = OF;
	SubsystemType = ST;

	AccountCredentials = FOnlineTestCommon::GetSubsystemTestAccountCredentials(SubsystemType);
	FOnlineAccountCredentials FriendAccountCredentials = FOnlineTestCommon::GetSubsystemFriendAccountCredentials(SubsystemType);

	OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
	OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalPlayerNumTestAccountAddFriend, bool bLoginWasSuccessfulTestAccountAddFriend, const FUniqueNetId& LoginUserIdTestAccountAddFriend, const FString& LoginErrorTestAccountAddFriend)
	{
		FString FriendAccountIdString = FOnlineTestCommon::GetSubsystemFriendAccountUniqueId(SubsystemType);
		FUniqueNetIdPtr FriendAccountId = OnlineIdentity->CreateUniquePlayerId(FriendAccountIdString);

		OnlineFriends->SendInvite(0, *FriendAccountId, EFriendsLists::ToString(EFriendsLists::Default), FOnSendInviteComplete::CreateLambda([this, FriendAccountId, TestDone](int32 SendInviteLocalPlayerNumTestAccountAddFriend, bool bSendInviteWasSuccessfulTestAccountAddFriend, const FUniqueNetId& SendInviteFriendIdTestAccountAddFriend, const FString& SendInviteListNameTestAccountAddFriend, const FString& SendInviteErrorStrTestAccountAddFriend)
		{
			OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, FriendAccountId, TestDone](int32 LoggedOutLocalUserNumTestAccountAddFriend, bool bLogoutWasSuccessfulTestAccountAddFriend)
			{
				OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);

				OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, FriendAccountId, TestDone](int32 LoginLocalPlayerNumFriendAccountAcceptInvite, bool bLoginWasSuccessfulFriendAccountAcceptInvite, const FUniqueNetId& LoginUserIdFriendAccountAcceptInvite, const FString& LoginErrorFriendAccountAcceptInvite)
				{
					OnlineFriends->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateLambda([this, FriendAccountId, TestDone](int32 ReadFriendsListLocalPlayerNumFriendAccountAcceptInvite, bool bReadFriendsListWasSuccessfulFriendAccountAcceptInvite, const FString& ReadFriendsListListNameFriendAccountAcceptInvite, const FString& ReadFriendsListErrorStrFriendAccountAcceptInvite)
					{
						FString TestAccountIdString = FOnlineTestCommon::GetSubsystemTestAccountUniqueId(SubsystemType);
						FUniqueNetIdPtr TestAccountId = OnlineIdentity->CreateUniquePlayerId(TestAccountIdString);

						OnlineFriends->AcceptInvite(0, *TestAccountId, EFriendsLists::ToString(EFriendsLists::Default), FOnAcceptInviteComplete::CreateLambda([this, FriendAccountId, TestDone](int32 AcceptInviteLocalPlayerNumFriendAccountAcceptInvite, bool bAcceptInviteWasSuccessfulFriendAccountAcceptInvite, const FUniqueNetId& AcceptInviteFriendIdFriendAccountAcceptInvite, const FString& AcceptInviteListNameFriendAccountAcceptInvite, const FString& AcceptInviteErrorStrFriendAccountAcceptInvite)
						{
							OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);

							OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, FriendAccountId, TestDone](int32 LoggedOutLocalUserNumFriendAccountAcceptInvite, bool bLogoutWasSuccessfulFriendAccountAcceptInvite)
							{
								OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
								OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, FriendAccountId, TestDone](int32 LoginLocalPlayerNumTestAccountBlockFriend, bool bLoginWasSuccessfulTestAccountBlockFriend, const FUniqueNetId& LoginUserIdTestAccountBlockFriend, const FString& LoginErrorTestAccountBlockFriend)
								{
									if (FriendAccountId.IsValid())
									{
										OnBlockedPlayerCompleteDelegateHandle = OnlineFriends->AddOnBlockedPlayerCompleteDelegate_Handle(0, FOnBlockedPlayerCompleteDelegate::CreateLambda([this, TestDone](int32 BlockedPlayerLocalUserNumTestAccountBlockFriend, bool bBlockedPlayerWasSuccessfulTestAccountBlockFriend, const FUniqueNetId& BlockedPlayerUniqueIDTestAccountBlockFriend, const FString& BlockedPlayerListNameTestAccountBlockFriend, const FString& BlockedPlayerErrorStrTestAccountBlockFriend)
										{
											OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
											OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, TestDone](int32 LoggedOutLocalUserNumTestAccountBlockFriend, bool bLogoutWasSuccessfulTestAccountBlockFriend)
											{
												TestDone.Execute();
												Cleanup();
											}));

											OnlineIdentity->Logout(0);
										}));

										OnlineFriends->BlockPlayer(0, *FriendAccountId);
									}
									else
									{
										UE_LOG_ONLINE(Error, TEXT("OSS Automation: IsValid() check on FriendAccountId failed after a call to OnlineIdentity->CreateUniquePlayerId()"));
										TestDone.Execute();
									}
								}));

								OnlineIdentity->Login(0, AccountCredentials);
							}));

							OnlineIdentity->Logout(0);
						}));
					}));
				}));

				FOnlineAccountCredentials FriendAccountLoginCallCredentials = FOnlineTestCommon::GetSubsystemFriendAccountCredentials(SubsystemType);

				OnlineIdentity->Login(0, FriendAccountLoginCallCredentials);
			}));

			OnlineIdentity->Logout(0);
		}));
	}));

	OnlineIdentity->Login(0, AccountCredentials);
}

void FOnlineTestCommon::UnblockFriendOnTestAccount(IOnlineIdentityPtr OI, IOnlineFriendsPtr OF, FName ST, const FDoneDelegate& TestDone)
{
	OnlineIdentity = OI;
	OnlineFriends = OF;
	SubsystemType = ST;

	AccountCredentials = FOnlineTestCommon::GetSubsystemTestAccountCredentials(SubsystemType);

	FOnlineAccountCredentials FriendAccountCredentials = FOnlineTestCommon::GetSubsystemFriendAccountCredentials(SubsystemType);

	OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(0, OnLoginCompleteDelegateHandle);
	OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalPlayerNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
	{
		FString FriendAccountIdString = FOnlineTestCommon::GetSubsystemFriendAccountUniqueId(SubsystemType);
		FUniqueNetIdPtr FriendAccountId = OnlineIdentity->CreateUniquePlayerId(FriendAccountIdString);

		if (FriendAccountId.IsValid())
		{
			OnUnblockedPlayerCompleteDelegateHandle = OnlineFriends->AddOnUnblockedPlayerCompleteDelegate_Handle(0, FOnBlockedPlayerCompleteDelegate::CreateLambda([this, TestDone](int32 UnblockedPlayerLocalUserNum, bool bUnblockedPlayerWasSuccessful, const FUniqueNetId& UnblockedPlayerUniqueID, const FString& UnblockedPlayerListName, const FString& UnblockedPlayerErrorStr)
			{
				// Also unfriend in case a previous test failed to block
				OnlineFriends->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateLambda([this, TestDone](int32 ReadFriendsListLocalPlayerNum, bool bReadFriendsListWasSuccessful, const FString& ReadFriendsListListName, const FString& ReadFriendsListErrorStr)
				{
					TArray<TSharedRef<FOnlineFriend>> FriendsList;
					OnlineFriends->GetFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FriendsList);
					if (FriendsList.Num() > 0)
					{
						FString DeletedFriendAccountIdString = FOnlineTestCommon::GetSubsystemFriendAccountUniqueId(SubsystemType);
						FUniqueNetIdPtr DeletedFriendAccountId = OnlineIdentity->CreateUniquePlayerId(DeletedFriendAccountIdString);

						OnlineFriends->ClearOnDeleteFriendCompleteDelegate_Handle(0, OnDeleteFriendCompleteDelegateHandle);
						OnDeleteFriendCompleteDelegateHandle = OnlineFriends->AddOnDeleteFriendCompleteDelegate_Handle(0, FOnDeleteFriendCompleteDelegate::CreateLambda([this, TestDone](int32 DeleteFriendLocalUserNum, bool bDeleteFriendWasSuccessful, const FUniqueNetId& DeleteFriendFriendId, const FString& DeleteFriendListName, const FString& DeleteFriendErrorStr)
						{
							OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
							OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, TestDone](int32 LoggedOutLocalUserNum, bool bLogoutWasSuccessful)
							{
								TestDone.Execute();
								Cleanup();
							}));

							OnlineIdentity->Logout(0);
						}));

						OnlineFriends->DeleteFriend(0, *DeletedFriendAccountId, EFriendsLists::ToString(EFriendsLists::Default));
					}
					else
					{
						OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
						OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, TestDone](int32 LoggedOutLocalUserNum, bool bLogoutWasSuccessful)
						{
							TestDone.Execute();
							Cleanup();
						}));

						OnlineIdentity->Logout(0);
					}
				}));
			}));

			OnlineFriends->UnblockPlayer(0, *FriendAccountId);
		}
		else
		{
			UE_LOG_ONLINE(Error, TEXT("OSS Automation: IsValid() check on FriendAccountId failed after a call to OnlineIdentity->CreateUniquePlayerId()"));
			TestDone.Execute();
		}
	}));

	OnlineIdentity->Login(0, AccountCredentials);
}

void FOnlineTestCommon::SendMessageToTestAccount(IOnlineIdentityPtr OI, IOnlineFriendsPtr OF, IOnlineMessagePtr OM, FName ST, const FDoneDelegate& TestDone)
{
	OnlineIdentity = OI;
	OnlineMessage = OM;
	SubsystemType = ST;

	AccountCredentials = FOnlineTestCommon::GetSubsystemTestAccountCredentials(SubsystemType);
	FOnlineAccountCredentials FriendAccountCredentials = FOnlineTestCommon::GetSubsystemFriendAccountCredentials(SubsystemType);

	OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalPlayerNumTestAccount, bool bLoginWasSuccessfulTestAccount, const FUniqueNetId& LoginUserIdTestAccount, const FString& LoginErrorTestAccount)
	{
		FString TestAccountIdString = FOnlineTestCommon::GetSubsystemTestAccountUniqueId(SubsystemType);
		FUniqueNetIdPtr TestAccountId = OnlineIdentity->CreateUniquePlayerId(TestAccountIdString);

		TArray<FUniqueNetIdRef> Recipients;
		Recipients.Add(TestAccountId.ToSharedRef());

		FOnlineMessagePayload TestPayload;
		TArray<uint8> TestData;
		TestData.Add(0xde);

		TestPayload.SetAttribute(TEXT("STRINGValue"), FVariantData(TestData));

		OnSendMessageCompleteDelegateHandle = OnlineMessage->AddOnSendMessageCompleteDelegate_Handle(0, FOnSendMessageCompleteDelegate::CreateLambda([this, TestDone](int32 SendMessageLocalUserNum, bool bSendMessageWasSuccessful, const FString& SendMessagePayload)
		{
			OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(0, OnLogoutCompleteDelegateHandle);
			OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, TestDone](int32 LoggedOutLocalUserNum, bool bLogoutWasSuccessful)
			{
				TestDone.Execute();
				Cleanup();
			}));

			OnlineIdentity->Logout(0);
		}));

		OnlineMessage->SendMessage(0, Recipients, TEXT("TEST"), TestPayload);
	}));

	OnlineIdentity->Login(0, FriendAccountCredentials);
}

void FOnlineTestCommon::AddAchievementToTestAccount(IOnlineIdentityPtr OI, IOnlineAchievementsPtr OA, const FDoneDelegate& TestDone)
{
	OnlineIdentity = OI;
	OnlineAchievements = OA;

	OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
	{
		FUniqueNetIdPtr TestAccountId = OnlineIdentity->GetUniquePlayerId(0);

		if (TestAccountId.IsValid())
		{
			OnlineAchievements->QueryAchievements(*TestAccountId, FOnQueryAchievementsCompleteDelegate::CreateLambda([this, TestAccountId, TestDone](const FUniqueNetId& QueryAchievementsPlayerId, const bool bQueryAchievementsWasSuccessful)
			{
				TArray<FOnlineAchievement> PlayerAchievements;
				OnlineAchievements->GetCachedAchievements(QueryAchievementsPlayerId, PlayerAchievements);

				if (PlayerAchievements.Num() > 0)
				{
					const FString TestAchievement = PlayerAchievements[0].Id;

					FOnlineAchievement SomeAchievement;
					OnlineAchievements->GetCachedAchievement(*TestAccountId, TestAchievement, SomeAchievement);

					FOnlineAchievementsWritePtr AchievementWriteObject = MakeShareable(new FOnlineAchievementsWrite());
					FOnlineAchievementsWriteRef AchievementWriter = AchievementWriteObject.ToSharedRef();
					AchievementWriteObject->SetFloatStat(TestAchievement, 1.0f);

					OnlineAchievements->WriteAchievements(*TestAccountId, AchievementWriter, FOnAchievementsWrittenDelegate::CreateLambda([this, AchievementWriteObject, TestAccountId, TestAchievement, TestDone](const FUniqueNetId& WriteAchievements, bool bWriteAchievementsWasSuccessful)
					{
						OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(0, FOnLogoutCompleteDelegate::CreateLambda([this, TestDone](int32 LoggedOutLocalUserNum, bool bLogoutWasSuccessful)
						{
							TestDone.Execute();
							Cleanup();
						}));

						OnlineIdentity->Logout(0);
					}));
				}
				else
				{
					UE_LOG_ONLINE(Error, TEXT("OSS Automation: PlayerAchievements array is empty after a call to GetCachedAchievements. No Achievements found for this subsystem."));
				}
			}));
		}
		else
		{
			UE_LOG_ONLINE(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->GetUniquePlayerId()"));
		}
	}));

	OnlineIdentity->Login(0, AccountCredentials);
}

void FOnlineTestCommon::ResetTestAccountAchievements(IOnlineIdentityPtr OI, IOnlineAchievementsPtr OA, const FDoneDelegate& TestDone)
{
	OnlineIdentity = OI;
	OnlineAchievements = OA;

	OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(0, FOnLoginCompleteDelegate::CreateLambda([this, TestDone](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
	{
		FUniqueNetIdPtr TestAccountId = OnlineIdentity->GetUniquePlayerId(0);

		if (TestAccountId.IsValid())
		{
#if !UE_BUILD_SHIPPING
			OnlineAchievements->ResetAchievements(*TestAccountId);
#endif //!UE_BUILD_SHIPPING
		}
		else
		{
			UE_LOG_ONLINE(Error, TEXT("OSS Automation: IsValid() check on TestAccountId failed after a call to OnlineIdentity->CreateUniquePlayerId()"));
		}

		TestDone.Execute();
		Cleanup();
	}));

	OnlineIdentity->Login(0, AccountCredentials);
}
