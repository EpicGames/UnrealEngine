// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "OnlineSubsystem.h"
#include "Interfaces/OnlineIdentityInterface.h"
#include "Interfaces/OnlineFriendsInterface.h"
#include "Misc/AutomationTest.h"

#define UE_API ONLINESUBSYSTEM_API

class FOnlineTestCommon
{
public:

	/**
	 * Constructor
	 */
	UE_API FOnlineTestCommon();

private:

	/**
	 * Cleans up this object's delegates and pointers to the online interfaces
	 */
	void Cleanup();

	/**
	 * Gets valid account credentials to use for testing this subsystem defined in DefaultEngine.ini
	 * KeyName takes on the form of: <ConfigPrefix>AccountCredentials<Subsystem>
	 * An empty FOnlineAccountCredentials object is returned if a invalid value is given.
	 * Ex:
	 *  [OnlineSubsystemAutomation]
	 *  TestAccountCredentialsPS4=username@epicgames.com:testing1:ps4:a609c6b855a619198cca7d35fc439347
	 *
	 * @param Subsystem the subsystem for the account credentials (e.g. Steam, IOS)
	 * @param ConfigPrefix the prefix for the type of account wanted (e.g. Test, Friend)
	 *
	 * @return FOnlineAccountCredentials from the DefaultEngine.ini
	 */
	static FOnlineAccountCredentials GetSubsystemCredentials(FName Subsystem, FString ConfigPrefix);

	/**
	 * Gets valid account UniqueId to use for testing this subsystem defined in DefaultEngine.ini
	 * KeyName takes on the form of: <ConfigPrefix>AccountCredentials<Subsystem>
	 * An empty FString is returned if a invalid value is given.
	 * Ex:
	 *  [OnlineSubsystemAutomation]
	 *  TestAccountCredentialsPS4=username@epicgames.com:testing1:ps4:a609c6b855a619198cca7d35fc439347
	 *
	 * @param Subsystem the subsystem for the account credentials (e.g. Steam, IOS)
	 * @param ConfigPrefix the prefix for the type of account wanted (e.g. Test, Friend)
	 *
	 * @return FString version of the account's UniqueId in the DefaultEngine.ini
	 */
	static FString GetSubsystemUniqueId(FName Subsystem, FString ConfigPrefix);

public:

	/**
	 * Gets valid Test account credentials to use for testing this subsystem defined in DefaultEngine.ini
	 * KeyName takes on the form of: TestAccountCredentials<Subsystem>
	 * An empty FOnlineAccountCredentials object is returned if a invalid value is given.
	 * Ex:
	 *  [OnlineSubsystemAutomation]
	 *  TestAccountCredentialsPS4=username@epicgames.com:testing1:ps4:a609c6b855a619198cca7d35fc439347
	 *
	 * @param Subsystem the subsystem for the account credentials (e.g. Steam, IOS)
	 *
	 * @return FOnlineAccountCredentials from the DefaultEngine.ini
	 */
	static UE_API FOnlineAccountCredentials GetSubsystemTestAccountCredentials(FName Subsystem);

	/**
	 * Gets valid Friend account credentials to use for testing this subsystem defined in DefaultEngine.ini
	 * KeyName takes on the form of: FriendAccountCredentials<Subsystem>
	 * An empty FOnlineAccountCredentials object is returned if a invalid value is given.
	 * Ex:
	 *  [OnlineSubsystemAutomation]
	 *  FriendAccountCredentialsPS4=username@epicgames.com:testing1:ps4:a609c6b855a619198cca7d35fc439347
	 *
	 * @param Subsystem the subsystem for the account credentials (e.g. Steam, IOS)
	 *
	 * @return FOnlineAccountCredentials from the DefaultEngine.ini
	 */
	static UE_API FOnlineAccountCredentials GetSubsystemFriendAccountCredentials(FName Subsystem);

	/**
	* Gets valid Test account UniqueId to use for testing this subsystem defined in DefaultEngine.ini
	* KeyName takes on the form of: <ConfigPrefix>AccountCredentials<Subsystem>
	* An empty FString is returned if a invalid value is given.
	* Ex:
	*  [OnlineSubsystemAutomation]
	*  TestAccountCredentialsPS4=username@epicgames.com:testing1:ps4:a609c6b855a619198cca7d35fc439347
	*
	* @param Subsystem the subsystem for the account credentials (e.g. Steam, IOS)
	*
	* @return FString version of the account's UniqueId in the DefaultEngine.ini
	*/
	static UE_API FString GetSubsystemTestAccountUniqueId(FName Subsystem);

	/**
	* Gets valid Friend account UniqueId to use for testing this subsystem defined in DefaultEngine.ini
	* KeyName takes on the form of: <ConfigPrefix>AccountCredentials<Subsystem>
	* An empty FString is returned if a invalid value is given.
	* Ex:
	*  [OnlineSubsystemAutomation]
	*  FriendAccountCredentialsPS4=username@epicgames.com:testing1:ps4:a609c6b855a619198cca7d35fc439347
	*
	* @param Subsystem the subsystem for the account credentials (e.g. Steam, IOS)
	*
	* @return FString version of the account's UniqueId in the DefaultEngine.ini
	*/
	static UE_API FString GetSubsystemFriendAccountUniqueId(FName Subsystem);

	/**
	 * Gets the list of subsystems defined in DefaultEngine.ini
	 *	Ex:
	 *	[OnlineSubsystemAutomation]
	 *	EnabledTestSubsystem=STEAM
	 *	+EnabledTestSubsystem=PS4
	 */
	static UE_API TArray<FName> GetEnabledTestSubsystems();

	UE_API void SendInviteToFriendAccount(IOnlineIdentityPtr OI, IOnlineFriendsPtr OF, FName ST, const FDoneDelegate& TestDone);

	/**
	 * Logs into the Test account defined in DefaultEngine.ini (TestAccountCredentials) and adds
	 * the Friend account defined in DefaultEngine.ini (FriendAccountCredentials) as a friend
	 *
	 * @param OI a pointer to the OnlineIdentity interface
	 * @param OF a pointer to the OnlineFriends interface
	 * @param ST the subsystem being tested
	 * @param TestDone the done delegate to fire and complete the test/BeforeEach/AfterEach
	 */
	UE_API void AddFriendToTestAccount(IOnlineIdentityPtr OI, IOnlineFriendsPtr OF, FName ST, const FDoneDelegate& TestDone);

	/**
	 * Logs into the Test account defined in DefaultEngine.ini (TestAccountCredentials) and removes
	 * the Friend account defined in DefaultEngine.ini (FriendAccountCredentials) as a friend
	 *
	 * @param OI a pointer to the OnlineIdentity interface
	 * @param OF a pointer to the OnlineFriends interface
	 * @param ST the subsystem being tested
	 * @param TestDone the done delegate to fire and complete the test/BeforeEach/AfterEach
	 */
	UE_API void RemoveFriendFromTestAccount(IOnlineIdentityPtr OI, IOnlineFriendsPtr OF, FName ST, const FDoneDelegate& TestDone);

	/**
	 * Logs into the Friend account defined in DefaultEngine.ini (FriendAccountCredentials) and rejects
	 * the pending invite from the test account
	 *
	 * @param OI a pointer to the OnlineIdentity interface
	 * @param OF a pointer to the OnlineFriends interface
	 * @param ST the subsystem being tested
	 * @param TestDone the done delegate to fire and complete the test/BeforeEach/AfterEach
	 */
	UE_API void RejectInviteOnFriendAccount(IOnlineIdentityPtr OI, IOnlineFriendsPtr OF, FName ST, const FDoneDelegate& TestDone);

	UE_API void BlockFriendOnTestAccount(IOnlineIdentityPtr OI, IOnlineFriendsPtr OF, FName ST, const FDoneDelegate& TestDone);

	UE_API void UnblockFriendOnTestAccount(IOnlineIdentityPtr OI, IOnlineFriendsPtr OF, FName ST, const FDoneDelegate& TestDone);

	UE_API void SendMessageToTestAccount(IOnlineIdentityPtr OI, IOnlineFriendsPtr OF, IOnlineMessagePtr OM, FName ST, const FDoneDelegate& TestDone);

	UE_API void AddAchievementToTestAccount(IOnlineIdentityPtr OI, IOnlineAchievementsPtr OA, const FDoneDelegate& TestDone);

	UE_API void ResetTestAccountAchievements(IOnlineIdentityPtr OI, IOnlineAchievementsPtr OA, const FDoneDelegate& TestDone);

public:

	FName SubsystemType;

	FOnlineAccountCredentials AccountCredentials;

	IOnlineIdentityPtr OnlineIdentity;
	IOnlineFriendsPtr OnlineFriends;
	IOnlineMessagePtr OnlineMessage;
	IOnlineAchievementsPtr OnlineAchievements;

	FDelegateHandle OnLogoutCompleteDelegateHandle;
	FDelegateHandle	OnLoginCompleteDelegateHandle;
	FDelegateHandle OnDeleteFriendCompleteDelegateHandle;
	FDelegateHandle OnInviteAcceptedDelegateHandle;
	FDelegateHandle OnRejectInviteCompleteDelegateHandle;
	FDelegateHandle OnBlockedPlayerCompleteDelegateHandle;
	FDelegateHandle OnUnblockedPlayerCompleteDelegateHandle;
	FDelegateHandle OnSendMessageCompleteDelegateHandle;
};

#undef UE_API
